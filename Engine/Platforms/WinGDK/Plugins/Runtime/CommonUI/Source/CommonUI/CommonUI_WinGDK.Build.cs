// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CommonUI_WinGDK : CommonUI
{
	public CommonUI_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
		
    }
}
