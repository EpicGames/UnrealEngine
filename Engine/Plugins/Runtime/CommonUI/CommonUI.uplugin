{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Common UI Plugin",
	"Description" : "A repository for game independent UI elements.",
	"Category" : "UI",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CommonUI",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name": "CommonUIEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name" : "CommonInput",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"		
		}
	],
	"Plugins":
	[
		{
			"Name": "EngineAssetDefinitions",
			"Enabled": true
		},
		{
			"Name": "GameplayTagsEditor",
			"Enabled": true
		},
		{
			"Name": "EnhancedInput",
			"Enabled": true
		}
	]
}