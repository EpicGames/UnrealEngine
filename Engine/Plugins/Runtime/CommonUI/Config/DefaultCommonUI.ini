[CoreRedirects]
+ClassRedirects=(OldName="CommonUIInputData",NewName="/Script/CommonInput.CommonUIInputData")
+ClassRedirects=(OldName="CommonUIKeyBrushDisplayData",NewName="/Script/CommonInput.CommonInputBaseControllerData")
+ClassRedirects=(OldName="CommonInputControllerData",NewName="/Script/CommonInput.CommonInputBaseControllerData")