[/Script/CommonUI.CommonUIInputSettings]
DefaultVirtualPointerClass=/CommonUI/WBP_VirtualPointer.WBP_VirtualPointer_C
