// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CommonInput : ModuleRules
{
	public CommonInput(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "InputCore",
				"EnhancedInput",
				"DeveloperSettings"
            }
		);

		PrivateDependencyModuleNames.AddRange(
		new string[]
			{
                "SlateCore",
				"Slate",
				"ApplicationCore",
                "EngineSettings"
            }
		);

        if (Target.Type == TargetType.Editor)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
                    "UnrealEd",
                }
            );
        }

		string CommonUIPlatform = ToCommonUIPlatform(Target.Platform);
		if (!string.IsNullOrEmpty(CommonUIPlatform))
		{
			PublicDefinitions.Add("UE_COMMONINPUT_PLATFORM_TYPE = " + CommonUIPlatform);
		}

		PrivateDefinitions.Add("UE_COMMONINPUT_PLATFORM_KBM_REQUIRES_ATTACHED_MOUSE=" + (bPlatformKBMRequiresAttachedMouse ? "1" : "0"));
		PrivateDefinitions.Add("UE_COMMONINPUT_FORCE_TOUCH_SUPPORT_DISABLED=" + (bForceTouchSupportDisabled ? "1" : "0"));
	}

	static public string ToCommonUIPlatform(UnrealTargetPlatform TargetPlatform)
	{
		if (TargetPlatform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			return "PC";
		}
		else if (TargetPlatform == UnrealTargetPlatform.Mac)
		{
			return "Mac";
		}
		else if (TargetPlatform == UnrealTargetPlatform.Linux)
		{
			return "PC";
		}
		else if (TargetPlatform == UnrealTargetPlatform.IOS)
		{
			return "IOS";
		}
		else if (TargetPlatform == UnrealTargetPlatform.Android)
		{
			return "Android";
		}

		return string.Empty;
	}

	protected virtual bool bPlatformKBMRequiresAttachedMouse { get { return false; } }
	protected virtual bool bForceTouchSupportDisabled { get { return false; } }
}
