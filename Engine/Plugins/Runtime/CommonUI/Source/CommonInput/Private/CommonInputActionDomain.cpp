// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonInputActionDomain.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonInputActionDomain)

DEFINE_LOG_CATEGORY(LogUIActionDomain);

bool UCommonInputActionDomain::ShouldBreakInnerEventFlow(bool bInputEventHandled) const
{
	switch (InnerBehavior)
	{
		case ECommonInputEventFlowBehavior::BlockIfActive:
		{
			return true;
		}
		case ECommonInputEventFlowBehavior::BlockIfHandled:
		{
			return bInputEventHandled;
		}
		case ECommonInputEventFlowBehavior::NeverBlock:
		{
			return false;
		}
	}

	return false;
}

bool UCommonInputActionDomain::ShouldBreakEventFlow(bool bDomainHadActiveRoots, bool bInputEventHandledAtLeastOnce) const
{
	switch (Behavior)
	{
		case ECommonInputEventFlowBehavior::BlockIfActive:
		{
			return bDomainHadActiveRoots;
		}
		case ECommonInputEventFlowBehavior::BlockIfHandled:
		{
			return bInputEventHandledAtLeastOnce;
		}
		case ECommonInputEventFlowBehavior::NeverBlock:
		{
			return false;
		}
	}

	return false;
}

