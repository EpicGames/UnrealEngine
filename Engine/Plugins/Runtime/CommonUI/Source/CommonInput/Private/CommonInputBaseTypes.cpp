// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonInputBaseTypes.h"
#include "Engine/UserInterfaceSettings.h"
#include "CommonInputTypeEnum.h"
#include "ICommonInputModule.h"
#include "Engine/PlatformSettings.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "InputCoreTypes.h"
#include "UObject/ObjectSaveContext.h"
#include "Styling/SlateBrush.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonInputBaseTypes)

const FName FCommonInputDefaults::PlatformPC(TEXT("PC"));
const FName FCommonInputDefaults::GamepadGeneric(TEXT("Generic"));

FCommonInputKeyBrushConfiguration::FCommonInputKeyBrushConfiguration()
{
	KeyBrush.DrawAs = ESlateBrushDrawType::Image;
}

FCommonInputKeySetBrushConfiguration::FCommonInputKeySetBrushConfiguration()
{
	KeyBrush.DrawAs = ESlateBrushDrawType::Image;
}

namespace CommonUIUtils
{
	bool TryGetInputBrushFromDataMap(FSlateBrush& OutBrush, const FKey& InKey, const TArray<FCommonInputKeyBrushConfiguration>& InInputBrushDataMap)
	{
		const FCommonInputKeyBrushConfiguration* DisplayConfig = InInputBrushDataMap.FindByPredicate([&InKey](const FCommonInputKeyBrushConfiguration& KeyBrushPair) -> bool
		{
			return KeyBrushPair.Key.IsSameResolvedKey(InKey);
		});

		if (DisplayConfig)
		{
			OutBrush = DisplayConfig->GetInputBrush();
			return true;
		}

		return false;
	}

	bool TryGetInputBrushFromKeySets(FSlateBrush& OutBrush, const TArray<FKey>& InKeys, const TArray<FCommonInputKeySetBrushConfiguration>& InInputBrushKeySets)
	{

		const FCommonInputKeySetBrushConfiguration* DisplayConfig = InInputBrushKeySets.FindByPredicate([&InKeys](const FCommonInputKeySetBrushConfiguration& KeyBrushPair) -> bool
		{
			if (KeyBrushPair.Keys.Num() < 2)
			{
				return false;
			}

			if (InKeys.Num() == KeyBrushPair.Keys.Num())
			{
				for (const FKey& Key : InKeys)
				{
					const bool bContainsKey = KeyBrushPair.Keys.ContainsByPredicate([Key](const FKey& BrushKey)
					{
						return BrushKey.IsSameResolvedKey(Key);
					});
					if (!bContainsKey)
					{
						return false;
					}
				}

				return true;
			}

			return false;
		});

		if (DisplayConfig)
		{
			OutBrush = DisplayConfig->GetInputBrush();
			return true;
		}

		return false;
	}
}

bool UCommonUIInputData::NeedsLoadForServer() const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	return UISettings->bLoadWidgetsOnDedicatedServer;
}

bool UCommonInputBaseControllerData::NeedsLoadForServer() const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	return UISettings->bLoadWidgetsOnDedicatedServer;
}

bool UCommonInputBaseControllerData::TryGetInputBrush(FSlateBrush& OutBrush, const FKey& Key) const
{
	return CommonUIUtils::TryGetInputBrushFromDataMap(OutBrush, Key, InputBrushDataMap);
}

bool UCommonInputBaseControllerData::TryGetInputBrush(FSlateBrush& OutBrush, const TArray<FKey>& Keys) const
{
	if (Keys.Num() == 0)
	{
		return false;
	}

	if (Keys.Num() == 1)
	{
		return CommonUIUtils::TryGetInputBrushFromDataMap(OutBrush, Keys[0], InputBrushDataMap);
	}

	return CommonUIUtils::TryGetInputBrushFromKeySets(OutBrush, Keys, InputBrushKeySets);
}

void UCommonInputBaseControllerData::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (!ObjectSaveContext.IsProceduralSave())
	{
		// These have been organized by a human already, better to sort using this array.
		TArray<FKey> AllKeys;
		EKeys::GetAllKeys(AllKeys);

		// Organize the keys so they're nice and clean
		InputBrushDataMap.Sort([&AllKeys](const FCommonInputKeyBrushConfiguration& A, const FCommonInputKeyBrushConfiguration& B) {
			return AllKeys.IndexOfByKey(A.Key) < AllKeys.IndexOfByKey(B.Key);
		});

		// Delete any brush data where we have no image assigned
		InputBrushDataMap.RemoveAll([](const FCommonInputKeyBrushConfiguration& A) {
			return A.GetInputBrush().GetResourceObject() == nullptr;
		});
	}
}

void UCommonInputBaseControllerData::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Have to clear it even though it's transient because it's saved into the CDO.
	SetButtonImageHeightTo = 0;
#endif
}

const TArray<FName>& UCommonInputBaseControllerData::GetRegisteredGamepads()
{
	auto GenerateRegisteredGamepads = []()
	{
		TArray<FName> RegisteredGamepads;
		RegisteredGamepads.Add(FCommonInputDefaults::GamepadGeneric);

		for (const TPair<FName, FDataDrivenPlatformInfo>& Platform : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			const FName PlatformName = Platform.Key;
			const FDataDrivenPlatformInfo& PlatformInfo = Platform.Value;

			// Don't add fake platforms that are used to group real platforms to make configuration for groups of platforms
			// simpler.
			if (PlatformInfo.bIsFakePlatform)
			{
				continue;
			}

			// If the platform uses the standard keyboard for default input, ignore it, all of those platforms will use "PC"
			// as their target, so Windows, Linux, but not Mac.
			if (PlatformInfo.bDefaultInputStandardKeyboard)
			{
				continue;
			}

			// Only add platforms with dedicated gamepads.
			if (PlatformInfo.bHasDedicatedGamepad)
			{
				RegisteredGamepads.Add(PlatformName);
			}
		}
		return RegisteredGamepads;
	};
	static TArray<FName> RegisteredGamepads = GenerateRegisteredGamepads();
	return RegisteredGamepads;
}

#if WITH_EDITOR
void UCommonInputBaseControllerData::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UCommonInputBaseControllerData, SetButtonImageHeightTo))
		{
			if (SetButtonImageHeightTo != 0)
			{
				for (FCommonInputKeyBrushConfiguration& BrushConfig : InputBrushDataMap)
				{
					FVector2D NewBrushSize = BrushConfig.KeyBrush.GetImageSize();
					if (NewBrushSize.X != 0 && NewBrushSize.Y != 0)
					{
						NewBrushSize.X = FMath::RoundToInt(SetButtonImageHeightTo * (NewBrushSize.X / NewBrushSize.Y));
						NewBrushSize.Y = SetButtonImageHeightTo;

						BrushConfig.KeyBrush.SetImageSize(NewBrushSize);
					}
				}

				for (FCommonInputKeySetBrushConfiguration& BrushConfig : InputBrushKeySets)
				{
					FVector2D NewBrushSize = BrushConfig.KeyBrush.GetImageSize();
					if (NewBrushSize.X != 0 && NewBrushSize.Y != 0)
					{
						NewBrushSize.X = FMath::RoundToInt(SetButtonImageHeightTo * (NewBrushSize.X / NewBrushSize.Y));
						NewBrushSize.Y = SetButtonImageHeightTo;

						BrushConfig.KeyBrush.SetImageSize(NewBrushSize);
					}
				}
			}

			SetButtonImageHeightTo = 0;
		}
	}
}
#endif




UCommonInputPlatformSettings::UCommonInputPlatformSettings()
{
	DefaultInputType = ECommonInputType::Gamepad;
	bSupportsMouseAndKeyboard = false;
	bSupportsGamepad = true;
	bCanChangeGamepadType = true;
	bSupportsTouch = false;
	DefaultGamepadName = FCommonInputDefaults::GamepadGeneric;
}

void UCommonInputPlatformSettings::PostLoad()
{
	Super::PostLoad();

	ControllerDataClasses.Reset();
	InitializeControllerData();
}

void UCommonInputPlatformSettings::InitializeControllerData() const
{
	if (ControllerData.Num() != ControllerDataClasses.Num())
	{
		ControllerDataClasses.Reset(ControllerData.Num());
		for (TSoftClassPtr<UCommonInputBaseControllerData> ControllerDataPtr : ControllerData)
		{
			if (TSubclassOf<UCommonInputBaseControllerData> ControllerDataClass = ControllerDataPtr.LoadSynchronous())
			{
				ControllerDataClasses.Add(ControllerDataClass);
			}
		}
	}
}

void UCommonInputPlatformSettings::InitializePlatformDefaults()
{
	const FName PlatformName = GetPlatformIniName();
	const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName);
	if (PlatformInfo.DefaultInputType == "Gamepad")
	{
		DefaultInputType = ECommonInputType::Gamepad;
	}
	else if (PlatformInfo.DefaultInputType == "Touch")
	{
		DefaultInputType = ECommonInputType::Touch;
	}
	else if (PlatformInfo.DefaultInputType == "MouseAndKeyboard")
	{
		DefaultInputType = ECommonInputType::MouseAndKeyboard;
	}

	bSupportsMouseAndKeyboard = PlatformInfo.bSupportsMouseAndKeyboard;
	bSupportsGamepad = PlatformInfo.bSupportsGamepad;
	bCanChangeGamepadType = PlatformInfo.bCanChangeGamepadType;
	bSupportsTouch = PlatformInfo.bSupportsTouch;

	DefaultGamepadName = PlatformName;
}

bool UCommonInputPlatformSettings::TryGetInputBrush(FSlateBrush& OutBrush, FKey Key, ECommonInputType InputType, const FName GamepadName) const
{
	InitializeControllerData();

	for (const UCommonInputBaseControllerData* DefaultControllerData: GetControllerDataForInputType(InputType, GamepadName))
	{
		if (DefaultControllerData->TryGetInputBrush(OutBrush, Key))
		{
			return true;
		}
	}

	return false;
}

bool UCommonInputPlatformSettings::TryGetInputBrush(FSlateBrush& OutBrush, const TArray<FKey>& Keys, ECommonInputType InputType, const FName GamepadName) const
{
	InitializeControllerData();

	for (const UCommonInputBaseControllerData* DefaultControllerData: GetControllerDataForInputType(InputType, GamepadName))
	{
		if (DefaultControllerData->TryGetInputBrush(OutBrush, Keys))
		{
			return true;
		}
	}

	return false;
}

TArray<const UCommonInputBaseControllerData*> UCommonInputPlatformSettings::GetControllerDataForInputType(ECommonInputType InputType, const FName GamepadName) const
{
	InitializeControllerData();

	TArray<const UCommonInputBaseControllerData*> ControllerDataForInputType;
	for (const TSubclassOf<UCommonInputBaseControllerData>& ControllerDataPtr : ControllerDataClasses)
	{
		const UCommonInputBaseControllerData* DefaultControllerData = ControllerDataPtr.GetDefaultObject();
		if (DefaultControllerData && DefaultControllerData->InputType == InputType)
		{
			if (DefaultControllerData->InputType != ECommonInputType::Gamepad || DefaultControllerData->GamepadName == GamepadName)
			{
				ControllerDataForInputType.Add(DefaultControllerData);
			}
		}
	}

	return ControllerDataForInputType;
}

void UCommonInputPlatformSettings::AddControllerDataEntry(TSoftClassPtr<UCommonInputBaseControllerData> Entry)
{
	if (ensure(!Entry.IsNull()))
	{
		ControllerData.AddUnique(Entry);
		if (ControllerData.Num() == (ControllerDataClasses.Num() + 1))
		{
			if (TSubclassOf<UCommonInputBaseControllerData> ControllerDataClass = Entry.LoadSynchronous())
			{
				ControllerDataClasses.Add(ControllerDataClass);
			}
		}
		else
		{
			InitializeControllerData();
		}
	}
}

void UCommonInputPlatformSettings::RemoveControllerDataEntry(TSoftClassPtr<UCommonInputBaseControllerData> Entry)
{
	ControllerData.RemoveAllSwap([&Entry](const TSoftClassPtr<UCommonInputBaseControllerData>& ControllerDataEntry) {
		return ControllerDataEntry.GetUniqueID() == Entry.GetUniqueID();
	});
	ControllerDataClasses.RemoveAllSwap([&Entry](const TSubclassOf<UCommonInputBaseControllerData>& ControllerDataClassEntry){
		return ControllerDataClassEntry.Get() == Entry.GetUniqueID();
	});
}

FName UCommonInputPlatformSettings::GetBestGamepadNameForHardware(FName CurrentGamepadName, FName InputDeviceName, const FString& HardwareDeviceIdentifier)
{
	InitializeControllerData();

	FName FirstMatch = NAME_None;

	// This is far more complicated than it should be because XInput exposes no information about device type,
	// so we want to be 'sticky', only switching to an Xbox controller if you don't already have one selected
	// and otherwise conserving the player UI chosen choice
	if (ControllerDataClasses.Num() > 0)
	{
		for (const TSubclassOf<UCommonInputBaseControllerData>& ControllerDataPtr : ControllerDataClasses)
		{
			if (const UCommonInputBaseControllerData* DefaultControllerData = ControllerDataPtr.GetDefaultObject())
			{
				bool bThisEntryMatches = false;
				for (const FInputDeviceIdentifierPair& Pair : DefaultControllerData->GamepadHardwareIdMapping)
				{
					if ((Pair.InputDeviceName == InputDeviceName) && (Pair.HardwareDeviceIdentifier == HardwareDeviceIdentifier))
					{
						bThisEntryMatches = true;
						break;
					}
				}

				if (bThisEntryMatches)
				{
					if (CurrentGamepadName == DefaultControllerData->GamepadName)
					{
						// Preferentially conserve the existing setting
						return CurrentGamepadName;
					}

					if (FirstMatch == NAME_None)
					{
						// Record the first match, which we'll use if the existing one doesn't work
						FirstMatch = DefaultControllerData->GamepadName;
					}
				}
			}

		}
	}

	return FirstMatch.IsNone() ? CurrentGamepadName : FirstMatch;
}

bool UCommonInputPlatformSettings::SupportsInputType(ECommonInputType InputType) const
{
	switch (InputType)
	{
	case ECommonInputType::MouseAndKeyboard:
	{
		return bSupportsMouseAndKeyboard;
	}
	case ECommonInputType::Gamepad:
	{
		return bSupportsGamepad;
	}
	case ECommonInputType::Touch:
	{
		return bSupportsTouch;
	}
	}
	return false;
}

#if WITH_EDITOR
void UCommonInputPlatformSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	ControllerDataClasses.Reset();
}
#endif

void UCommonInputPlatformSettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	ControllerDataClasses.Reset();
}

bool FCommonInputPlatformBaseData::TryGetInputBrush(FSlateBrush& OutBrush, FKey Key, ECommonInputType InputType, const FName& GamepadName) const
{
	if (ControllerDataClasses.Num() > 0)
	{
		for (const TSubclassOf<UCommonInputBaseControllerData>& ControllerDataPtr : ControllerDataClasses)
		{
			const UCommonInputBaseControllerData* DefaultControllerData = ControllerDataPtr.GetDefaultObject();
			if (DefaultControllerData && DefaultControllerData->InputType == InputType)
			{
				if (DefaultControllerData->InputType != ECommonInputType::Gamepad || DefaultControllerData->GamepadName == GamepadName)
				{
					return DefaultControllerData->TryGetInputBrush(OutBrush, Key);
				}
			}
		}
	}

	return false;
}

bool FCommonInputPlatformBaseData::TryGetInputBrush(FSlateBrush& OutBrush, const TArray<FKey>& Keys, ECommonInputType InputType, const FName& GamepadName) const
{
	if (ControllerDataClasses.Num() > 0)
	{
		for (const TSubclassOf<UCommonInputBaseControllerData>& ControllerDataPtr : ControllerDataClasses)
		{
			const UCommonInputBaseControllerData* DefaultControllerData = ControllerDataPtr.GetDefaultObject();
			if (DefaultControllerData && DefaultControllerData->InputType == InputType)
			{
				if (DefaultControllerData->InputType != ECommonInputType::Gamepad || DefaultControllerData->GamepadName == GamepadName)
				{
					return DefaultControllerData->TryGetInputBrush(OutBrush, Keys);
				}
			}
		}
	}

	return false;
}

const TArray<FName>& FCommonInputPlatformBaseData::GetRegisteredPlatforms()
{
	auto GenerateRegisteredPlatforms = []()
	{
		TArray<FName> RegisteredPlatforms;
		RegisteredPlatforms.Add(FCommonInputDefaults::PlatformPC);

		for (const TPair<FName, FDataDrivenPlatformInfo>& Platform : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			const FName PlatformName = Platform.Key;
			const FDataDrivenPlatformInfo& PlatformInfo = Platform.Value;

			// Don't add fake platforms that are used to group real platforms to make configuration for groups of platforms
			// simpler.
			if (PlatformInfo.bIsFakePlatform)
			{
				continue;
			}

			// If the platform uses the standard keyboard for default input, ignore it, all of those platforms will use "PC"
			// as their target, so Windows, Linux, but not Mac.
			if (PlatformInfo.bDefaultInputStandardKeyboard)
			{
				continue;
			}

			RegisteredPlatforms.Add(PlatformName);
		}

		return RegisteredPlatforms;
	};
	static TArray<FName> RegisteredPlatforms = GenerateRegisteredPlatforms();
	return RegisteredPlatforms;
}

FName FCommonInputBase::GetCurrentPlatformName()
{
#if defined(UE_COMMONINPUT_PLATFORM_TYPE)
	return FName(UE_STRINGIZE(UE_COMMONINPUT_PLATFORM_TYPE));
#else
	return FCommonInputDefaults::PlatformPC;
#endif
}

UCommonInputSettings* FCommonInputBase::GetInputSettings()
{
	return &ICommonInputModule::GetSettings();
}

void FCommonInputBase::GetCurrentPlatformDefaults(ECommonInputType& OutDefaultInputType, FName& OutDefaultGamepadName)
{
	OutDefaultInputType = UCommonInputPlatformSettings::Get()->GetDefaultInputType();
	OutDefaultGamepadName = UCommonInputPlatformSettings::Get()->GetDefaultGamepadName();
}

