// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonInputPrivate.h"
#include "CommonInputSettings.h"
#include "ICommonInputModule.h"
#include "UObject/Package.h"
#if WITH_EDITOR
#include "ISettingsModule.h"
#endif // WITH_EDITOR

/**
 * Implements the FCommonInputModule module.
 */
class FCommonInputModule
	: public ICommonInputModule
{
public:
	FCommonInputModule();
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

protected:
	virtual UCommonInputSettings* GetSettingsInstance() const override;

private:
	void CreateInputSettings();

	UCommonInputSettings* CommonInputSettings;
};


FCommonInputModule::FCommonInputModule()
{
	CommonInputSettings = nullptr;
}

void FCommonInputModule::StartupModule()
{
	CreateInputSettings();
}

void FCommonInputModule::CreateInputSettings()
{
	CommonInputSettings = NewObject<UCommonInputSettings>(GetTransientPackage(), UCommonInputSettings::StaticClass());
	check(CommonInputSettings);
	CommonInputSettings->AddToRoot();

#if WITH_EDITOR

	// Register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	//if (SettingsModule != nullptr)
	//{
	//	ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "CommonInput",
	//		NSLOCTEXT("CommonInputPlugin", "CommonInputSettingsName", "Common Input"),
	//		NSLOCTEXT("CommonInputPlugin", "CommonInputSettingsDescription", "Configure Common Input defaults."),
	//		CommonInputSettings
	//	);
	//}

#endif //WITH_EDITOR
}

void FCommonInputModule::ShutdownModule()
{
#if WITH_EDITOR
	// Unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "CommonInput");
	}
#endif //WITH_EDITOR

	if (!GExitPurge) // If GExitPurge Object is already gone
	{
		CommonInputSettings->RemoveFromRoot();
	}
	CommonInputSettings = nullptr;
}

UCommonInputSettings* FCommonInputModule::GetSettingsInstance() const
{
	return CommonInputSettings;
}

IMPLEMENT_MODULE(FCommonInputModule, CommonInput);

DEFINE_LOG_CATEGORY(LogCommonInput);
