// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonInputPreprocessor.h"
#include "CommonInputSubsystem.h"
#include "CommonInputPrivate.h"
#include "CommonInputTypeEnum.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "GameFramework/InputDeviceSubsystem.h"
#include "ICommonInputModule.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Engine/PlatformSettingsManager.h"
#include "Engine/World.h"
#include "EnhancedInputSubsystems.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformStackWalk.h"
#include "Widgets/SViewport.h"
#include "CommonInputSettings.h"
#include "ICommonInputModule.h"

#if WITH_EDITOR
#include "Settings/LevelEditorPlaySettings.h"
#endif

static const TAutoConsoleVariable<bool> CVarShouldMouseWheelAndGestureRefreshInputMethod(
	TEXT("CommonUI.ShouldMouseWheelAndGestureRefreshInputMethod"),
	true,
	TEXT("Controls whether Mouse Wheel and Gesture input events should update the current input method"));

FCommonInputPreprocessor::FCommonInputPreprocessor(UCommonInputSubsystem& InCommonInputSubsystem)
	: InputSubsystem(InCommonInputSubsystem)
	, bIgnoreNextMove(false)
{
	for (uint8 InputTypeIndex = 0; InputTypeIndex < (uint8)ECommonInputType::Count; InputTypeIndex++)
	{
		InputMethodPermissions[InputTypeIndex] = false;
	}
}

void FCommonInputPreprocessor::Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor)
{
}

bool FCommonInputPreprocessor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	const ECommonInputType InputType = GetInputType(InKeyEvent.GetKey());
	if (IsRelevantInput(SlateApp, InKeyEvent, InputType))
	{
		if (IsInputMethodBlocked(InputType))
		{
			return true;
		}

		RefreshCurrentInputMethod(InputType);
	}
	return false;
}

bool FCommonInputPreprocessor::HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent)
{
	const ECommonInputType InputType = GetInputType(InAnalogInputEvent.GetKey());
	return IsRelevantInput(SlateApp, InAnalogInputEvent, InputType) && IsInputMethodBlocked(InputType);
}

bool FCommonInputPreprocessor::HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent)
{
	const ECommonInputType InputType = GetInputType(InPointerEvent);
	if (IsRelevantInput(SlateApp, InPointerEvent, InputType))
	{
		if (bIgnoreNextMove)
		{
			bIgnoreNextMove = false;
		}
		else if (!InPointerEvent.GetCursorDelta().IsNearlyZero())
		{
			if (IsInputMethodBlocked(InputType))
			{
				return true;
			}

			RefreshCurrentInputMethod(InputType);
		}
	}
	
	return false;
}

bool FCommonInputPreprocessor::HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent)
{
	const ECommonInputType InputType = GetInputType(InPointerEvent);
	if (IsRelevantInput(SlateApp, InPointerEvent, InputType))
	{
		if (IsInputMethodBlocked(InputType))
		{
			return true;
		}
		RefreshCurrentInputMethod(InputType);
	}
	return false;
}

bool FCommonInputPreprocessor::HandleMouseButtonDoubleClickEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent)
{
	const ECommonInputType InputType = GetInputType(InPointerEvent);
	if (IsRelevantInput(SlateApp, InPointerEvent, InputType))
	{
		if (IsInputMethodBlocked(InputType))
		{
			return true;
		}
		RefreshCurrentInputMethod(InputType);
	}
	return false;
}

bool FCommonInputPreprocessor::HandleMouseWheelOrGestureEvent(FSlateApplication& SlateApp, const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent)
{
	const FPointerEvent& EventToConsider = InGestureEvent ? *InGestureEvent : InWheelEvent;
	const ECommonInputType InputType = GetInputType(EventToConsider);
	if (IsRelevantInput(SlateApp, EventToConsider, InputType))
	{
		if (IsInputMethodBlocked(InputType))
		{
			return true;
		}

		if (CVarShouldMouseWheelAndGestureRefreshInputMethod.GetValueOnGameThread())
		{
			RefreshCurrentInputMethod(InputType);
		}
	}
	return false;
}

void FCommonInputPreprocessor::SetInputTypeFilter(ECommonInputType InputType, FName InReason, bool InFilter)
{
	TMap<FName, bool>& Reasons = FilterInputTypeWithReasons[(uint8)InputType];
	Reasons.Add(InReason, InFilter);

	bool ComputedFilter = false;
	for (const auto& Entry : Reasons)
	{
		ComputedFilter |= Entry.Value;
	}

	InputMethodPermissions[(uint8)InputType] = ComputedFilter;
}

bool FCommonInputPreprocessor::IsInputMethodBlocked(ECommonInputType InputType) const
{
	return InputMethodPermissions[(uint8)InputType];
}

bool FCommonInputPreprocessor::IsRelevantInput(FSlateApplication& SlateApp, const FInputEvent& InputEvent, const ECommonInputType DesiredInputType)
{
#if WITH_EDITOR
	// If we're stopped at a breakpoint we need for this input preprocessor to just ignore all incoming input
	// because we're now doing stuff outside the game loop in the editor and it needs to not block all that.
	// This can happen if you suspend input while spawning a dialog and then hit another breakpoint and then
	// try and use the editor, you can suddenly be unable to do anything.
	if (GIntraFrameDebuggingGameThread)
	{
		return false;
	}
#endif
	
	if (SlateApp.IsActive() 
		|| SlateApp.GetHandleDeviceInputWhenApplicationNotActive() 
		|| (ICommonInputModule::GetSettings().GetAllowOutOfFocusDeviceInput() && DesiredInputType == ECommonInputType::Gamepad))
	{
		const ULocalPlayer& LocalPlayer = *InputSubsystem.GetLocalPlayerChecked();
		int32 ControllerId = LocalPlayer.GetControllerId();

#if WITH_EDITOR
		// PIE is a very special flower, especially when running two clients - we have two LocalPlayers with ControllerId 0
		// The editor has existing shenanigans for handling this scenario, so we're using those for now
		// Ultimately this would ideally be something the editor resolves at the SlateApplication level with a custom ISlateInputMapping or something
		GEngine->RemapGamepadControllerIdForPIE(LocalPlayer.ViewportClient, ControllerId);
#endif
		return ControllerId == InputEvent.GetUserIndex();
	}
	
	return false;
}

void FCommonInputPreprocessor::RefreshCurrentInputMethod(ECommonInputType InputMethod)
{
#if WITH_EDITOR
	// Lots of special-case fun for PIE - there are special scenarios wherein we want to ignore the update attempt
	const ULocalPlayer& LocalPlayer = *InputSubsystem.GetLocalPlayerChecked();
	bool bCanApplyInputMethodUpdate = false;
	if (LocalPlayer.ViewportClient)
	{
		TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(LocalPlayer.GetControllerId());
		if (ensure(SlateUser))
		{
			bool bRoutingGamepadToNextPlayer = false;
			if (!GetDefault<ULevelEditorPlaySettings>()->GetRouteGamepadToSecondWindow(bRoutingGamepadToNextPlayer))
			{
				// This looks strange - it's because the getter will set the output param based on the value of the setting, but return
				//	whether the setting actually matters. So we're making sure that even if the setting is true, we revert to false when it's irrelevant.
				bRoutingGamepadToNextPlayer = false;
			}

			if (SlateUser->IsWidgetInFocusPath(LocalPlayer.ViewportClient->GetGameViewportWidget()))
			{
				// Our owner's game viewport is in the focus path, which in a PIE scenario means we shouldn't
				// acknowledge gamepad input if it's being routed to another PIE client
				if (InputMethod != ECommonInputType::Gamepad || !bRoutingGamepadToNextPlayer)
				{
					bCanApplyInputMethodUpdate = true;
				}
			}
			else if (InputMethod == ECommonInputType::Gamepad)
			{
				bCanApplyInputMethodUpdate = bRoutingGamepadToNextPlayer;
			}
		}
	}
	if (!bCanApplyInputMethodUpdate)
	{
		return;
	}
#endif

	InputSubsystem.SetCurrentInputType(InputMethod);

	// Try to auto-detect the type of gamepad
	if (InputMethod == ECommonInputType::Gamepad
	    && ICommonInputModule::GetSettings().GetEnableAutomaticGamepadTypeDetection()
	    && UCommonInputPlatformSettings::Get()->CanChangeGamepadType())
	{
		const ULocalPlayer* LP = InputSubsystem.GetLocalPlayer<ULocalPlayer>();
		const UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get();
		
		if (LP && DeviceSubsystem)
		{
			const FHardwareDeviceIdentifier DeviceScope = DeviceSubsystem->GetMostRecentlyUsedHardwareDevice(LP->GetPlatformUserId());
			const FString HardwareDeviceIdentifierString = DeviceScope.HardwareDeviceIdentifier.ToString();
			if ((DeviceScope.InputClassName != LastSeenGamepadInputDeviceName) || (HardwareDeviceIdentifierString != LastSeenGamepadHardwareDeviceIdentifier))
			{
				LastSeenGamepadInputDeviceName = DeviceScope.InputClassName;
				LastSeenGamepadHardwareDeviceIdentifier = HardwareDeviceIdentifierString;

				const FName GamepadInputType = InputSubsystem.GetCurrentGamepadName();
				const FName BestGamepadType = UCommonInputPlatformSettings::Get()->GetBestGamepadNameForHardware(GamepadInputType, DeviceScope.InputClassName, HardwareDeviceIdentifierString);
				if (BestGamepadType != GamepadInputType)
				{
					UE_LOGF(LogCommonInput, Log, "UCommonInputSubsystem: Autodetect changed GamepadInputType to %ls", *BestGamepadType.ToString());
					InputSubsystem.SetGamepadInputType(BestGamepadType);
					OnGamepadChangeDetected.Broadcast(BestGamepadType);
				}
			}
		}
	}
}

ECommonInputType FCommonInputPreprocessor::GetInputType(const FKey& Key)
{
	// If the key is a gamepad key or if the key is a Click key (which simulates a click), we should be in Gamepad mode
	if (Key.IsGamepadKey() || InputSubsystem.GetIsGamepadSimulatedClick())
	{
		if (UCommonInputSubsystem::IsMobileGamepadKey(Key))
		{
			return ECommonInputType::Touch;
		}
		
		return ECommonInputType::Gamepad;
	}

	return ECommonInputType::MouseAndKeyboard;
}

ECommonInputType FCommonInputPreprocessor::GetInputType(const FPointerEvent& PointerEvent)
{
#if !UE_BUILD_SHIPPING
	const ULocalPlayer& LocalPlayer = *InputSubsystem.GetLocalPlayerChecked();
#endif

	if (PointerEvent.IsTouchEvent())
	{
		return ECommonInputType::Touch;
	}
	else if (InputSubsystem.GetIsGamepadSimulatedClick())
	{
		return ECommonInputType::Gamepad;
	}
#if !UE_BUILD_SHIPPING
	else if ((LocalPlayer.ViewportClient && LocalPlayer.ViewportClient->GetUseMouseForTouch()) || FSlateApplication::Get().IsFakingTouchEvents())
	{
		return ECommonInputType::Touch;
	}
#endif
	return ECommonInputType::MouseAndKeyboard;
}
