// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Dependencies
 *****************************************************************************/

#include "Styling/SlateStyle.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCommonInput, Log, All);