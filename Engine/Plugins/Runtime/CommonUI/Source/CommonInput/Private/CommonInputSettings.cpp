// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonInputSettings.h"

#include "ICommonInputModule.h"
#include "Engine/DataTable.h"
#include "Engine/PlatformSettingsManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonInputSettings)

#if !WITH_EDITOR
#include "CommonInputPrivate.h"
#endif

UCommonInputSettings::UCommonInputSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, bInputDataLoaded(false)
{
	PlatformInput.Initialize(UCommonInputPlatformSettings::StaticClass());
}

void UCommonInputSettings::LoadData()
{
	LoadInputData();
	LoadActionDomainTable();
}

#if WITH_EDITOR
void UCommonInputSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bInputDataLoaded = false;
	bActionDomainTableLoaded = false;
	LoadData();
}
#endif

void UCommonInputSettings::LoadInputData()
{
	if (!bInputDataLoaded)
	{
		// If we were created early enough to be disregarded by the GC (which we should be), then we need to 
		// add all of our members to the root set, since our hard reference to them is totally meaningless to the GC.
		const bool bIsDisregardForGC = GUObjectArray.IsDisregardForGC(this);
		
		InputDataClass = InputData.LoadSynchronous();
		if (InputDataClass)
		{
			if (bIsDisregardForGC)
			{
				InputDataClass->AddToRoot();
			}
		}
		
		//CurrentPlatform = CommonInputPlatformData[FCommonInputBase::GetCurrentPlatformName()];
		//for (TSoftClassPtr<UCommonInputBaseControllerData> ControllerData : CurrentPlatform.GetControllerData())
		//{
		//	if (TSubclassOf<UCommonInputBaseControllerData> ControllerDataClass = ControllerData.LoadSynchronous())
		//	{
		//		CurrentPlatform.ControllerDataClasses.Add(ControllerDataClass);
		//		if (bIsDisregardForGC)
		//		{
		//			ControllerDataClass->AddToRoot();
		//		}
		//	}
		//}
		bInputDataLoaded = true;
	}
}

void UCommonInputSettings::LoadActionDomainTable()
{
	if (!bActionDomainTableLoaded)
	{
		// If we were created early enough to be disregarded by the GC (which we should be), then we need to 
		// add all of our members to the root set, since our hard reference to them is totally meaningless to the GC.
		const bool bIsDisregardForGC = GUObjectArray.IsDisregardForGC(this);

		ActionDomainTablePtr = ActionDomainTable.LoadSynchronous();
		if (ActionDomainTablePtr)
		{
			if (bIsDisregardForGC)
			{
				ActionDomainTablePtr->AddToRoot();
			}
		}

		bActionDomainTableLoaded = true;
	}
}

void UCommonInputSettings::ValidateData()
{
    bInputDataLoaded &= !InputData.IsPending();
  //  for (TSoftClassPtr<UCommonInputBaseControllerData> ControllerData : CurrentPlatform.GetControllerData())
  //  {
		//bInputDataLoaded &= CurrentPlatform.ControllerDataClasses.ContainsByPredicate([&ControllerData](const TSubclassOf<UCommonInputBaseControllerData>& ControllerDataClass)
		//	{
		//		return ControllerDataClass.Get() == ControllerData.Get();
		//	});

  //      bInputDataLoaded &= !ControllerData.IsPending();
  //  }
 
#if !WITH_EDITOR
    UE_CLOGF(!bInputDataLoaded, LogCommonInput, Warning, "Trying to access unloaded CommmonInputSettings data. This may force a sync load.");
#endif // !WITH_EDITOR

    LoadData();
}

FDataTableRowHandle UCommonInputSettings::GetDefaultClickAction() const
{
	ensure(bInputDataLoaded);

	if (InputDataClass)
	{
		if (const UCommonUIInputData* InputDataPtr = InputDataClass.GetDefaultObject())
		{
			return InputDataPtr->DefaultClickAction;
		}
	}
	return FDataTableRowHandle();
}

FDataTableRowHandle UCommonInputSettings::GetDefaultBackAction() const
{
	ensure(bInputDataLoaded);

	if (InputDataClass)
	{
		if (const UCommonUIInputData* InputDataPtr = InputDataClass.GetDefaultObject())
		{
			return InputDataPtr->DefaultBackAction;
		}
	}
	return FDataTableRowHandle();
}

TSubclassOf<UCommonUIHoldData> UCommonInputSettings::GetDefaultHoldData() const
{
	if (InputDataClass)
	{
		if (const UCommonUIInputData* InputDataPtr = InputDataClass.GetDefaultObject())
		{
			TSubclassOf<UCommonUIHoldData> DefaultHoldDataLoaded = InputDataPtr->DefaultHoldData.LoadSynchronous();
			return DefaultHoldDataLoaded;
		}
	}
	return nullptr;
}

UInputAction* UCommonInputSettings::GetEnhancedInputClickAction() const
{
	ensureMsgf(bInputDataLoaded, TEXT("Warning, CommonUI input data not loaded."));

	if (InputDataClass && IsEnhancedInputSupportEnabled())
	{
		return InputDataClass.GetDefaultObject()->EnhancedInputClickAction;
	}
	return nullptr;
}

UInputAction* UCommonInputSettings::GetEnhancedInputBackAction() const
{
	ensureMsgf(bInputDataLoaded, TEXT("Warning, CommonUI input data not loaded."));

	if (InputDataClass && IsEnhancedInputSupportEnabled())
	{
		return InputDataClass.GetDefaultObject()->EnhancedInputBackAction;
	}
	return nullptr;
}

bool UCommonInputSettings::IsEnhancedInputSupportEnabled() 
{
	static bool bEnabled = ICommonInputModule::Get().GetSettings().GetEnableEnhancedInputSupport();
	return bEnabled;
};

void UCommonInputSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (CommonInputPlatformData_DEPRECATED.Num())
	{
		for (const auto& PlatformData : CommonInputPlatformData_DEPRECATED)
		{
			const FCommonInputPlatformBaseData& OriginalData = PlatformData.Value;

			if (UCommonInputPlatformSettings* Settings = UPlatformSettingsManager::Get().GetSettingsForPlatform<UCommonInputPlatformSettings>(PlatformData.Key))
			{
				Settings->bSupportsMouseAndKeyboard = OriginalData.bSupportsMouseAndKeyboard;
				Settings->bSupportsGamepad = OriginalData.bSupportsGamepad;
				Settings->bSupportsTouch = OriginalData.bSupportsTouch;
				Settings->bCanChangeGamepadType = OriginalData.bCanChangeGamepadType;
				Settings->DefaultGamepadName = OriginalData.DefaultGamepadName;
				Settings->DefaultInputType = OriginalData.DefaultInputType;
				Settings->ControllerData = OriginalData.ControllerData;

				Settings->TryUpdateDefaultConfigFile();
			}
			else if (PlatformData.Key == FCommonInputDefaults::PlatformPC)
			{
				TArray<UCommonInputPlatformSettings*> PCPlatforms;
				PCPlatforms.Add(UPlatformSettingsManager::Get().GetSettingsForPlatform<UCommonInputPlatformSettings>("Windows"));
				PCPlatforms.Add(UPlatformSettingsManager::Get().GetSettingsForPlatform<UCommonInputPlatformSettings>("WinGDK"));
				PCPlatforms.Add(UPlatformSettingsManager::Get().GetSettingsForPlatform<UCommonInputPlatformSettings>("Linux"));

				for (UCommonInputPlatformSettings* PCPlatform : PCPlatforms)
				{
					if (PCPlatform)
					{
						PCPlatform->bSupportsMouseAndKeyboard = OriginalData.bSupportsMouseAndKeyboard;
						PCPlatform->bSupportsGamepad = OriginalData.bSupportsGamepad;
						PCPlatform->bSupportsTouch = OriginalData.bSupportsTouch;
						PCPlatform->bCanChangeGamepadType = OriginalData.bCanChangeGamepadType;
						PCPlatform->DefaultGamepadName = OriginalData.DefaultGamepadName;
						PCPlatform->DefaultInputType = OriginalData.DefaultInputType;
						PCPlatform->ControllerData = OriginalData.ControllerData;

						PCPlatform->TryUpdateDefaultConfigFile();
					}
				}
			}
		}

		CommonInputPlatformData_DEPRECATED.Reset();
		TryUpdateDefaultConfigFile();
	}
#endif
}

