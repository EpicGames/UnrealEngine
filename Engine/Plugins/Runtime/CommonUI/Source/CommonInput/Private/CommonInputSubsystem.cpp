// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonInputSubsystem.h"
#include "CommonInputPreprocessor.h"
#include "CommonInputPrivate.h"
#include "CommonInputTypeEnum.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Engine/PlatformSettingsManager.h"
#include "Engine/World.h"
#include "EnhancedInputSubsystems.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformStackWalk.h"
#include "Widgets/SViewport.h"
#include "CommonInputSettings.h"
#include "ICommonInputModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonInputSubsystem)

#if !UE_BUILD_SHIPPING
static int32 bDumpInputTypeChangeCallstack = 0;
static FAutoConsoleVariableRef CVarDumpInputTypeChangeCallstack(
	TEXT("CommonUI.bDumpInputTypeChangeCallstack"),
	bDumpInputTypeChangeCallstack,
	TEXT("Dump callstack when input type changes."));
#endif // !UE_BUILD_SHIPPING


FPlatformInputSupportOverrideDelegate UCommonInputSubsystem::OnPlatformInputSupportOverride;

//////////////////////////////////////////////////////////////////////////
// UCommonInputSubsystem
//////////////////////////////////////////////////////////////////////////

static int32 GCommonInputKeysVisible = 1;
static FAutoConsoleVariableRef CVarInputKeysVisible
(
	TEXT("CommonInput.ShowKeys"),
	GCommonInputKeysVisible,
	TEXT("Should we show the keys for the current input device."),
	ECVF_Default
);

bool bEnableGamepadPlatformCursor = false;
static const FAutoConsoleVariableRef CVarInputEnableGamepadPlatformCursor
(
	TEXT("CommonInput.EnableGamepadPlatformCursor"),
	bEnableGamepadPlatformCursor,
	TEXT("Should the cursor be allowed to be used during gamepad input")
);

UCommonInputSubsystem* UCommonInputSubsystem::Get(const ULocalPlayer* LocalPlayer)
{
	return LocalPlayer ? LocalPlayer->GetSubsystem<UCommonInputSubsystem>() : nullptr;
}

UCommonInputSubsystem::UCommonInputSubsystem()
{

}

void UCommonInputSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// There is a dependency on the Enhanced Input subsystem below so we need to make sure it is available
	// in a packaged game
	Collection.InitializeDependency<UEnhancedInputLocalPlayerSubsystem>();

	FCommonInputBase::GetInputSettings()->LoadData();

	const UCommonInputPlatformSettings* Settings = UPlatformSettingsManager::Get().GetSettingsForPlatform<UCommonInputPlatformSettings>();

	GamepadInputType = Settings->GetDefaultGamepadName();
	RawInputType = Settings->GetDefaultInputType();
	CurrentInputType = RawInputType;

	CommonInputPreprocessor = MakeInputProcessor();
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RegisterInputPreProcessor(CommonInputPreprocessor, EInputPreProcessorType::PreGame);
	}

	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCommonInputSubsystem::Tick), 0.1f);

	CVarInputKeysVisible->SetOnChangedCallback(FConsoleVariableDelegate::CreateUObject(this, &UCommonInputSubsystem::ShouldShowInputKeysChanged));

	SetActionDomainTable(FCommonInputBase::GetInputSettings()->GetActionDomainTable());

	if (FSlateApplication::IsInitialized())
	{
		if (GenericApplication* GenericApplication = FSlateApplication::Get().GetPlatformApplication().Get())
		{
			if (!GenericApplication->OnVirtualKeyboardShown().IsBoundToObject(this))
			{
				GenericApplication->OnVirtualKeyboardShown().AddUObject(this, &ThisClass::HandleOnVirtualKeyboardShown);
			}

			if (!GenericApplication->OnVirtualKeyboardHidden().IsBoundToObject(this))
			{
				GenericApplication->OnVirtualKeyboardHidden().AddUObject(this, &ThisClass::HandleOnVirtualKeyboardHidden);
			}
		}
	}
}

void UCommonInputSubsystem::Deinitialize()
{
	Super::Deinitialize();
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(CommonInputPreprocessor);
	}
	CommonInputPreprocessor.Reset();

	FTSTicker::RemoveTicker(TickHandle);

	if (FSlateApplication::IsInitialized())
	{
		if (GenericApplication* GenericApplication = FSlateApplication::Get().GetPlatformApplication().Get())
		{
			GenericApplication->OnVirtualKeyboardShown().RemoveAll(this);
			GenericApplication->OnVirtualKeyboardHidden().RemoveAll(this);
		}
	}
}

FGamepadChangeDetectedEvent& UCommonInputSubsystem::GetOnGamepadChangeDetected()
{
	return CommonInputPreprocessor->OnGamepadChangeDetected;
}

void UCommonInputSubsystem::SetInputTypeFilter(ECommonInputType InputType, FName Reason, bool Filter)
{
	CommonInputPreprocessor->SetInputTypeFilter(InputType, Reason, Filter);
}

bool UCommonInputSubsystem::GetInputTypeFilter(ECommonInputType InputType) const
{
	return CommonInputPreprocessor->IsInputMethodBlocked(InputType);
}

void UCommonInputSubsystem::AddOrRemoveInputTypeLock(FName InReason, ECommonInputType InInputType, bool bAddLock)
{
	// Make sure the input is supported before locking it, otherwise remove it if it exists
	if (bAddLock && PlatformSupportsInputType(InInputType))
	{
		ECommonInputType& CurrentValue = CurrentInputLocks.FindOrAdd(InReason);
		CurrentValue = InInputType;

		UE_LOGF(LogCommonInput, Log, "Adding Input Method Lock: %ls - %d", *InReason.ToString(), (int32)InInputType);
	}
	else
	{
		CurrentInputLocks.Remove(InReason);

		UE_LOGF(LogCommonInput, Log, "Removing Input Method Lock: %ls - %d", *InReason.ToString(), (int32)InInputType);
	}

	int32 ComputedInputLock = INDEX_NONE;
	for (const TPair<FName, ECommonInputType>& Entry : CurrentInputLocks)
	{
		// Take the most restrictive lock, e.g. Gamepad lock is more restrictive than a Keyboard/Mouse lock.
		if (((int32)Entry.Value) > ComputedInputLock)
		{
			ComputedInputLock = (int32)Entry.Value;
		}
	}

	if (ComputedInputLock == INDEX_NONE)
	{
		CurrentInputLock.Reset();
	}
	else
	{
		CurrentInputLock = (ECommonInputType)ComputedInputLock;
	}

	RecalculateCurrentInputType();
}

bool UCommonInputSubsystem::IsInputMethodActive(ECommonInputType InputMethod) const
{
	return GetCurrentInputType() == InputMethod;
}

bool UCommonInputSubsystem::HadAnyChangeOfInputMethodInTheLastThrashingWindow() const
{
	const UCommonInputSettings& InputSettings = ICommonInputModule::GetSettings();
	const double ChangeDelta = (FPlatformTime::Seconds() - LastInputMethodChangeTime);
	return ChangeDelta < InputSettings.GetInputMethodThrashingWindowInSeconds();
}

TSharedPtr<FCommonInputPreprocessor> UCommonInputSubsystem::MakeInputProcessor()
{
	return MakeShared<FCommonInputPreprocessor>(*this);
}

ECommonInputType UCommonInputSubsystem::LockInput(ECommonInputType InputToLock) const
{
	return CurrentInputLock.Get(InputToLock);
}

ECommonInputType UCommonInputSubsystem::GetCurrentInputType() const
{
	return CurrentInputType;
}

ECommonInputType UCommonInputSubsystem::GetDefaultInputType() const
{
	return UCommonInputPlatformSettings::Get()->GetDefaultInputType();
}

void UCommonInputSubsystem::BroadcastInputMethodChanged()
{
	if (UWorld* World = GetWorld())
	{
		if (!World->bIsTearingDown)
		{
			OnInputMethodChangedNative.Broadcast(CurrentInputType);
			OnInputMethodChanged.Broadcast(CurrentInputType);
			LastInputMethodChangeTime = FPlatformTime::Seconds();
		}
	}
}

bool UCommonInputSubsystem::CheckForInputMethodThrashing(ECommonInputType NewInputType)
{
	UCommonInputSettings& InputSettings = ICommonInputModule::GetSettings();


	if (InputSettings.GetEnableInputMethodThrashingProtection())
	{
		const double Now = FPlatformTime::Seconds();

		if (LastTimeInputMethodThrashingBegan + InputSettings.GetInputMethodThrashingCooldownInSeconds() > Now)
		{
			return true;
		}
		else if (CurrentInputLocks.Contains(TEXT("InputMethodThrashing")))
		{
			//Remove the thrashing lock.
			AddOrRemoveInputTypeLock(TEXT("InputMethodThrashing"), ECommonInputType::MouseAndKeyboard, false);
		}

		switch (NewInputType)
		{
		case ECommonInputType::Gamepad:
		case ECommonInputType::MouseAndKeyboard:
			break;
		default:
			// Ignore any thrashing that's not exclusively between mouse and gamepad.
			NumberOfInputMethodChangesRecently = 0;
			return false;
		}

		if (HadAnyChangeOfInputMethodInTheLastThrashingWindow())
		{
			NumberOfInputMethodChangesRecently++;
			if (NumberOfInputMethodChangesRecently > InputSettings.GetInputMethodThrashingLimit())
			{
				LastTimeInputMethodThrashingBegan = Now;
				//Add the thrashing lock
				AddOrRemoveInputTypeLock(TEXT("InputMethodThrashing"), ECommonInputType::MouseAndKeyboard, true);
				NumberOfInputMethodChangesRecently = 0;
				return true;
			}
		}
		else
		{
			NumberOfInputMethodChangesRecently = 0;
		}
	}
	return false;
}

void UCommonInputSubsystem::RecalculateCurrentInputType()
{
	ECommonInputType LockedInput = LockInput(RawInputType);

	if (LockedInput != CurrentInputType)
	{
#if !UE_BUILD_SHIPPING
		if (bDumpInputTypeChangeCallstack)
		{
			const uint32 DumpCallstackSize = 65535;
			ANSICHAR DumpCallstack[DumpCallstackSize] = { 0 };
			FString ScriptStack = FFrame::GetScriptCallstack(true /* bReturnEmpty */);
			FPlatformStackWalk::StackWalkAndDump(DumpCallstack, DumpCallstackSize, 0);
			UE_LOGF(LogCommonInput, Log, "--- Input Changing Callstack ---");
			UE_LOGF(LogCommonInput, Log, "Script Stack:\n%ls", *ScriptStack);
			UE_LOGF(LogCommonInput, Log, "Callstack:\n%ls", ANSI_TO_TCHAR(DumpCallstack));
		}
#endif // !UE_BUILD_SHIPPING
		
		CurrentInputType = LockedInput;

		FSlateApplication& SlateApplication = FSlateApplication::Get();
		ULocalPlayer* LocalPlayer = GetLocalPlayerChecked();
		bool bCursorUser = LocalPlayer && LocalPlayer->GetSlateUser() == SlateApplication.GetCursorUser();

		switch (CurrentInputType)
		{
		case ECommonInputType::Gamepad:
			UE_LOGF(LogCommonInput, Log, "UCommonInputSubsystem::RecalculateCurrentInputType(): Using Gamepad");
			if (bCursorUser)
			{
				SlateApplication.UsePlatformCursorForCursorUser(bEnableGamepadPlatformCursor);
			}
			SlateApplication.SetGameAllowsFakingTouchEvents(false);
			break;
		case ECommonInputType::Touch:
			UE_LOGF(LogCommonInput, Log, "UCommonInputSubsystem::RecalculateCurrentInputType(): Using Touch");
			SlateApplication.SetGameAllowsFakingTouchEvents(true);
			SlateApplication.SetGameIsFakingTouchEvents(LocalPlayer && LocalPlayer->ViewportClient && LocalPlayer->ViewportClient->GetUseMouseForTouch());
			break;
		case ECommonInputType::MouseAndKeyboard:
		default:				
			UE_LOGF(LogCommonInput, Log, "UCommonInputSubsystem::RecalculateCurrentInputType(): Using Mouse");
			if (bCursorUser)
			{
				SlateApplication.UsePlatformCursorForCursorUser(true);
			}
			SlateApplication.SetGameAllowsFakingTouchEvents(false);
			break;
		}

		BroadcastInputMethodChanged();
	}
}

void UCommonInputSubsystem::HandleOnVirtualKeyboardShown(FPlatformRect VkRect)
{
	if (FPlatformMisc::IsDeviceGamingHandheld() && CurrentInputType == ECommonInputType::Gamepad)
	{
		AddOrRemoveInputTypeLock(TEXT("HandheldVirtualKeyboard"), ECommonInputType::Gamepad, true);
	}
}

void UCommonInputSubsystem::HandleOnVirtualKeyboardHidden()
{
	if (FPlatformMisc::IsDeviceGamingHandheld())
	{
		AddOrRemoveInputTypeLock(TEXT("HandheldVirtualKeyboard"), ECommonInputType::Gamepad, false);
	}
}

void UCommonInputSubsystem::SetCurrentInputType(ECommonInputType NewInputType)
{
	if (((RawInputType != NewInputType) || bInputMethodLockedByThrashing) && PlatformSupportsInputType(NewInputType))
	{
		RawInputType = NewInputType;

		bInputMethodLockedByThrashing = CheckForInputMethodThrashing(NewInputType);
		if (!bInputMethodLockedByThrashing)
		{
			RecalculateCurrentInputType();
		}
	}
}

const FName UCommonInputSubsystem::GetCurrentGamepadName() const
{
	return GamepadInputType;
}
 
void UCommonInputSubsystem::SetGamepadInputType(const FName InGamepadInputType) 
{
	if (ensure(UCommonInputPlatformSettings::Get()->CanChangeGamepadType()))
	{
		GamepadInputType = InGamepadInputType;

		// Send out notifications so we update our buttons
		//BroadcastLastInputDeviceChanged();
		BroadcastInputMethodChanged();
	}
}

bool UCommonInputSubsystem::IsUsingPointerInput() const
{
	bool bUsingPointerInput = false;

	switch (RawInputType)
	{
		case ECommonInputType::MouseAndKeyboard:
		case ECommonInputType::Touch:
		{
			bUsingPointerInput = true;
		}
		break;

		case ECommonInputType::Gamepad:
		default:
		{
			bUsingPointerInput = false;
		}
		break;
	}

	return bUsingPointerInput;
}

bool UCommonInputSubsystem::ShouldShowInputKeys() const
{
	return GCommonInputKeysVisible != 0;
}

bool UCommonInputSubsystem::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UCommonInputSubsystem_Tick);
	
	return true; //repeat ticking
}

void UCommonInputSubsystem::ShouldShowInputKeysChanged(IConsoleVariable* Var)
{
	//BroadcastLastInputDeviceChanged();
	BroadcastInputMethodChanged();
}

bool UCommonInputSubsystem::PlatformSupportsHardwareCursor() const
{
#if PLATFORM_DESKTOP
	return true;
#else
	return false;
#endif
}

void UCommonInputSubsystem::SetCursorPosition(FVector2D NewPosition, bool bForce)
{
	if (CommonInputPreprocessor)
	{
		CommonInputPreprocessor->bIgnoreNextMove = true;
	}

	ULocalPlayer* LocalPlayer = GetLocalPlayerChecked();
	if (TSharedPtr<FSlateUser> SlateUser = LocalPlayer ? LocalPlayer->GetSlateUser() : nullptr)
	{
		UpdateCursorPosition(SlateUser.ToSharedRef(), NewPosition, bForce);
	}
}

void UCommonInputSubsystem::UpdateCursorPosition(TSharedRef<FSlateUser> SlateUser, const FVector2D& NewPosition, bool bForce)
{
	const FVector2D ClampedNewPosition = ClampPositionToViewport(NewPosition);

	//grab the old position
	const FVector2D OldPosition = SlateUser->GetCursorPosition();

	//make sure we are actually moving
	int32 NewIntPosX = ClampedNewPosition.X;
	int32 NewIntPosY = ClampedNewPosition.Y;
	int32 OldIntPosX = OldPosition.X;
	int32 OldIntPosY = OldPosition.Y;
	if (bForce || OldIntPosX != NewIntPosX || OldIntPosY != NewIntPosY)
	{
		//put the cursor in the correct spot
		SlateUser->SetCursorPosition(NewIntPosX, NewIntPosY);

		// Since the cursor may have been locked and its location clamped, get the actual new position
		const FVector2D UpdatedPosition = SlateUser->GetCursorPosition();

		FSlateApplication& SlateApp = FSlateApplication::Get();

		//create a new mouse event
		FPointerEvent MouseEvent(
			FSlateApplicationBase::CursorPointerIndex,
			UpdatedPosition,
			OldPosition,
			SlateApp.GetPressedMouseButtons(),
			EKeys::Invalid,
			0,
			SlateApp.GetPlatformApplication()->GetModifierKeys()
		);

		//process the event
		SlateApp.ProcessMouseMoveEvent(MouseEvent);
	}
}

bool UCommonInputSubsystem::GetIsGamepadSimulatedClick() const
{
	return bIsGamepadSimulatedClick;
}

void UCommonInputSubsystem::SetIsGamepadSimulatedClick(bool bNewIsGamepadSimulatedClick)
{
	if (bIsGamepadSimulatedClick != bNewIsGamepadSimulatedClick)
	{
		bIsGamepadSimulatedClick = bNewIsGamepadSimulatedClick;
		UE_CLOGF(bIsGamepadSimulatedClick, LogCommonInput, VeryVerbose, "UCommonInputSubsystem::SetIsGamepadSimulatedClick(): Click is being simulated");
	}
}

FVector2D UCommonInputSubsystem::ClampPositionToViewport(const FVector2D& InPosition) const
{
	UWorld* World = GetWorld();
	if (!World || !World->IsGameWorld())
	{
		return InPosition;
	}

	UGameViewportClient* GameViewport = World->GetGameViewport();
	if (!GameViewport || !GameViewport->Viewport || !GameViewport->GetWindow().IsValid())
	{
		return InPosition;
	}

	TSharedPtr<SViewport> GameViewportWidget = GameViewport->GetGameViewportWidget();
	if (GameViewportWidget.IsValid())
	{
		const FGeometry& ViewportGeometry = GameViewportWidget->GetCachedGeometry();
		FVector2D LocalPosition = ViewportGeometry.AbsoluteToLocal(InPosition);
		LocalPosition.X = FMath::Clamp(LocalPosition.X, 1.0f, ViewportGeometry.GetLocalSize().X - 1.0f);
		LocalPosition.Y = FMath::Clamp(LocalPosition.Y, 1.0f, ViewportGeometry.GetLocalSize().Y - 1.0f);

		return ViewportGeometry.LocalToAbsolute(LocalPosition);
	}

	return InPosition;
}

bool UCommonInputSubsystem::PlatformSupportsInputType(ECommonInputType InInputType) const
{
	bool bPlatformSupportsInput = UCommonInputPlatformSettings::Get()->SupportsInputType(InInputType);
	switch (InInputType)
	{
		case ECommonInputType::MouseAndKeyboard:
		{
#if UE_COMMONINPUT_PLATFORM_KBM_REQUIRES_ATTACHED_MOUSE
			bPlatformSupportsInput &= FSlateApplication::Get().IsMouseAttached();
#endif
		}
		break;
		case ECommonInputType::Touch:
		{
			bPlatformSupportsInput &= !UE_COMMONINPUT_FORCE_TOUCH_SUPPORT_DISABLED;
#if !UE_BUILD_SHIPPING
			// Support touch testing (testing with UseMouseForTouch setting enabled or with URemote in the editor) until touch is supported on desktop
			bPlatformSupportsInput = true;
#endif
		}
		break;
		case ECommonInputType::Gamepad:
#if PLATFORM_IOS
		{
			bool bAllowControllers = false;
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bAllowControllers"), bAllowControllers, GEngineIni);
			bPlatformSupportsInput &= bAllowControllers;
		}
#elif PLATFORM_ANDROID
		{
			bool bAllowControllers = false;
			GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bAllowControllers"), bAllowControllers, GEngineIni);
			bPlatformSupportsInput &= bAllowControllers;
		}
#endif
		break;
	}

	GetOnPlatformInputSupportOverride().Broadcast(GetLocalPlayer(), InInputType, bPlatformSupportsInput);
	
	return bPlatformSupportsInput;
}

bool UCommonInputSubsystem::IsMobileGamepadKey(const FKey& InKey)
{
	// Mobile keys that can be physically present on the device
	static TArray<FKey> PhysicalMobileKeys = { 
		EKeys::Android_Back, 
		EKeys::Android_Menu, 
		EKeys::Android_Volume_Down, 
		EKeys::Android_Volume_Up 
	};

	return PhysicalMobileKeys.Contains(InKey);
}

