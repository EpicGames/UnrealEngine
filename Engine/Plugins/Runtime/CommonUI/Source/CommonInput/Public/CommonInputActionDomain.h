// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "Engine/EngineBaseTypes.h"
#include "Types/ISlateMetaData.h"
#include "CommonInputModeTypes.h"

#include "CommonInputActionDomain.generated.h"

#define UE_API COMMONINPUT_API

class UCommonInputActionDomain;

DECLARE_LOG_CATEGORY_EXTERN(LogUIActionDomain, Log, All);

UENUM()
enum class ECommonInputEventFlowBehavior {
	BlockIfActive,
	BlockIfHandled,
	NeverBlock,
};

/** Slate meta data to store the owning Action Domain */
class ICommonInputActionDomainMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(ICommonInputActionDomainMetaData, ISlateMetaData);

	explicit ICommonInputActionDomainMetaData(const TWeakObjectPtr<UCommonInputActionDomain> InActionDomain)
		: ActionDomain(InActionDomain)
	{}

	TWeakObjectPtr<UCommonInputActionDomain> ActionDomain;
};

/**
 * Describes an input-event handling domain. It's InnerBehavior determines how events
 * flow between widgets within the domain and Behavior determines how events will flow to
 * other Domains in the DomainTable.
 */
UCLASS(MinimalAPI)
class UCommonInputActionDomain : public UDataAsset
{
	GENERATED_BODY()

public:
	// Behavior of an input event between Action Domains, i.e., how an event flows into the next Action Domain
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	ECommonInputEventFlowBehavior Behavior = ECommonInputEventFlowBehavior::BlockIfActive;

	// Behavior of an input event within an Action Domain, i.e., how an event flows to a lower ZOrder active widget
	// within the same Action Domain
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	ECommonInputEventFlowBehavior InnerBehavior = ECommonInputEventFlowBehavior::BlockIfHandled;

	UPROPERTY(EditDefaultsOnly, Category = "Default")
	bool bUseActionDomainDesiredInputConfig;

	UPROPERTY(EditDefaultsOnly, Category = "Default", meta = (EditCondition = "bUseActionDomainDesiredInputConfig"))
	ECommonInputMode InputMode = ECommonInputMode::Game;

	UPROPERTY(EditDefaultsOnly, Category = "Default", meta = (EditCondition = "bUseActionDomainDesiredInputConfig"))
	EMouseCaptureMode MouseCaptureMode = EMouseCaptureMode::CapturePermanently;

	UE_API bool ShouldBreakInnerEventFlow(bool bInputEventHandled) const;

	UE_API bool ShouldBreakEventFlow(bool bDomainHadActiveRoots, bool bInputEventHandledAtLeastOnce) const;
};

/**
 * An ordered array of ActionDomains.
 */
UCLASS(MinimalAPI)
class UCommonInputActionDomainTable : public UDataAsset
{
	GENERATED_BODY()

public:
	// Domains will receive events in ascending index order
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Default")
	TArray<TObjectPtr<UCommonInputActionDomain>> ActionDomains;

	UPROPERTY(EditDefaultsOnly, Category = "Default")
	ECommonInputMode InputMode = ECommonInputMode::Game;

	UPROPERTY(EditDefaultsOnly, Category = "Default")
	EMouseCaptureMode MouseCaptureMode = EMouseCaptureMode::CapturePermanently;

	UPROPERTY(EditDefaultsOnly, Category = "Default")
	bool bHideCursorDuringViewportCapture = true;
};

#undef UE_API
