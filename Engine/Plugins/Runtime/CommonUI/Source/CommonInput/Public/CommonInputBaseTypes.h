// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CommonInputTypeEnum.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/SubclassOf.h"
#include "InputCoreTypes.h"
#include "Engine/UserDefinedEnum.h"
#include "Templates/SharedPointer.h"
#include "Styling/SlateBrush.h"
#include "Engine/DataTable.h"
#include "UObject/SoftObjectPtr.h"
#include "Engine/PlatformSettings.h"
#include "InputAction.h"

#include "CommonInputBaseTypes.generated.h"

#define UE_API COMMONINPUT_API


class UCommonUIHoldData;
class UTexture2D;
class UMaterial;
class UCommonInputSettings; 

struct FCommonInputDefaults
{
	static UE_API const FName PlatformPC;
	static UE_API const FName GamepadGeneric;
};

USTRUCT(Blueprintable)
struct FCommonInputKeyBrushConfiguration
{
	GENERATED_BODY()

public:
	UE_API FCommonInputKeyBrushConfiguration();

	const FSlateBrush& GetInputBrush() const { return KeyBrush; }

public:
	UPROPERTY(EditAnywhere, Category = "Key Brush Configuration")
	FKey Key;

	UPROPERTY(EditAnywhere, Category = "Key Brush Configuration")
	FSlateBrush KeyBrush;
};

USTRUCT(Blueprintable)
struct FCommonInputKeySetBrushConfiguration
{
	GENERATED_BODY()

public:
	UE_API FCommonInputKeySetBrushConfiguration();

	const FSlateBrush& GetInputBrush() const { return KeyBrush; }

public:
	UPROPERTY(EditAnywhere, Category = "Key Brush Configuration", Meta = (TitleProperty = "KeyName"))
	TArray<FKey> Keys;

	UPROPERTY(EditAnywhere, Category = "Key Brush Configuration")
	FSlateBrush KeyBrush;
};

USTRUCT()
struct FInputDeviceIdentifierPair
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad")
	FName InputDeviceName;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad")
	FString HardwareDeviceIdentifier;
};

/* Data values needed for Hold interaction per input type. */
USTRUCT()
struct FInputHoldData
{
	GENERATED_BODY()
	UPROPERTY(EditDefaultsOnly, Category = "Properties", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float HoldTime = 0.0;
	
	UPROPERTY(EditDefaultsOnly, Category = "Properties", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float HoldRollbackTime = 0.0;
};

namespace CommonUIUtils
{
	COMMONINPUT_API bool TryGetInputBrushFromDataMap(FSlateBrush& OutBrush, const FKey& InKey, const TArray<FCommonInputKeyBrushConfiguration>& InInputBrushDataMap);
	COMMONINPUT_API bool TryGetInputBrushFromKeySets(FSlateBrush& OutBrush, const TArray<FKey>& InKeys, const TArray<FCommonInputKeySetBrushConfiguration>& InInputBrushKeySets);
}

/* Derive from this class to store the Input data. It is referenced in the Common Input Settings, found in the project settings UI. */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = Input, meta = (Category = "Common Input"))
class UCommonUIInputData : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual bool NeedsLoadForServer() const override;

public:
	UPROPERTY(EditDefaultsOnly, Category = "Properties", meta = (RowType = "/Script/CommonUI.CommonInputActionDataBase"))
	FDataTableRowHandle DefaultClickAction;

	UPROPERTY(EditDefaultsOnly, Category = "Properties", meta = (RowType = "/Script/CommonUI.CommonInputActionDataBase"))
	FDataTableRowHandle DefaultBackAction;

	/**
    * Newly created CommonButton widgets will use these hold values by default if bRequiresHold is true.
    * Inherits from UCommonUIHoldData.
    */
    UPROPERTY(EditDefaultsOnly, Category = "Properties")
    TSoftClassPtr<UCommonUIHoldData> DefaultHoldData;

	UPROPERTY(EditDefaultsOnly, Category = "Properties", meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	TObjectPtr<UInputAction> EnhancedInputClickAction;

	UPROPERTY(EditDefaultsOnly, Category = "Properties", meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	TObjectPtr<UInputAction> EnhancedInputBackAction;
};

/* Defines values for hold behavior per input type: for mouse Press and Hold, for gamepad focused Press and Hold, for touch Press and Hold. */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = Input, meta = (Category = "Common Input"))
class UCommonUIHoldData : public UObject
{
	GENERATED_BODY()
public:
	UCommonUIHoldData()
	{
		KeyboardAndMouse.HoldTime = 0.75f;
		KeyboardAndMouse.HoldRollbackTime = 0.0f;
		Gamepad.HoldTime = 0.75f;
		Gamepad.HoldRollbackTime = 0.0f;
		Touch.HoldTime = 0.75f;
		Touch.HoldRollbackTime = 0.0f;
	}
	
	UPROPERTY(EditDefaultsOnly, Category = "KeyboardAndMouse", meta = (ShowOnlyInnerProperties))
	FInputHoldData KeyboardAndMouse;
	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", meta = (ShowOnlyInnerProperties))
	FInputHoldData Gamepad;
	UPROPERTY(EditDefaultsOnly, Category = "Touch", meta = (ShowOnlyInnerProperties))
	FInputHoldData Touch;
};

/* Derive from this class to store the Input data. It is referenced in the Common Input Settings, found in the project settings UI. */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = Input, meta = (Category = "Common Input"))
class UCommonInputBaseControllerData : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual bool NeedsLoadForServer() const override;
	UE_API virtual bool TryGetInputBrush(FSlateBrush& OutBrush, const FKey& Key) const;
	UE_API virtual bool TryGetInputBrush(FSlateBrush& OutBrush, const TArray<FKey>& Keys) const;

	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void PostLoad() override;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, EditAnywhere, Category = "Editor")
	int32 SetButtonImageHeightTo = 0;
#endif

public:
	UPROPERTY(EditDefaultsOnly, Category = "Default")
	ECommonInputType InputType;
	
	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", meta=(EditCondition="InputType == ECommonInputType::Gamepad", GetOptions = GetRegisteredGamepads))
	FName GamepadName;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", meta = (EditCondition = "InputType == ECommonInputType::Gamepad"))
	FText GamepadDisplayName;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", meta=(EditCondition="InputType == ECommonInputType::Gamepad"))
	FText GamepadCategory;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", meta = (EditCondition = "InputType == ECommonInputType::Gamepad"))
	FText GamepadPlatformName;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", meta=(EditCondition="InputType == ECommonInputType::Gamepad"))
	TArray<FInputDeviceIdentifierPair> GamepadHardwareIdMapping;

	UPROPERTY(EditDefaultsOnly, Category = "Display")
	TSoftObjectPtr<UTexture2D> ControllerTexture;

	UPROPERTY(EditDefaultsOnly, Category = "Display")
	TSoftObjectPtr<UTexture2D> ControllerButtonMaskTexture;

	UPROPERTY(EditDefaultsOnly, Category = "Display", Meta = (TitleProperty = "Key"))
	TArray<FCommonInputKeyBrushConfiguration> InputBrushDataMap;

	UPROPERTY(EditDefaultsOnly, Category = "Display", Meta = (TitleProperty = "Keys"))
	TArray<FCommonInputKeySetBrushConfiguration> InputBrushKeySets;

	UFUNCTION()
	static UE_API const TArray<FName>& GetRegisteredGamepads();

private:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

UCLASS(MinimalAPI, config = Game, defaultconfig)
class UCommonInputPlatformSettings : public UPlatformSettings
{
	GENERATED_BODY()

	friend class UCommonInputSettings;

public:
	UE_API UCommonInputPlatformSettings();

	UE_API virtual void PostLoad() override;

	static UCommonInputPlatformSettings* Get()
	{
		return UPlatformSettingsManager::Get().GetSettingsForPlatform<UCommonInputPlatformSettings>();
	}

	UE_API bool TryGetInputBrush(FSlateBrush& OutBrush, FKey Key, ECommonInputType InputType, const FName GamepadName) const;
	UE_API bool TryGetInputBrush(FSlateBrush& OutBrush, const TArray<FKey>& Keys, ECommonInputType InputType, const FName GamepadName) const;

	TArray<TSoftClassPtr<UCommonInputBaseControllerData>> GetControllerData() const
	{
		return ControllerData;
	}
	UE_API TArray<const UCommonInputBaseControllerData*> GetControllerDataForInputType(ECommonInputType InputType, const FName GamepadName) const;
	UE_API void AddControllerDataEntry(TSoftClassPtr<UCommonInputBaseControllerData> Entry);
	UE_API void RemoveControllerDataEntry(TSoftClassPtr<UCommonInputBaseControllerData> Entry);

	UE_API FName GetBestGamepadNameForHardware(FName CurrentGamepadName, FName InputDeviceName, const FString& HardwareDeviceIdentifier); 

	ECommonInputType GetDefaultInputType() const
	{
		return DefaultInputType;
	}

	UE_API bool SupportsInputType(ECommonInputType InputType) const;

	const FName GetDefaultGamepadName() const
	{
		return DefaultGamepadName;
	}

	bool CanChangeGamepadType() const 
	{
		return bCanChangeGamepadType;
	}

#if WITH_EDITOR
	UE_API void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
#endif
	UE_API virtual void PostReloadConfig(FProperty* PropertyThatWasLoaded) override;

protected:
	UE_API void InitializeControllerData() const;
	UE_API virtual void InitializePlatformDefaults();

	UPROPERTY(config, EditAnywhere, Category = "Default")
	ECommonInputType DefaultInputType;

	UPROPERTY(config, EditAnywhere, Category = "Default")
	bool bSupportsMouseAndKeyboard;

	UPROPERTY(config, EditAnywhere, Category = "Default")
	bool bSupportsTouch;

	UPROPERTY(config, EditAnywhere, Category = "Default")
	bool bSupportsGamepad;

	UPROPERTY(config, EditAnywhere, Category = "Default", Meta = (EditCondition = "bSupportsGamepad"))
	FName DefaultGamepadName;

	UPROPERTY(config, EditAnywhere, Category = "Default", Meta = (EditCondition = "bSupportsGamepad"))
	bool bCanChangeGamepadType;

	UPROPERTY(config, EditAnywhere, Category = "Default", Meta = (TitleProperty = "InputType"))
	TArray<TSoftClassPtr<UCommonInputBaseControllerData>> ControllerData;

	UPROPERTY(Transient)
	mutable TArray<TSubclassOf<UCommonInputBaseControllerData>> ControllerDataClasses;
};

/* DEPRECATED Legacy! */
USTRUCT()
struct FCommonInputPlatformBaseData
{
	GENERATED_BODY()

	friend class UCommonInputSettings;

public:
	FCommonInputPlatformBaseData()
	{
		DefaultInputType = ECommonInputType::Gamepad;
		bSupportsMouseAndKeyboard = false;
		bSupportsGamepad = true;
		bCanChangeGamepadType = true;
		bSupportsTouch = false;
		DefaultGamepadName = FCommonInputDefaults::GamepadGeneric;
	}
	virtual ~FCommonInputPlatformBaseData() = default;

	UE_API virtual bool TryGetInputBrush(FSlateBrush& OutBrush, FKey Key, ECommonInputType InputType, const FName& GamepadName) const;
	UE_API virtual bool TryGetInputBrush(FSlateBrush& OutBrush, const TArray<FKey>& Keys, ECommonInputType InputType,  const FName& GamepadName) const;


	ECommonInputType GetDefaultInputType() const
	{
		return DefaultInputType;
	};

	bool SupportsInputType(ECommonInputType InputType) const 
	{
		switch (InputType)
		{
		case ECommonInputType::MouseAndKeyboard:
		{
			return bSupportsMouseAndKeyboard;
		}
		break;
		case ECommonInputType::Gamepad:
		{
			return bSupportsGamepad;
		}
		break;
		case ECommonInputType::Touch:
		{
			return bSupportsTouch;
		}
		break;
		}
		return false;
	}

	const FName GetDefaultGamepadName() const
	{
		return DefaultGamepadName;
	};

	bool CanChangeGamepadType() const 
	{
		return bCanChangeGamepadType;
	}

	TArray<TSoftClassPtr<UCommonInputBaseControllerData>> GetControllerData()
	{
		return ControllerData;
	}

	static UE_API const TArray<FName>& GetRegisteredPlatforms();

protected:
	UPROPERTY(EditDefaultsOnly, Category = "Properties")
	ECommonInputType DefaultInputType;

	UPROPERTY(EditDefaultsOnly, Category = "Properties")
	bool bSupportsMouseAndKeyboard;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad")
	bool bSupportsGamepad;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", Meta = (EditCondition = "bSupportsGamepad"))
	FName DefaultGamepadName;

	UPROPERTY(EditDefaultsOnly, Category = "Gamepad", Meta = (EditCondition = "bSupportsGamepad"))
	bool bCanChangeGamepadType;

	UPROPERTY(EditDefaultsOnly, Category = "Properties")
	bool bSupportsTouch;

	UPROPERTY(EditDefaultsOnly, Category = "Properties", Meta = (TitleProperty = "GamepadName"))
	TArray<TSoftClassPtr<UCommonInputBaseControllerData>> ControllerData;

	UPROPERTY(Transient)
	TArray<TSubclassOf<UCommonInputBaseControllerData>> ControllerDataClasses;
};

class FCommonInputBase
{
public:
	COMMONINPUT_API static FName GetCurrentPlatformName();

	COMMONINPUT_API static UCommonInputSettings* GetInputSettings();

	COMMONINPUT_API static void GetCurrentPlatformDefaults(ECommonInputType& OutDefaultInputType, FName& OutDefaultGamepadName);
};

#undef UE_API
