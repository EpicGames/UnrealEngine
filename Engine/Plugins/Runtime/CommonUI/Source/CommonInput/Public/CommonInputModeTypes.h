// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"

#include "CommonInputModeTypes.generated.h"

UENUM(BlueprintType)
enum class ECommonInputMode : uint8
{
	Menu	UMETA(Tooltip = "Input is received by the UI only"),
	Game	UMETA(Tooltip = "Input is received by the Game only"),
	All		UMETA(Tooltip = "Input is received by UI and the Game"),

	MAX UMETA(Hidden)
};

inline const TCHAR* LexToString(ECommonInputMode Value)
{
	switch (Value)
	{
	case ECommonInputMode::Menu: return TEXT("Menu"); break;
	case ECommonInputMode::Game: return TEXT("Game"); break;
	case ECommonInputMode::All: return TEXT("All"); break;
	}
	return TEXT("Unknown");
}
