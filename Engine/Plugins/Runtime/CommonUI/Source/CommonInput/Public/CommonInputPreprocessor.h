// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/LocalPlayerSubsystem.h"
#include "CommonInputBaseTypes.h"
#include "CommonInputSubsystem.h"
#include "Containers/Ticker.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"

#define UE_API COMMONINPUT_API

/**
 * Helper class that is designed to fire before any UI has a chance to process input so that
 * we can properly set the current input type of the application.
 */
class FCommonInputPreprocessor : public IInputProcessor
{
public:
	UE_API FCommonInputPreprocessor(UCommonInputSubsystem& InCommonInputSubsystem);

	//~ Begin IInputProcessor Interface
	UE_API virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override;
	UE_API virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
	UE_API virtual bool HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent) override;
	UE_API virtual bool HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent) override;
	UE_API virtual bool HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent) override;
	UE_API virtual bool HandleMouseButtonDoubleClickEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent) override;
	UE_API virtual bool HandleMouseWheelOrGestureEvent(FSlateApplication& SlateApp, const FPointerEvent& InWheelEvent, const FPointerEvent* InGestureEvent) override;
	virtual const TCHAR* GetDebugName() const override { return TEXT("CommonInput"); }
	//~ End IInputProcessor Interface

	UE_API void SetInputTypeFilter(ECommonInputType InputType, FName InReason, bool InFilter);

	UE_API bool IsInputMethodBlocked(ECommonInputType InputType) const;

	FGamepadChangeDetectedEvent OnGamepadChangeDetected;

protected:
	UE_API bool IsRelevantInput(FSlateApplication& SlateApp, const FInputEvent& InputEvent, const ECommonInputType DesiredInputType);

	UE_API void RefreshCurrentInputMethod(ECommonInputType InputMethod);

	UE_API ECommonInputType GetInputType(const FKey& Key);

	UE_API ECommonInputType GetInputType(const FPointerEvent& PointerEvent);
	
protected:
	UCommonInputSubsystem& InputSubsystem;
	
	bool bIgnoreNextMove = false;
	bool InputMethodPermissions[(uint8)ECommonInputType::Count];

	// The reasons we might be filtering input right now.
	TMap<FName, bool> FilterInputTypeWithReasons[(uint8)ECommonInputType::Count];

	FName LastSeenGamepadInputDeviceName;
	FString LastSeenGamepadHardwareDeviceIdentifier;

	friend class UCommonInputSubsystem;
};

#undef UE_API
