// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/SubclassOf.h"
#include "InputCoreTypes.h"
#include "Engine/StreamableManager.h"
#include "Templates/SharedPointer.h"
#include "CommonInputActionDomain.h"
#include "CommonInputSubsystem.h"
#include "CommonInputBaseTypes.h"

#include "Engine/DataTable.h"
#include "Engine/PlatformSettings.h"
#include "CommonInputSettings.generated.h"

#define UE_API COMMONINPUT_API

class UInputAction;

UCLASS(MinimalAPI, config = Game, defaultconfig)
class UCommonInputSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UCommonInputSettings(const FObjectInitializer& Initializer);

	// Called to load CommonUISetting data, if bAutoLoadData if set to false then game code must call LoadData().
	UE_API void LoadData();

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
    
    // Called to check that the data we have previously attempted to load is actually loaded and will attempt to load if it is not.
    UE_API void ValidateData();

	UE_API FDataTableRowHandle GetDefaultClickAction() const;
	UE_API FDataTableRowHandle GetDefaultBackAction() const;
	
	/** Default Hold Data */
	UE_API TSubclassOf<UCommonUIHoldData> GetDefaultHoldData() const;

	UE_API UInputAction* GetEnhancedInputClickAction() const;
	UE_API UInputAction* GetEnhancedInputBackAction() const;

	bool GetEnableInputMethodThrashingProtection() const { return bEnableInputMethodThrashingProtection; }

	int32 GetInputMethodThrashingLimit() const { return InputMethodThrashingLimit; }

	double GetInputMethodThrashingWindowInSeconds() const {	return InputMethodThrashingWindowInSeconds;	}

	double GetInputMethodThrashingCooldownInSeconds() const { return InputMethodThrashingCooldownInSeconds; }

	bool GetAllowOutOfFocusDeviceInput() const { return bAllowOutOfFocusDeviceInput; }

	bool GetEnableDefaultInputConfig() const { return bEnableDefaultInputConfig; }

	bool GetEnableEnhancedInputSupport() const { return bEnableEnhancedInputSupport; }

	bool GetEnableAutomaticGamepadTypeDetection() const { return bEnableAutomaticGamepadTypeDetection; }

	TObjectPtr<UCommonInputActionDomainTable> GetActionDomainTable() const { return ActionDomainTablePtr; }

	const TMap<FName, FName>& GetPlatformNameUpgradeMap() const { return PlatformNameUpgrades; }

public:

	/** Static version of enhanced input support check, exists to hide based on edit condition */
	UFUNCTION()
	static UE_API bool IsEnhancedInputSupportEnabled(); 

private:
	UE_API virtual void PostInitProperties() override;

	/** Create a derived asset from UCommonUIInputData to store Input data for your game.*/
	UPROPERTY(config, EditAnywhere, Category = "Input", Meta=(AllowAbstract=false))
	TSoftClassPtr<UCommonUIInputData> InputData;
	
	UPROPERTY(EditAnywhere, Category = "Input")
	FPerPlatformSettings PlatformInput;

	UPROPERTY(config)
	TMap<FName, FCommonInputPlatformBaseData> CommonInputPlatformData_DEPRECATED;

	UPROPERTY(config, EditAnywhere, Category = "Thrashing Settings")
	bool bEnableInputMethodThrashingProtection = true;

	UPROPERTY(config, EditAnywhere, Category = "Thrashing Settings")
	int32 InputMethodThrashingLimit = 30;

	UPROPERTY(config, EditAnywhere, Category = "Thrashing Settings")
	double InputMethodThrashingWindowInSeconds = 3.0;

	UPROPERTY(config, EditAnywhere, Category = "Thrashing Settings")
	double InputMethodThrashingCooldownInSeconds = 1.0;

	UPROPERTY(config, EditAnywhere, Category = "Input")
	bool bAllowOutOfFocusDeviceInput = false;

	/**
	* Controls whether a default Input Config will be set when the active CommonActivatableWidgets do not specify a desired one.
	* Disable this if you want to control the Input Mode via alternative means.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Input")
	bool bEnableDefaultInputConfig = true;

	/** Controls if Enhanced Input Support plugin-wide. Requires restart due to caching. */
	UPROPERTY(config, EditAnywhere, Category = "Input", meta = (ConfigRestartRequired = true))
	bool bEnableEnhancedInputSupport = false;

	/**
	* Controls automatic detection of the gamepad type.
	* Disable this if you want to manually control the gamepad type using the UCommonInputSubsystem::SetGamepadInputType() function.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Input")
	bool bEnableAutomaticGamepadTypeDetection = true;

	/** Create a derived asset from UCommonInputActionDomainTable to store ordered ActionDomain data for your game */
	UPROPERTY(config, EditAnywhere, Category = "Action Domain")
	TSoftObjectPtr<UCommonInputActionDomainTable> ActionDomainTable;
	
	/**
	* A map of Common Input platform names to a new one, which you can use
	* to upgrade your Input Action data tables if you add a new platform to your
	* project and wish to copy from some existing data
	*/
	UPROPERTY(config, EditAnywhere, Category = "Input", meta = (ConfigRestartRequired = true))
	TMap<FName, FName> PlatformNameUpgrades;

private:
	UE_API void LoadInputData();
	UE_API void LoadActionDomainTable();

	bool bInputDataLoaded;
	bool bActionDomainTableLoaded;

	UPROPERTY(Transient)
	TSubclassOf<UCommonUIInputData> InputDataClass;

	UPROPERTY(Transient)
	TObjectPtr<UCommonInputActionDomainTable> ActionDomainTablePtr;
};

#undef UE_API
