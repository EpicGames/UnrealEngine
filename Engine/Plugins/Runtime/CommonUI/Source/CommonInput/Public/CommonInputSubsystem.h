// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/LocalPlayerSubsystem.h"
#include "CommonInputBaseTypes.h"
#include "Containers/Ticker.h"
#include "CommonInputSubsystem.generated.h"

#define UE_API COMMONINPUT_API

class UWidget;
class ULocalPlayer;
class APlayerController;
class FCommonInputPreprocessor;
class FSlateUser;
class UCommonInputActionDomainTable;
struct FPlatformRect;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInputMethodChangedDelegate, ECommonInputType, bNewInputType);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FPlatformInputSupportOverrideDelegate, ULocalPlayer*, ECommonInputType, bool&);
DECLARE_EVENT_OneParam(FCommonInputPreprocessor, FGamepadChangeDetectedEvent, FName);

UCLASS(MinimalAPI, DisplayName = "CommonInput")
class UCommonInputSubsystem : public ULocalPlayerSubsystem
{
	GENERATED_BODY()

public:
	static UE_API UCommonInputSubsystem* Get(const ULocalPlayer* LocalPlayer);

	UE_API UCommonInputSubsystem();
	
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	DECLARE_EVENT_OneParam(UCommonInputSubsystem, FInputMethodChangedEvent, ECommonInputType);
	FInputMethodChangedEvent OnInputMethodChangedNative;

	UE_API FGamepadChangeDetectedEvent& GetOnGamepadChangeDetected();

	UE_API void SetInputTypeFilter(ECommonInputType InputType, FName Reason, bool Filter);
	UE_API bool GetInputTypeFilter(ECommonInputType InputType) const;

	/**  */
	UE_API void AddOrRemoveInputTypeLock(FName InReason, ECommonInputType InInputType, bool bAddLock);

	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API bool IsInputMethodActive(ECommonInputType InputMethod) const;

	/** The current input type based on the last input received on the device. */
	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API ECommonInputType GetCurrentInputType() const;

	/** The default input type for the current platform. */
	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API ECommonInputType GetDefaultInputType() const;

	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API void SetCurrentInputType(ECommonInputType NewInputType);

	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API const FName GetCurrentGamepadName() const;

	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API void SetGamepadInputType(const FName InGamepadInputType);

	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API bool IsUsingPointerInput() const;

	/** Should display indicators for the current input device on screen.  This is needed when capturing videos, but we don't want to reveal the capture source device. */
	UFUNCTION(BlueprintCallable, Category = CommonInputSubsystem)
	UE_API bool ShouldShowInputKeys() const;

	void SetActionDomainTable(TObjectPtr<UCommonInputActionDomainTable> Table) { ActionDomainTable = Table; }

	TObjectPtr<UCommonInputActionDomainTable> GetActionDomainTable() const { return ActionDomainTable; }

	/** Returns true if the specified key can be present on both a mobile device and mobile gamepads */
	static UE_API bool IsMobileGamepadKey(const FKey& InKey);

	/** Returns true if the current platform supports a hardware cursor */
	UE_API bool PlatformSupportsHardwareCursor() const;

	UE_API void SetCursorPosition(FVector2D NewPosition, bool bForce);

	UE_API void UpdateCursorPosition(TSharedRef<FSlateUser> SlateUser, const FVector2D& NewPosition, bool bForce = false);

	/** Getter */
	UE_API bool GetIsGamepadSimulatedClick() const;

	/** Setter */
	UE_API void SetIsGamepadSimulatedClick(bool bNewIsGamepadSimulatedClick);

	/** 
	* Gets the delegate that allows external systems to override which input methods are supported on this current platform.
	* @param LocalPlayer								The Local Player.
	* @param InputType									The current input type that is being tested.
	* @param InOutCurrentPlatformInputSupportState		The state of if we support the input type as set by PlatformSupportsInputType() or the previous callee of this delegate.
	* 
	* Note : Calling order is not guaranteed. Also, keep in mind that you might need to honor the previous callee's request to not support the input type being tested.
	*/
	static FPlatformInputSupportOverrideDelegate& GetOnPlatformInputSupportOverride() { return OnPlatformInputSupportOverride; }

	/** Returns true if the input method changed in the last thrashing window. See UCommonInputSettings::GetInputMethodThrashingWindowInSeconds */
	UE_API bool HadAnyChangeOfInputMethodInTheLastThrashingWindow() const;

protected:
	UE_API virtual TSharedPtr<FCommonInputPreprocessor> MakeInputProcessor();

	UE_API ECommonInputType LockInput(ECommonInputType InputToLock) const;

	UFUNCTION()
	UE_API void BroadcastInputMethodChanged();

protected:
	TSharedPtr<FCommonInputPreprocessor> CommonInputPreprocessor;

private:
	UE_API bool Tick(float DeltaTime);

	UE_API void ShouldShowInputKeysChanged(IConsoleVariable* Var);

	UE_API FVector2D ClampPositionToViewport(const FVector2D& InPosition) const;

	/** Returns true if the current platform supports the input type */
	UE_API bool PlatformSupportsInputType(ECommonInputType InInputType) const;

	UE_API bool CheckForInputMethodThrashing(ECommonInputType NewInputType);

	UE_API void RecalculateCurrentInputType();

	void HandleOnVirtualKeyboardShown(FPlatformRect VkRect);
	void HandleOnVirtualKeyboardHidden();

	FTSTicker::FDelegateHandle TickHandle;

	UPROPERTY(BlueprintAssignable, Category = CommonInputSubsystem, meta = (AllowPrivateAccess))
	FInputMethodChangedDelegate OnInputMethodChanged;

	UPROPERTY(Transient)
	int32 NumberOfInputMethodChangesRecently = 0;

	UPROPERTY(Transient)
	double LastInputMethodChangeTime = 0;

	UPROPERTY(Transient)
	double LastTimeInputMethodThrashingBegan = 0;

	/** The most recent input type that the user used, before considering locks and thrashing, but does consider PlatformSupportsInputType() */
	UPROPERTY(Transient)
	ECommonInputType RawInputType;

	/** The current effective input type after considering input locks and thrashing */
	UPROPERTY(Transient)
	ECommonInputType CurrentInputType;

	/**  */
	UPROPERTY(Transient)
	FName GamepadInputType;

	/** Whether the last input type recalculation was locked out by input thrashing */
	UPROPERTY(Transient)
	bool bInputMethodLockedByThrashing = false;

	/**  */
	UPROPERTY(Transient)
	TMap<FName, ECommonInputType> CurrentInputLocks;

	TOptional<ECommonInputType> CurrentInputLock;

	UPROPERTY(Transient)
	TObjectPtr<UCommonInputActionDomainTable> ActionDomainTable;

	/** Is the current click simulated by the gamepad's face button down/right (platform dependent) */
	UPROPERTY(Transient)
	bool bIsGamepadSimulatedClick;

	/**
	* The delegate that allows external systems to override which input methods are supported on this current platform.
	* @param LocalPlayer								The Local Player.
	* @param InputType									The current input type that is being tested.
	* @param InOutCurrentPlatformInputSupportState		The state of if we support the input type as set by PlatformSupportsInputType() or the previous callee of this delegate.
	*
	* Note : Calling order is not guaranteed. Also, keep in mind that you might need to honor the previous callee's request to not support the input type being tested.
	*/
	static UE_API FPlatformInputSupportOverrideDelegate OnPlatformInputSupportOverride;
};

#undef UE_API
