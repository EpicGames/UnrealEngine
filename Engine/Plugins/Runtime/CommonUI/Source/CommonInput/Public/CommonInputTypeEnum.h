// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumRange.h"
#include "CommonInputTypeEnum.generated.h"

UENUM(BlueprintType)
enum class ECommonInputType : uint8
{
	MouseAndKeyboard,
	Gamepad,
	Touch,
	Count
};

ENUM_RANGE_BY_COUNT(ECommonInputType, ECommonInputType::Count);
