// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Modules/ModuleManager.h"
#include "CommonInputSettings.h"

struct FStreamableManager;

/**
 * Interface for the input state system.
 */
class ICommonInputModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline ICommonInputModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ICommonInputModule>("CommonInput");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("CommonInput");
	}

	/**
	 * @return reference to Common Input settings
 	*/
	static inline UCommonInputSettings& GetSettings()
	{
		ICommonInputModule& Module = IsInGameThread() ? Get() : FModuleManager::GetModuleChecked<ICommonInputModule>("CommonInput");
		UCommonInputSettings* Settings = Module.GetSettingsInstance();
		check(Settings);
		return *Settings;
	}

protected:

	virtual UCommonInputSettings* GetSettingsInstance() const = 0;
};
