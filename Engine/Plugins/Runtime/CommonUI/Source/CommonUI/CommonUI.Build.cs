// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CommonUI : ModuleRules
{
	public CommonUI(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"ApplicationCore",
                "Engine",
				"InputCore",
				"Slate",
                "UMG",
                "WidgetCarousel",
				"CommonInput",
				"EnhancedInput",
				"MediaAssets",
				"GameplayTags"
			}
		);

		PrivateDependencyModuleNames.AddRange(
		new string[]
			{
				"SlateCore",
                "Analytics",
				"AnalyticsET",
                "EngineSettings",
				"AudioMixer",
				"DeveloperSettings"
			}
		);

		if (Target.Type == TargetType.Editor)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
                    "UnrealEd",
                }
            );
        }
		PrivateDefinitions.Add("UE_COMMONUI_PLATFORM_KBM_REQUIRES_ATTACHED_MOUSE=" + (bPlatformKBMRequiresAttachedMouse ? "1" : "0"));
		PrivateDefinitions.Add("UE_COMMONUI_PLATFORM_REQUIRES_CURSOR_HIDDEN_FOR_TOUCH=" + (bPlatformRequiresCursorHiddenForTouch ? "1" : "0"));

		bDisableAutoRTFMInstrumentation = true;
	}

	protected virtual bool bPlatformKBMRequiresAttachedMouse { get { return false; } }
	protected virtual bool bPlatformRequiresCursorHiddenForTouch { get { return false; } }
}
