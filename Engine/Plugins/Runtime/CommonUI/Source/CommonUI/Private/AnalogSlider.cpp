// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalogSlider.h"
#include "CommonInputSubsystem.h"
#include "CommonInputTypeEnum.h"
#include "SAnalogSlider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnalogSlider)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UAnalogSlider

UAnalogSlider::UAnalogSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetOrientation(EOrientation::Orient_Horizontal);
	SetSliderBarColor(FLinearColor::White);
	SetSliderHandleColor(FLinearColor::White);
	SetStepSize(0.01f);
	IsFocusable = true;
}

TSharedRef<SWidget> UAnalogSlider::RebuildWidget()
{
	MySlider = MyAnalogSlider = SNew(SAnalogSlider)
		.Style(&GetWidgetStyle())
		.IsFocusable(IsFocusable)
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd))
		.OnControllerCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureBegin))
		.OnControllerCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureEnd))
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnAnalogCapture(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnAnalogCapture));

	
	if (UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(GetOwningLocalPlayer()))
	{
		MyAnalogSlider->SetUsingGamepad(CommonInputSubsystem->GetCurrentInputType() == ECommonInputType::Gamepad);
		CommonInputSubsystem->OnInputMethodChangedNative.AddUObject(this, &UAnalogSlider::HandleInputMethodChanged);
	}

	return MySlider.ToSharedRef();
}

void UAnalogSlider::SynchronizeProperties()
{
	Super::SynchronizeProperties();
}

void UAnalogSlider::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyAnalogSlider.Reset();
	if (UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(GetOwningLocalPlayer()))
	{
		CommonInputSubsystem->OnInputMethodChangedNative.RemoveAll(this);
	}
}

void UAnalogSlider::HandleOnAnalogCapture(float InValue)
{
	OnAnalogCapture.Broadcast(InValue);
}

void UAnalogSlider::HandleInputMethodChanged(ECommonInputType CurrentInputType)
{
	MyAnalogSlider->SetUsingGamepad(CurrentInputType == ECommonInputType::Gamepad);
}

#undef LOCTEXT_NAMESPACE

