// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonActionHandlerInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonActionHandlerInterface)

EInputActionState FCommonInputActionHandlerData::GetState(ECommonInputType InputType, const FName& GamepadName) const
{
	const FCommonInputActionDataBase* InputActionData = CommonUI::GetInputActionData(InputActionRow);
	if (InputActionData)
	{
		const EInputActionState StateForInputType = InputActionData->GetInputTypeInfo(InputType, GamepadName).OverrrideState;
		if (StateForInputType != EInputActionState::Enabled)
		{
			return StateForInputType;
		}
	}
	return State;
}

UCommonActionHandlerInterface::UCommonActionHandlerInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

