// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonActionWidget.h"

#include "CommonInputBaseTypes.h"
#include "CommonInputSubsystem.h"
#include "CommonInputTypeEnum.h"
#include "CommonUITypes.h"
#include "CommonWidgetPaletteCategories.h"
#include "Engine/LocalPlayer.h"
#include "EnhancedInputSubsystems.h"
#include "Input/UIActionBinding.h"
#include "InputAction.h"
#include "InputTriggers.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Styling/StyleDefaults.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonActionWidget)

UCommonActionWidget::UCommonActionWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	IconRimBrush = *FStyleDefaults::GetNoBrush();
}

void UCommonActionWidget::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (!InputActionDataRow_DEPRECATED.IsNull())
		{
			InputActions.Add(InputActionDataRow_DEPRECATED);
			InputActionDataRow_DEPRECATED = FDataTableRowHandle();
		}
	}
#endif
}

TSharedRef<SWidget> UCommonActionWidget::RebuildWidget()
{
	if (!IsDesignTime() && ProgressDynamicMaterial == nullptr)
	{
		UMaterialInstanceDynamic* const ParentMaterialDynamic = Cast<UMaterialInstanceDynamic>(ProgressMaterialBrush.GetResourceObject());
		if (ParentMaterialDynamic == nullptr)
		{
			UMaterialInterface* ParentMaterial = Cast<UMaterialInterface>(ProgressMaterialBrush.GetResourceObject());
			if (ParentMaterial)
			{
				ProgressDynamicMaterial = UMaterialInstanceDynamic::Create(ParentMaterial, nullptr);
				ProgressMaterialBrush.SetResourceObject(ProgressDynamicMaterial);
			}
			else
			{
				ProgressDynamicMaterial = nullptr;
			}
		}
	}

	MyKeyBox = SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center);

	MyKeyBox->SetContent(	
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SAssignNew(MyIconRim, SImage)
			.Image(&IconRimBrush)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SAssignNew(MyProgressImage, SImage)
			.Image(&ProgressMaterialBrush)
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SAssignNew(MyIcon, SImage)
			.Image(&Icon)
		]);
	
	return MyKeyBox.ToSharedRef();
}

void UCommonActionWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	MyProgressImage.Reset();
	MyIcon.Reset();
	MyKeyBox.Reset();
	
	ListenToInputMethodChanged(false);
	Super::ReleaseSlateResources(bReleaseChildren);
}

void UCommonActionWidget::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyKeyBox.IsValid() && IsDesignTime())
	{
		UpdateActionWidget();
	}
}

FSlateBrush UCommonActionWidget::GetIcon() const
{
	if (!IsDesignTime())
	{
		if (const UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem())
		{
			return EnhancedInputAction && CommonUI::IsEnhancedInputSupportEnabled()
				? CommonUI::GetIconForEnhancedInputAction(CommonInputSubsystem, EnhancedInputAction)
				: CommonUI::GetIconForInputActions(CommonInputSubsystem, InputActions);
		}
	}
#if WITH_EDITORONLY_DATA
	else
	{
		if (DesignTimeKey.IsValid())
		{
			ECommonInputType Dummy;
			FName OutDefaultGamepadName;
			FCommonInputBase::GetCurrentPlatformDefaults(Dummy, OutDefaultGamepadName);

			ECommonInputType KeyInputType = ECommonInputType::MouseAndKeyboard;
			if (DesignTimeKey.IsGamepadKey())
			{
				KeyInputType = ECommonInputType::Gamepad;
			}
			else if (DesignTimeKey.IsTouch())
			{
				KeyInputType = ECommonInputType::Touch;
			}

			FSlateBrush InputBrush;
			if (UCommonInputPlatformSettings::Get()->TryGetInputBrush(InputBrush, TArray<FKey> { DesignTimeKey }, KeyInputType, OutDefaultGamepadName))
			{
				return InputBrush;
			}
		}
	}
#endif

	return *FStyleDefaults::GetNoBrush();
}

UCommonInputSubsystem* UCommonActionWidget::GetInputSubsystem() const
{
	// In the new system, we may be representing an action for any player, not necessarily the one that technically owns this action icon widget
	// We want to be sure to use the LocalPlayer that the binding is actually for so we can display the icon that corresponds to their current input method
	const ULocalPlayer* BoundLocalPlayer = DisplayedBindingHandle.GetBoundLocalPlayer();
	const ULocalPlayer* BindingOwner = BoundLocalPlayer ? BoundLocalPlayer : GetOwningLocalPlayer();
	return UCommonInputSubsystem::Get(BindingOwner);
}

const FCommonInputActionDataBase* UCommonActionWidget::GetInputActionData() const
{
	if (InputActions.Num() > 0)
	{
		const FCommonInputActionDataBase* InputActionData = CommonUI::GetInputActionData(InputActions[0]);
		return InputActionData;
	}

	return nullptr;
}

FText UCommonActionWidget::GetDisplayText() const
{
	if (EnhancedInputAction && CommonUI::IsEnhancedInputSupportEnabled())
	{
		return EnhancedInputAction->ActionDescription;
	}

	const UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();
	if (GetGameInstance() && CommonInputSubsystem)
	{
		if (const FCommonInputActionDataBase* InputActionData = GetInputActionData())
		{
			const FCommonInputTypeInfo& InputTypeInfo = InputActionData->GetCurrentInputTypeInfo(CommonInputSubsystem);

			if (InputTypeInfo.bActionRequiresHold)
			{
				return InputActionData->HoldDisplayName;
			}
			return InputActionData->DisplayName;
		}
	}
	return FText();
}

UMaterialInstanceDynamic* UCommonActionWidget::GetIconDynamicMaterial()
{
	if (UMaterialInterface* Material = Cast<UMaterialInterface>(Icon.GetResourceObject()))
	{
		UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);

		if (!DynamicMaterial)
		{
			DynamicMaterial = UMaterialInstanceDynamic::Create(Material, this);
			Icon.SetResourceObject(DynamicMaterial);

			if (MyIcon.IsValid())
			{
				MyIcon->InvalidateImage();
			}
		}
		return DynamicMaterial;
	}

	return nullptr;
}

bool UCommonActionWidget::IsHeldAction() const
{
	if (EnhancedInputAction && CommonUI::IsEnhancedInputSupportEnabled())
	{
		for (const TObjectPtr<UInputTrigger>& Trigger : EnhancedInputAction->Triggers)
		{
			if (Trigger != nullptr && EnumHasAnyFlags(Trigger->GetSupportedTriggerEvents(), ETriggerEventsSupported::Ongoing))
			{
				return true;
			}
		}

		return false;
	}

	const UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();
	if (GetGameInstance() && CommonInputSubsystem)
	{
		if (const FCommonInputActionDataBase* InputActionData = GetInputActionData())
		{
			const FCommonInputTypeInfo& InputTypeInfo = InputActionData->GetCurrentInputTypeInfo(CommonInputSubsystem);

			return InputTypeInfo.bActionRequiresHold;
		}
	}
	return false;
}

void UCommonActionWidget::SetEnhancedInputAction(UInputAction* InInputAction)
{
	UpdateBindingHandleInternal(FUIActionBindingHandle());
	EnhancedInputAction = InInputAction;
	InputActions.Reset();
	UpdateActionWidget();
}

const UInputAction* UCommonActionWidget::GetEnhancedInputAction() const
{
	return EnhancedInputAction;
}

void UCommonActionWidget::SetInputAction(FDataTableRowHandle InputActionRow)
{
	UpdateBindingHandleInternal(FUIActionBindingHandle());
	EnhancedInputAction = nullptr;
	InputActions.Reset();
	InputActions.Add(InputActionRow);

	UpdateActionWidget();
}

void UCommonActionWidget::SetInputActionBinding(FUIActionBindingHandle BindingHandle)
{
	UpdateBindingHandleInternal(BindingHandle);
	if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(BindingHandle))
	{
		InputActions.Reset();

		const UInputAction* InputAction = Binding->InputAction.Get();
		if (CommonUI::IsEnhancedInputSupportEnabled() && InputAction)
		{
			EnhancedInputAction = const_cast<UInputAction*>(InputAction);
		}
		else
		{
			EnhancedInputAction = nullptr;
			InputActions.Add(Binding->LegacyActionTableRow);
		}

		UpdateActionWidget();
	}
}

void UCommonActionWidget::SetInputActions(TArray<FDataTableRowHandle> InInputActions)
{
	UpdateBindingHandleInternal(FUIActionBindingHandle());
	EnhancedInputAction = nullptr;
	InputActions = InInputActions;

	UpdateActionWidget();
}

void UCommonActionWidget::SetIconRimBrush(FSlateBrush InIconRimBrush)
{
	IconRimBrush = InIconRimBrush;
}

void UCommonActionWidget::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();
	UpdateActionWidget();
	ListenToInputMethodChanged();
}

void UCommonActionWidget::UpdateBindingHandleInternal(FUIActionBindingHandle BindingHandle)
{
	if (DisplayedBindingHandle != BindingHandle && (DisplayedBindingHandle.IsValid() || BindingHandle.IsValid()))
	{
		// When the binding handle changes, the player that owns the binding may be different, so we clear & rebind just in case
		ListenToInputMethodChanged(false);
		DisplayedBindingHandle = BindingHandle;
		ListenToInputMethodChanged(true);
	}
}

void UCommonActionWidget::UpdateActionWidget()
{
	if (GetWorld())
	{
		const UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();
		if (IsDesignTime() || (GetGameInstance() && CommonInputSubsystem && CommonInputSubsystem->ShouldShowInputKeys()))
		{
			if (ShouldUpdateActionWidgetIcon())
			{
				Icon = GetIcon();

				if (Icon.DrawAs == ESlateBrushDrawType::NoDrawType)
				{
					if (!IsDesignTime())
					{
						SetVisibility(ESlateVisibility::Collapsed);
					}
				}
				else if (MyIcon.IsValid())
				{
					MyIcon->SetImage(&Icon);
					OnInputIconUpdated.Broadcast();

					if (GetVisibility() != ESlateVisibility::Collapsed)
					{
						// The object being passed into SetImage is the same each time so layout is never invalidated
						// Manually invalidate it here as the dimensions may have changed
						MyIcon->Invalidate(EInvalidateWidgetReason::Layout);
					}

					if (MyProgressImage.IsValid())
					{
						if (IsHeldAction())
						{
							MyProgressImage->SetVisibility(EVisibility::SelfHitTestInvisible);
						}
						else
						{
							MyProgressImage->SetVisibility(EVisibility::Collapsed);
						}
					}

					if (MyKeyBox.IsValid())
					{
						MyKeyBox->Invalidate(EInvalidateWidget::LayoutAndVolatility);
					}

					if (!IsDesignTime())
					{
						SetVisibility(ESlateVisibility::SelfHitTestInvisible);
					}

					return;
				}
			}
		}

		if (!IsDesignTime())
		{
			SetVisibility(ESlateVisibility::Collapsed);
		}
	}
}

bool UCommonActionWidget::ShouldUpdateActionWidgetIcon() const
{
	if (bAlwaysHideOverride)
	{
		return false;
	}

	const FCommonInputActionDataBase* InputActionData = GetInputActionData();
	const bool bIsEnhancedInputAction = EnhancedInputAction && CommonUI::IsEnhancedInputSupportEnabled();

#if WITH_EDITORONLY_DATA
	const bool bIsDesignPreview = IsDesignTime();
#else
	const bool bIsDesignPreview = false;
#endif

	return InputActionData || bIsEnhancedInputAction || bIsDesignPreview;
}

void UCommonActionWidget::ListenToInputMethodChanged(bool bListen)
{
	if (UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem())
	{
		CommonInputSubsystem->OnInputMethodChangedNative.RemoveAll(this);
		if (bListen)
		{
			CommonInputSubsystem->OnInputMethodChangedNative.AddUObject(this, &ThisClass::HandleInputMethodChanged);
		}
	}

	if (CommonUI::IsEnhancedInputSupportEnabled())
	{	
		const ULocalPlayer* BoundLocalPlayer = DisplayedBindingHandle.GetBoundLocalPlayer();
     	const ULocalPlayer* LocalPlayer = BoundLocalPlayer ? BoundLocalPlayer : GetOwningLocalPlayer();
		if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(LocalPlayer))
		{
			if (bListen)
			{
				EnhancedInputLocalPlayerSubsystem->ControlMappingsRebuiltDelegate.AddUniqueDynamic(this, &UCommonActionWidget::OnEnhancedInputMappingsRebuilt);
			}
			else
			{
				EnhancedInputLocalPlayerSubsystem->ControlMappingsRebuiltDelegate.RemoveDynamic(this, &UCommonActionWidget::OnEnhancedInputMappingsRebuilt);
			}
		}
	}
}

void UCommonActionWidget::HandleInputMethodChanged(ECommonInputType InInputType)
{
	UpdateActionWidget();
	OnInputMethodChanged.Broadcast(InInputType==ECommonInputType::Gamepad);
}

void UCommonActionWidget::OnEnhancedInputMappingsRebuilt()
{
	UpdateActionWidget();
}

#if WITH_EDITOR
const FText UCommonActionWidget::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif

void UCommonActionWidget::OnActionProgress(float HeldPercent)
{
	if (ProgressDynamicMaterial && !ProgressMaterialParam.IsNone())
	{
		ProgressDynamicMaterial->SetScalarParameterValue(ProgressMaterialParam, HeldPercent);
	}
}

void UCommonActionWidget::OnActionComplete()
{
	if (ProgressDynamicMaterial && !ProgressMaterialParam.IsNone())
	{
		ProgressDynamicMaterial->SetScalarParameterValue(ProgressMaterialParam, 0.f);
	}
}

void UCommonActionWidget::SetProgressMaterial(const FSlateBrush& InProgressMaterialBrush, const FName& InProgressMaterialParam)
{
	ProgressMaterialBrush = InProgressMaterialBrush;
	ProgressMaterialParam = InProgressMaterialParam;

	UMaterialInterface* ParentMaterial = Cast<UMaterialInterface>(ProgressMaterialBrush.GetResourceObject());
	if (ParentMaterial)
	{
		ProgressDynamicMaterial = UMaterialInstanceDynamic::Create(ParentMaterial, nullptr);
		ProgressMaterialBrush.SetResourceObject(ProgressDynamicMaterial);
	}
	else
	{
		ProgressDynamicMaterial = nullptr;
	}

	if (MyProgressImage.IsValid())
	{
		MyProgressImage->SetImage(&ProgressMaterialBrush);
	}
}

void UCommonActionWidget::SetHidden(bool bAlwaysHidden)
{
	bAlwaysHideOverride = bAlwaysHidden;
	UpdateActionWidget();
}
