// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonActivatableWidget.h"
#include "CommonInputSettings.h"
#include "CommonUIPrivate.h"
#include "CommonUITypes.h"
#include "Engine/GameInstance.h"
#include "EnhancedInputSubsystems.h"
#include "InputMappingContext.h"
#include "Input/CommonUIInputTypes.h"
#include "Input/UIActionBinding.h"
#include "Input/UIActionRouterTypes.h"
#include "ICommonInputModule.h"
#include "Slate/SObjectWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonActivatableWidget)

//////////////////////////////////////////////////////////////////////////

static bool bWarnDesiredFocusNotImplemented = false;
static FAutoConsoleVariableRef CVarWarnDesiredFocusNotImplemented(
	TEXT("CommonUI.Debug.WarnDesiredFocusNotImplemented"),
	bWarnDesiredFocusNotImplemented,
	TEXT("Warn when a widget is activated without GetDesiredFocus implemented."));

//////////////////////////////////////////////////////////////////////////

UCommonActivatableWidget::FActivatableWidgetRebuildEvent UCommonActivatableWidget::OnRebuilding;

void UCommonActivatableWidget::NativeConstruct()
{
	Super::NativeConstruct();

	if (bIsBackHandler)
	{
		if (CommonUI::IsEnhancedInputSupportEnabled() && ICommonInputModule::GetSettings().GetEnhancedInputBackAction())
		{
			FBindUIActionArgs BindArgs(ICommonInputModule::GetSettings().GetEnhancedInputBackAction(), FSimpleDelegate::CreateUObject(this, &UCommonActivatableWidget::HandleBackAction));
			BindArgs.bDisplayInActionBar = bIsBackActionDisplayedInActionBar;
			BindArgs.OverrideDisplayName = OverrideBackActionDisplayName;

			DefaultBackActionHandle = RegisterUIActionBinding(BindArgs);
		}
		else
		{
			FBindUIActionArgs BindArgs(ICommonInputModule::GetSettings().GetDefaultBackAction(), FSimpleDelegate::CreateUObject(this, &UCommonActivatableWidget::HandleBackAction));
			BindArgs.bDisplayInActionBar = bIsBackActionDisplayedInActionBar;
			BindArgs.OverrideDisplayName = OverrideBackActionDisplayName;

			DefaultBackActionHandle = RegisterUIActionBinding(BindArgs);
		}
	}

	if (bAutoActivate)
	{
		UE_LOGF(LogCommonUI, Verbose, "[%ls] auto-activated", *GetName());
		ActivateWidget();
	}
}

void UCommonActivatableWidget::NativeDestruct()
{
	if (UGameInstance* GameInstance = GetGameInstance<UGameInstance>())
	{
		// Deactivations might rely on members of the game instance to validly run.
		// If there's no game instance, any cleanup done in Deactivation will be irrelevant; we're shutting down the game
		DeactivateWidget();
	}
	Super::NativeDestruct();

	if (DefaultBackActionHandle.IsValid())
	{
		DefaultBackActionHandle.Unregister();
	}
}

UWidget* UCommonActivatableWidget::GetDesiredFocusTarget() const
{
	return NativeGetDesiredFocusTarget();
}

void UCommonActivatableWidget::ClearFocusRestorationTarget()
{
	if (TSharedPtr<FActivatableTreeNode> PinnedTreeNode = InputTreeNode.Pin())
	{
		PinnedTreeNode->ClearFocusRestorationTarget();
	}
}

TOptional<FActivationMetadata> UCommonActivatableWidget::GetActivationMetadata() const
{
	return TOptional<FActivationMetadata>();
}

UWidget* UCommonActivatableWidget::NativeGetDesiredFocusTarget() const
{
	// Prioritize BP implementation of this function.
	UWidget* DesiredFocusTarget = BP_GetDesiredFocusTarget();

	if (!DesiredFocusTarget)
	{
		// BP didn't specify focus target, fallback to DesiredFocusWidget property on UserWidget.
		DesiredFocusTarget = GetDesiredFocusWidget();
	}

	return DesiredFocusTarget;
}

TOptional<FUIInputConfig> UCommonActivatableWidget::GetDesiredInputConfig() const
{
	// Check if there is a BP implementation for input configs
	if (GetClass()->IsFunctionImplementedInScript(GET_FUNCTION_NAME_CHECKED(UCommonActivatableWidget, BP_GetDesiredInputConfig)))
	{
		return BP_GetDesiredInputConfig();
	}

	// No particular config is desired by default
	return TOptional<FUIInputConfig>();
}

void UCommonActivatableWidget::RequestRefreshFocus()
{
	OnRequestRefreshFocusEvent.Broadcast();
}

void UCommonActivatableWidget::ActivateWidget()
{
	if (!bIsActive)
	{
		InternalProcessActivation();
	}
}

void UCommonActivatableWidget::InternalProcessActivation()
{
	UE_LOGF(LogCommonUI, Verbose, "[%ls] -> Activated", *GetName());

	bIsActive = true;
	NativeOnActivated();
}

void UCommonActivatableWidget::DeactivateWidget()
{
	if (bIsActive)
	{
		InternalProcessDeactivation();
	}
}

void UCommonActivatableWidget::SetBindVisibilities(ESlateVisibility OnActivatedVisibility, ESlateVisibility OnDeactivatedVisibility, bool bInAllActive)
{
	ActivatedBindVisibility = OnActivatedVisibility;
	DeactivatedBindVisibility = OnDeactivatedVisibility;
	bAllActive = bInAllActive;
}

void UCommonActivatableWidget::BindVisibilityToActivation(UCommonActivatableWidget* ActivatableWidget)
{
	if (ActivatableWidget && !VisibilityBoundWidgets.Contains(ActivatableWidget))
	{
		VisibilityBoundWidgets.Add(ActivatableWidget);
		ActivatableWidget->OnActivated().AddUObject(this, &UCommonActivatableWidget::HandleVisibilityBoundWidgetActivations);
		ActivatableWidget->OnDeactivated().AddUObject(this, &UCommonActivatableWidget::HandleVisibilityBoundWidgetActivations);

		HandleVisibilityBoundWidgetActivations();
	}
}

void UCommonActivatableWidget::InternalProcessDeactivation()
{
	UE_LOGF(LogCommonUI, Verbose, "[%ls] -> Deactivated", *GetName());

	bIsActive = false;
	NativeOnDeactivated();
}

TWeakPtr<FActivatableTreeNode> UCommonActivatableWidget::GetInputTreeNode() const
{
	return InputTreeNode;
}

void UCommonActivatableWidget::RegisterInputTreeNode(const TSharedPtr<FActivatableTreeNode>& OwnerNode)
{
	InputTreeNode = OwnerNode;
}

void UCommonActivatableWidget::ClearActiveHoldInputs()
{
	if (InputTreeNode.IsValid())
	{
		TSharedPtr<FActivatableTreeNode> Node = InputTreeNode.Pin();
		if (Node->HasHoldBindings())
		{
			const TArray<FUIActionBindingHandle>& NodeActionBindings = Node->GetActionBindings();
			for (const FUIActionBindingHandle& Handle : NodeActionBindings)
			{
				const TSharedPtr<FUIActionBinding>& UIActionBinding = FUIActionBinding::FindBinding(Handle);
				if (UIActionBinding.IsValid() && UIActionBinding->IsHoldActive())
				{
					UIActionBinding->CancelHold();
					UIActionBinding->OnHoldActionProgressed.Broadcast(0.0f);
				}
			}
		}
	}
}

TObjectPtr<UCommonInputActionDomain> UCommonActivatableWidget::GetCalculatedActionDomain()
{
	if (CalculatedActionDomainCache.IsSet())
	{
		return CalculatedActionDomainCache.GetValue().Get();
	}

	const ULocalPlayer* OwningLocalPlayer = GetOwningLocalPlayer();
	TSharedPtr<SWidget> CurrentWidget = GetCachedWidget();
	while (CurrentWidget)
	{
		if (CurrentWidget->GetMetaData<FCommonActivatableSlateMetaData>().IsValid())
		{
			const TSharedPtr<ICommonInputActionDomainMetaData> Metadata = CurrentWidget->GetMetaData<ICommonInputActionDomainMetaData>();
			if (Metadata.IsValid())
			{
				UCommonInputActionDomain* ActionDomain = Metadata->ActionDomain.Get();
				CalculatedActionDomainCache = ActionDomain;
				return ActionDomain;
			}

			if (UCommonActivatableWidget* CurrentActivatable = Cast<UCommonActivatableWidget>(StaticCastSharedPtr<SObjectWidget>(CurrentWidget)->GetWidgetObject()))
			{
				if (CurrentActivatable->bOverrideActionDomain)
				{
					UCommonInputActionDomain* CurrentActionDomain = CurrentActivatable->GetOwningLocalPlayer() == OwningLocalPlayer ? CurrentActivatable->ActionDomainOverride.Get() : nullptr;
					CalculatedActionDomainCache = CurrentActionDomain;
					return CurrentActionDomain;
				}
			}
		}

		CurrentWidget = CurrentWidget->GetParentWidget();
	}

	CalculatedActionDomainCache = nullptr;
	return nullptr;
}

void UCommonActivatableWidget::ResetCalculatedActionDomainCache()
{
	CalculatedActionDomainCache.Reset();
}

TSharedRef<SWidget> UCommonActivatableWidget::RebuildWidget()
{
	// Note: the scoped builder guards against design-time so we don't need to here (as it'd make the scoped lifetime more awkward to leverage)
	//FScopedActivatableTreeBuilder ScopedBuilder(*this);
	if (!IsDesignTime())
	{
		OnRebuilding.Broadcast(*this);
	}
	
	return Super::RebuildWidget();
}

void UCommonActivatableWidget::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();

	TSharedPtr<SObjectWidget> GCWidget = MyGCWidget.Pin();
	if (GCWidget.IsValid() && !GCWidget->GetMetaData<FCommonActivatableSlateMetaData>().IsValid())
	{
		GCWidget->AddMetadata(MakeShared<FCommonActivatableSlateMetaData>());
	}
}

void UCommonActivatableWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	OnSlateReleased().Broadcast();
}

void UCommonActivatableWidget::NativeOnActivated()
{
	if (ensureMsgf(bIsActive, TEXT("[%s] has called NativeOnActivated, but isn't actually activated! Never call this directly - call ActivateWidget()"), *GetName()))
	{
		if (bSetVisibilityOnActivated)
		{
			SetVisibility(ActivatedVisibility);
			UE_LOGF(LogCommonUI, Verbose, "[%ls] set visibility to [%ls] on activation", *GetName(), *StaticEnum<ESlateVisibility>()->GetDisplayValueAsText(ActivatedVisibility).ToString());
		}

		ActivateMappingContext();
		BP_OnActivated();
		OnActivated().Broadcast();
		BP_OnWidgetActivated.Broadcast();
	}
}

void UCommonActivatableWidget::NativeOnDeactivated()
{
	if (ensure(!bIsActive))
	{
		if (bSetVisibilityOnDeactivated)
		{
			SetVisibility(DeactivatedVisibility);
			UE_LOGF(LogCommonUI, Verbose, "[%ls] set visibility to [%ls] on deactivation", *GetName(), *StaticEnum<ESlateVisibility>()->GetDisplayValueAsText(DeactivatedVisibility).ToString());
		}

		DeactivateMappingContext();

		// Cancel any holds that were active
		ClearActiveHoldInputs();

		BP_OnDeactivated();
		OnDeactivated().Broadcast();
		BP_OnWidgetDeactivated.Broadcast();
	}
}

void UCommonActivatableWidget::ActivateMappingContext()
{
	if (CommonUI::IsEnhancedInputSupportEnabled() && InputMapping)
	{
		if (const ULocalPlayer* LocalPlayer = GetOwningLocalPlayer())
		{
			if (UEnhancedInputLocalPlayerSubsystem* InputSystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
			{
				InputSystem->AddMappingContext(InputMapping, InputMappingPriority);
			}
		}
	}
}

void UCommonActivatableWidget::DeactivateMappingContext()
{
	if (CommonUI::IsEnhancedInputSupportEnabled() && InputMapping)
	{
		if (const ULocalPlayer* LocalPlayer = GetOwningLocalPlayer())
		{
			if (UEnhancedInputLocalPlayerSubsystem* InputSystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
			{
				InputSystem->RemoveMappingContext(InputMapping);
			}
		}
	}
}

bool UCommonActivatableWidget::NativeOnHandleBackAction()
{
	if (bIsBackHandler)
	{
		if (!BP_OnHandleBackAction())
		{
			// Default behavior is unconditional deactivation
			UE_LOGF(LogCommonUI, Verbose, "[%ls] handled back with default implementation. Deactivating immediately.", *GetName());
			DeactivateWidget();
		}
		return true;
	}
	return false;
}

void UCommonActivatableWidget::HandleBackAction()
{
	NativeOnHandleBackAction();
}

void UCommonActivatableWidget::HandleVisibilityBoundWidgetActivations()
{
	ESlateVisibility OldDeactivatedVisibility = DeactivatedBindVisibility;
	OldDeactivatedVisibility = bSetVisibilityOnActivated && IsActivated() ? ActivatedVisibility : OldDeactivatedVisibility;
	OldDeactivatedVisibility = bSetVisibilityOnDeactivated  && !IsActivated() ? DeactivatedVisibility : OldDeactivatedVisibility;

	for (const TWeakObjectPtr<UCommonActivatableWidget>& VisibilityBoundWidget : VisibilityBoundWidgets)
	{
		if (VisibilityBoundWidget.IsValid())
		{
			if (bAllActive)
			{
				if (!VisibilityBoundWidget->IsActivated())
				{
					SetVisibility(OldDeactivatedVisibility);
					return;
				}
			}
			else 
			{
				if (VisibilityBoundWidget->IsActivated())
				{
					SetVisibility(ActivatedBindVisibility);
					return;
				}
			}
		}
	}

	SetVisibility(bAllActive ? ActivatedBindVisibility : OldDeactivatedVisibility);
}

void UCommonActivatableWidget::Reset()
{
	bIsActive = false;

	BP_OnWidgetActivated.Clear();
	BP_OnWidgetDeactivated.Clear();
}

