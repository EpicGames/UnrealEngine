// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonActivatableWidgetSwitcher.h"

#include "CommonActivatableWidget.h"
#include "CommonUIUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonActivatableWidgetSwitcher)

void UCommonActivatableWidgetSwitcher::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	UnbindOwningActivatableWidget();
}

void UCommonActivatableWidgetSwitcher::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();

	BindOwningActivatableWidget();
}

void UCommonActivatableWidgetSwitcher::HandleOutgoingWidget()
{
	DeactivateActiveWidget();
}

void UCommonActivatableWidgetSwitcher::HandleSlateActiveIndexChanged(int32 ActiveIndex)
{
	Super::HandleSlateActiveIndexChanged(ActiveIndex);

	if (!WeakOwningActivatableWidget.IsSet())
	{
		BindOwningActivatableWidget();
	}

	AttemptToActivateActiveWidget();
}

void UCommonActivatableWidgetSwitcher::HandleOwningWidgetActivationChanged(const bool bIsActivated)
{
	if (bIsActivated)
	{
		AttemptToActivateActiveWidget();
	}
	else
	{
		DeactivateActiveWidget();
	}
}

void UCommonActivatableWidgetSwitcher::AttemptToActivateActiveWidget()
{
	if (UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(GetActiveWidget()))
	{
		const UCommonActivatableWidget* OwningActivatableWidget = GetOwningActivatableWidget();

		// Only activate if our owning activatable widget is activated, or if we don't have an owning activatable widget
		if (OwningActivatableWidget == nullptr || OwningActivatableWidget->IsActivated())
		{
			ActivatableWidget->ActivateWidget();
		}
	}
}

void UCommonActivatableWidgetSwitcher::DeactivateActiveWidget()
{
	if (UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(GetActiveWidget()))
	{
		ActivatableWidget->DeactivateWidget();

		if (bClearFocusRestorationTargetOfDeactivatedWidgets)
		{
			ActivatableWidget->ClearFocusRestorationTarget();
		}
	}
}

void UCommonActivatableWidgetSwitcher::BindOwningActivatableWidget()
{
	UCommonActivatableWidget* OwningActivatableWidget = CommonUIUtils::GetOwningUserWidget<UCommonActivatableWidget>(this);

	if (GetOwningActivatableWidget() != OwningActivatableWidget)
	{
		UnbindOwningActivatableWidget();

		if (OwningActivatableWidget)
		{
			OwningActivatableWidget->OnActivated().AddUObject(this, &UCommonActivatableWidgetSwitcher::HandleOwningWidgetActivationChanged, true);
			OwningActivatableWidget->OnDeactivated().AddUObject(this, &UCommonActivatableWidgetSwitcher::HandleOwningWidgetActivationChanged, false);

			HandleOwningWidgetActivationChanged(OwningActivatableWidget->IsActivated());
		}
	}
	
	WeakOwningActivatableWidget = OwningActivatableWidget;
}

void UCommonActivatableWidgetSwitcher::UnbindOwningActivatableWidget()
{
	if (UCommonActivatableWidget* OwningActivatableWidget = GetOwningActivatableWidget())
	{
		OwningActivatableWidget->OnActivated().RemoveAll(this);
		OwningActivatableWidget->OnDeactivated().RemoveAll(this);
	}

	WeakOwningActivatableWidget.Reset();
}

UCommonActivatableWidget* UCommonActivatableWidgetSwitcher::GetOwningActivatableWidget() const
{
	return WeakOwningActivatableWidget.IsSet() ? WeakOwningActivatableWidget.GetValue().Get() : nullptr;
}

