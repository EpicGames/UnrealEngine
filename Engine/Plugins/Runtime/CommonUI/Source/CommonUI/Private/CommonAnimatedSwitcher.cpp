// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonAnimatedSwitcher.h"

#include "CommonWidgetPaletteCategories.h"
#include "Components/WidgetSwitcherSlot.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonAnimatedSwitcher)

UCommonAnimatedSwitcher::UCommonAnimatedSwitcher(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TransitionType(ECommonSwitcherTransition::FadeOnly)
	, TransitionCurveType(ETransitionCurve::CubicInOut)
	, TransitionDuration(0.4f)
{

}

void UCommonAnimatedSwitcher::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	bSetOnce = false;

	MyOverlay.Reset();
	MyInputGuard.Reset();
	MyAnimatedSwitcher.Reset();
}

#if WITH_EDITOR
const FText UCommonAnimatedSwitcher::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif

void UCommonAnimatedSwitcher::SetActiveWidgetIndex(int32 Index)
{
	SetActiveWidgetIndex_Internal(Index);
}

void UCommonAnimatedSwitcher::SetActiveWidget(UWidget* Widget)
{
	const int32 ChildIndex = GetChildIndex(Widget);
	SetActiveWidgetIndex_Internal(ChildIndex);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UCommonAnimatedSwitcher::ActivateNextWidget(bool bCanWrap)
{
	if (Slots.Num() > 1)
	{
		if (ActiveWidgetIndex == Slots.Num() - 1)
		{
			if (bCanWrap)
			{
				SetActiveWidgetIndex(0);
			}
		}
		else
		{
			SetActiveWidgetIndex(ActiveWidgetIndex + 1);
		}
	}
}

void UCommonAnimatedSwitcher::ActivatePreviousWidget(bool bCanWrap)
{
	if (Slots.Num() > 1)
	{
		if (ActiveWidgetIndex == 0)
		{
			if (bCanWrap)
			{
				SetActiveWidgetIndex(Slots.Num() - 1);
			}
		}
		else
		{
			SetActiveWidgetIndex(ActiveWidgetIndex - 1);
		}
	}
}

bool UCommonAnimatedSwitcher::HasWidgets() const
{
	return Slots.Num() > 0;
}

void UCommonAnimatedSwitcher::SetDisableTransitionAnimation(bool bDisableAnimation)
{
	bInstantTransition = bDisableAnimation;
}

bool UCommonAnimatedSwitcher::IsCurrentlySwitching() const
{
	return bCurrentlySwitching;
}

void UCommonAnimatedSwitcher::HandleSlateActiveIndexChanged(int32 ActiveIndex)
{
	if (Slots.IsValidIndex(ActiveWidgetIndex))
	{
		OnActiveWidgetIndexChanged.Broadcast(GetWidgetAtIndex(ActiveWidgetIndex), ActiveWidgetIndex);
		OnActiveWidgetIndexChangedBP.Broadcast(GetWidgetAtIndex(ActiveWidgetIndex), ActiveWidgetIndex);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TSharedRef<SWidget> UCommonAnimatedSwitcher::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyWidgetSwitcher = MyAnimatedSwitcher = SNew(SCommonAnimatedSwitcher)
		.InitialIndex(ActiveWidgetIndex)
		.TransitionCurveType(TransitionCurveType)
		.TransitionDuration(TransitionDuration)
		.TransitionType(TransitionType)
		.TransitionFallbackStrategy(TransitionFallbackStrategy)
		.OnActiveIndexChanged_UObject(this, &UCommonAnimatedSwitcher::HandleSlateActiveIndexChanged)
		.OnIsTransitioningChanged_UObject(this, &UCommonAnimatedSwitcher::HandleSlateIsTransitioningChanged);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (UPanelSlot* CurrentSlot : Slots)
	{
		if (UWidgetSwitcherSlot* TypedSlot = Cast<UWidgetSwitcherSlot>(CurrentSlot))
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyWidgetSwitcher.ToSharedRef());
		}
	}

	return SAssignNew(MyOverlay, SOverlay)
		+ SOverlay::Slot()
		[
			MyAnimatedSwitcher.ToSharedRef()
		]
		+ SOverlay::Slot()
		[
			SAssignNew(MyInputGuard, SSpacer)
			.Visibility(EVisibility::Collapsed)
		];
}

void UCommonAnimatedSwitcher::HandleSlateIsTransitioningChanged(bool bIsTransitioning)
{
	// While the switcher is transitioning, put up the guard to intercept all input
	MyInputGuard->SetVisibility(bIsTransitioning ? EVisibility::Visible : EVisibility::Collapsed);

	OnTransitioningChanged.Broadcast(bIsTransitioning);
}

void UCommonAnimatedSwitcher::SetActiveWidgetIndex_Internal(int32 Index)
{
#if WITH_EDITOR
	if (IsDesignTime())
	{
		Super::SetActiveWidgetIndex(Index);
		return;
	}
#endif
	
	TGuardValue<bool> bCurrentlySwitchingGuard(bCurrentlySwitching, true);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Index >= 0 && Index < Slots.Num() && (Index != ActiveWidgetIndex || !bSetOnce))
	{
		HandleOutgoingWidget();
		
		// When we're setting up, and the index goes 0->0, MyAnimatedSwitcher won't fire its ActiveIndexChanged Event, so we manually call HandleSlateActiveIndexChanged below.
		const bool bIsSettingInitialIndex = !bSetOnce && (!MyAnimatedSwitcher.IsValid() || Index == MyAnimatedSwitcher->GetActiveWidgetIndex());

		// For now we can't call setter since it calls MyWidgetSwitcher->SetActiveWidgetIndex(SafeIndex)
		if (ActiveWidgetIndex != Index)
		{
			ActiveWidgetIndex = Index;
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::ActiveWidgetIndex);
		}

		if (MyAnimatedSwitcher.IsValid())
		{
			// Ensure the index is clamped to a valid range.
			int32 SafeIndex = FMath::Clamp(ActiveWidgetIndex, 0, FMath::Max(0, Slots.Num() - 1));
			MyAnimatedSwitcher->TransitionToIndex(SafeIndex, bInstantTransition);
		}

		if (bIsSettingInitialIndex)
		{
			HandleSlateActiveIndexChanged(ActiveWidgetIndex);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		bSetOnce = true;
	}
}

bool UCommonAnimatedSwitcher::IsTransitionPlaying() const
{
	if (MyAnimatedSwitcher.IsValid())
	{
		return MyAnimatedSwitcher->IsTransitionPlaying();
	}
	else
	{
		return false;
	}
}

UWidget* UCommonAnimatedSwitcher::GetPendingActiveWidget() const
{
	if (MyAnimatedSwitcher.IsValid())
	{
		const int32 PendingIndex = GetPendingActiveWidgetIndex();
		if (PendingIndex >= 0)
		{
			return GetWidgetAtIndex(PendingIndex);
		}
	}
	
	return nullptr;
}

int32 UCommonAnimatedSwitcher::GetPendingActiveWidgetIndex() const
{
	if (MyAnimatedSwitcher.IsValid())
	{
		return MyAnimatedSwitcher->GetPendingActiveWidgetIndex();
	}
	return INDEX_NONE;
}

