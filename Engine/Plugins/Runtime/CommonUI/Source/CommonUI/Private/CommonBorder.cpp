// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonBorder.h"
#include "CommonUIEditorSettings.h"
#include "CommonWidgetPaletteCategories.h"

#include "Framework/Application/SlateApplication.h"
#include "ICommonUIModule.h"
#include "Misc/CoreDelegates.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SViewport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonBorder)

UCommonBorderStyle::UCommonBorderStyle()
{
}

void UCommonBorderStyle::GetBackgroundBrush(FSlateBrush& Brush) const
{
	Brush = Background;
}

UCommonBorder::UCommonBorder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bReducePaddingBySafezone(false)
	, MinimumPadding(0)
{
	Background.DrawAs = ESlateBrushDrawType::NoDrawType;

	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UCommonBorder::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// We will remove this once existing content is fixed up. Since previously the native CDO was actually the default style, this code will attempt to set the style on assets that were once using this default
	if (!Style && !bStyleNoLongerNeedsConversion && !IsRunningDedicatedServer())
	{
		UCommonBorder* CDO = Cast<UCommonBorder>(GetClass()->GetDefaultObject());
		if (Background == CDO->Background)
		{
			UCommonUIEditorSettings& Settings = ICommonUIModule::GetEditorSettings();
			Settings.ConditionalPostLoad();
			Style = Settings.GetTemplateBorderStyle();
		}
	}

	bStyleNoLongerNeedsConversion = true;
#endif
}

void UCommonBorder::SetStyle(TSubclassOf<UCommonBorderStyle> InStyle)
{
	Style = InStyle;

	if (MyBorder.IsValid())
	{
		// Don't synchronize properties if there is no slate widget.
		SynchronizeProperties();
	}
}

void UCommonBorder::ReleaseSlateResources(bool bReleaseChildren)
{
	if (bReducePaddingBySafezone)
	{
		FCoreDelegates::OnSafeFrameChangedEvent.RemoveAll(this);
	}

	Super::ReleaseSlateResources(bReleaseChildren);
}


TSharedRef<SWidget> UCommonBorder::RebuildWidget()
{
	if (bReducePaddingBySafezone)
	{
		FCoreDelegates::OnSafeFrameChangedEvent.AddUObject(this, &UCommonBorder::SafeAreaUpdated);
#if WITH_EDITOR
		FSlateApplication::Get().OnDebugSafeZoneChanged.AddUObject(this, &UCommonBorder::DebugSafeAreaUpdated);
#endif

	}

	return Super::RebuildWidget();
}

void UCommonBorder::SynchronizeProperties()
{
	if (const UCommonBorderStyle* BorderStyle = GetStyleCDO())
	{
		// Update our styling to match the assigned style
		Background = BorderStyle->Background;
	}

	Super::SynchronizeProperties();

	SafeAreaUpdated();
}

void UCommonBorder::SafeAreaUpdated()
{
	FMargin SafeMargin;
	if (bReducePaddingBySafezone && MyBorder.IsValid())
	{
#if WITH_EDITOR
		if (DesignerSize.IsSet() && !DesignerSize.GetValue().IsZero())
		{
			FSlateApplication::Get().GetSafeZoneSize(SafeMargin, DesignerSize.GetValue());
		}
		else
#endif
		{
			// Need to get owning viewport not display 
			// use pixel values (same as custom safe zone above)
			TSharedPtr<SViewport> GameViewport = FSlateApplication::Get().GetGameViewport();
			if (GameViewport.IsValid())
			{
				TSharedPtr<ISlateViewport> ViewportInterface = GameViewport->GetViewportInterface().Pin();
				if (ViewportInterface.IsValid())
				{
					const FIntPoint ViewportSize = ViewportInterface->GetSize();
					FSlateApplication::Get().GetSafeZoneSize(SafeMargin, ViewportSize);
				}
			}
		}

		FMargin NewMargin;
		NewMargin.Left = FMath::Max(MinimumPadding.Left, GetPadding().Left - SafeMargin.Left);
		NewMargin.Right = FMath::Max(MinimumPadding.Right, GetPadding().Right - SafeMargin.Right);
		NewMargin.Top = FMath::Max(MinimumPadding.Top, GetPadding().Top - SafeMargin.Top);
		NewMargin.Bottom = FMath::Max(MinimumPadding.Bottom, GetPadding().Bottom - SafeMargin.Bottom);
		
		MyBorder->SetPadding(NewMargin);
	}
}

#if WITH_EDITOR
void UCommonBorder::OnCreationFromPalette()
{
	bStyleNoLongerNeedsConversion = true;
	if (!Style)
	{
		Style = ICommonUIModule::GetEditorSettings().GetTemplateBorderStyle();
	}
	Super::OnCreationFromPalette();
}

const FText UCommonBorder::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

void UCommonBorder::OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs)
{
	if (EventArgs.bScreenPreview)
	{
		DesignerSize = EventArgs.Size;
	}
	else
	{
		DesignerSize = FVector2D(0, 0);
	}

	SafeAreaUpdated();
}

#endif // WITH_EDITOR

const UCommonBorderStyle* UCommonBorder::GetStyleCDO() const
{
	if (Style)
	{
		if (const UCommonBorderStyle* BorderStyle = Cast<UCommonBorderStyle>(Style->GetDefaultObject(false)))
		{
			return BorderStyle;
		}
	}
	return nullptr;
}

