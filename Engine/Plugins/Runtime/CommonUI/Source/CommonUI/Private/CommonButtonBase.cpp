// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonButtonBase.h"

#include "Binding/States/WidgetStateBitfield.h"
#include "Binding/States/WidgetStateRegistration.h"
#include "CommonActionWidget.h"
#include "CommonUISubsystemBase.h"
#include "CommonInputSubsystem.h"
#include "CommonUIEditorSettings.h"
#include "CommonWidgetPaletteCategories.h"
#include "Components/ButtonSlot.h"
#include "Blueprint/WidgetTree.h"
#include "CommonButtonTypes.h"
#include "CommonUITypes.h"
#include "ICommonUIModule.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/UserInterfaceSettings.h"
#include "ICommonInputModule.h"
#include "CommonInputSettings.h"
#include "Input/CommonUIInputTypes.h"
#include "InputAction.h"
#include "Sound/SoundBase.h"
#include "Slate/UMGDragDropOp.h"
#include "Styling/UMGCoreStyle.h"
#include "CommonUITypes.h"
#include "CommonUIPrivate.h"
#include "Input/CommonUIActionRouterBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonButtonBase)

namespace UE::CommonUI::Private
{
	int32 EnableSimulateHoverOnTouchInput = 1;
	FAutoConsoleVariableRef CVarEnableSimulateHoverOnTouchInput(
		TEXT("CommonButton.EnableSimulateHoverOnTouch"),
		EnableSimulateHoverOnTouchInput,
		TEXT("Allows buttons to simulate hovering on touch in accordance with the property SimulateHoverOnTouchInput.\n0: Disable, 1: Enable (default), 2: Legacy Mode (Deprecated)\n\nNote on Legacy Mode: This mode was previously the default (CommonButton.EnableSimulateHoverOnTouch=1) and is preserved for retro-compatibility with CommonButton.EnableSimulateHoverOnTouch=2 but will be removed in a future update. In this legacy implementation, the property SimulateHoverOnTouch=true simulates Hover events as expected. However, SimulateHoverOnTouch=false was not blocking all hover events on touch. This implementation was simply preventing the Press & Release functions from simulating more hover events. This implementation was causing inconsistent behaviors for widgets with SimulateHoverOnTouch=false. The new implementation will effectively block all Hover events for UCommonButtonBase if the property SimulateHoverOnTouch is set to false in the editor. This ensures a consistent behavior for touch input so a widget can be built without any Hover events being simulated on a touch screen."),
		ECVF_Default);

	bool bDisableDimensionValueSetInDesignTime = false;
	FAutoConsoleVariableRef CVarDisableDimensionValueSetInDesignTime(
		TEXT("CommonButton.DisableDimensionValueSetInDesignTime"),
		bDisableDimensionValueSetInDesignTime,
		TEXT("Block SetMin/MaxDimension to set directly the dimension variable (MinHeight, MaxHeight,MinWidth, MaxWidth) during design time to prevent possible data corruptoion from Mobile value"),
		ECVF_Default);
}

//////////////////////////////////////////////////////////////////////////
// UCommonButtonStyle
//////////////////////////////////////////////////////////////////////////

bool UCommonButtonStyle::NeedsLoadForServer() const
{
	return GetDefault<UUserInterfaceSettings>()->bLoadWidgetsOnDedicatedServer;
}

void UCommonButtonStyle::GetButtonPadding(FMargin& OutButtonPadding) const
{
	OutButtonPadding = ButtonPadding;
}

void UCommonButtonStyle::GetCustomPadding(FMargin& OutCustomPadding) const
{
	OutCustomPadding = CustomPadding;
}

UCommonTextStyle* UCommonButtonStyle::GetNormalTextStyle() const
{
	if (NormalTextStyle)
	{
		if (UCommonTextStyle* TextStyle = Cast<UCommonTextStyle>(NormalTextStyle->GetDefaultObject(false)))
		{
			return TextStyle;
		}
	}
	return nullptr;
}

UCommonTextStyle* UCommonButtonStyle::GetNormalHoveredTextStyle() const
{
	if (NormalHoveredTextStyle)
	{
		if (UCommonTextStyle* TextStyle = Cast<UCommonTextStyle>(NormalHoveredTextStyle->GetDefaultObject(false)))
		{
			return TextStyle;
		}
	}
	return nullptr;
}

UCommonTextStyle* UCommonButtonStyle::GetSelectedTextStyle() const
{
	if (SelectedTextStyle)
	{
		if (UCommonTextStyle* TextStyle = Cast<UCommonTextStyle>(SelectedTextStyle->GetDefaultObject(false)))
		{
			return TextStyle;
		}
	}
	return nullptr;
}

UCommonTextStyle* UCommonButtonStyle::GetSelectedHoveredTextStyle() const
{
	if (SelectedHoveredTextStyle)
	{
		if (UCommonTextStyle* TextStyle = Cast<UCommonTextStyle>(SelectedHoveredTextStyle->GetDefaultObject(false)))
		{
			return TextStyle;
		}
	}
	return nullptr;
}

UCommonTextStyle* UCommonButtonStyle::GetDisabledTextStyle() const
{
	if (DisabledTextStyle)
	{
		if (UCommonTextStyle* TextStyle = Cast<UCommonTextStyle>(DisabledTextStyle->GetDefaultObject(false)))
		{
			return TextStyle;
		}
	}
	return nullptr;
}

void UCommonButtonStyle::GetMaterialBrush(FSlateBrush& Brush) const
{
	Brush = SingleMaterialBrush;
}

void UCommonButtonStyle::GetNormalBaseBrush(FSlateBrush& Brush) const
{
	Brush = NormalBase;
}

void UCommonButtonStyle::GetNormalHoveredBrush(FSlateBrush& Brush) const
{
	Brush = NormalHovered;
}

void UCommonButtonStyle::GetNormalPressedBrush(FSlateBrush& Brush) const
{
	Brush = NormalPressed;
}

void UCommonButtonStyle::GetSelectedBaseBrush(FSlateBrush& Brush) const
{
	Brush = SelectedBase;
}

void UCommonButtonStyle::GetSelectedHoveredBrush(FSlateBrush& Brush) const
{
	Brush = SelectedHovered;
}

void UCommonButtonStyle::GetSelectedPressedBrush(FSlateBrush& Brush) const
{
	Brush = SelectedPressed;
}

void UCommonButtonStyle::GetDisabledBrush(FSlateBrush& Brush) const
{
	Brush = Disabled;
}

//////////////////////////////////////////////////////////////////////////
// UCommonButtonInternalBase
//////////////////////////////////////////////////////////////////////////

static int32 bUseTransparentButtonStyleAsDefault = 0;
static FAutoConsoleVariableRef CVarUseTransparentButtonStyleAsDefault(
	TEXT("UseTransparentButtonStyleAsDefault"),
	bUseTransparentButtonStyleAsDefault,
	TEXT("If true, the default Button Style for the CommonButtonBase's SButton will be set to NoBorder, which has a transparent background and no padding"));

UCommonButtonInternalBase::UCommonButtonInternalBase(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, bButtonEnabled(true)
	, bInteractionEnabled(true)
{
	if (bUseTransparentButtonStyleAsDefault)
	{
		// SButton will have a transparent background and have no padding if Button Style is set to None
		static const FButtonStyle* TransparentButtonStyle = new FButtonStyle(FUMGCoreStyle::Get().GetWidgetStyle<FButtonStyle>("NoBorder"));
		SetStyle(*TransparentButtonStyle);
	}
}

void UCommonButtonInternalBase::SetButtonEnabled(bool bInIsButtonEnabled)
{
	bButtonEnabled = bInIsButtonEnabled;
	if (MyCommonButton.IsValid())
	{
		MyCommonButton->SetIsButtonEnabled(bInIsButtonEnabled);
	}
}

void UCommonButtonInternalBase::SetButtonFocusable(bool bInIsButtonFocusable)
{
	InitIsFocusable(bInIsButtonFocusable);
	if (MyCommonButton.IsValid())
	{
		MyCommonButton->SetIsButtonFocusable(bInIsButtonFocusable);
	}
}

void UCommonButtonInternalBase::SetInteractionEnabled(bool bInIsInteractionEnabled)
{
	if (bInteractionEnabled == bInIsInteractionEnabled)
	{
		return;
	}

	bInteractionEnabled = bInIsInteractionEnabled;
	if (MyCommonButton.IsValid())
	{
		MyCommonButton->SetIsInteractionEnabled(bInIsInteractionEnabled);
	}
}

bool UCommonButtonInternalBase::IsHovered() const
{
	if (MyCommonButton.IsValid())
	{
		return MyCommonButton->IsHovered();
	}
	return false;
}

bool UCommonButtonInternalBase::IsPressed() const
{
	if (MyCommonButton.IsValid())
	{
		return MyCommonButton->IsPressed();
	}
	return false;
}

void UCommonButtonInternalBase::SetMinDesiredHeight(int32 InMinHeight)
{
	MinHeight = InMinHeight;
	if (MyBox.IsValid())
	{
		MyBox->SetMinDesiredHeight(InMinHeight);
	}
}

void UCommonButtonInternalBase::SetMinDesiredWidth(int32 InMinWidth)
{
	MinWidth = InMinWidth;
	if (MyBox.IsValid())
	{
		MyBox->SetMinDesiredWidth(InMinWidth);
	}
}

void UCommonButtonInternalBase::SetMaxDesiredHeight(int32 InMaxHeight)
{
	MaxHeight = InMaxHeight;
	if (MyBox.IsValid())
	{
		MyBox->SetMaxDesiredHeight(InMaxHeight > 0 ? InMaxHeight : FOptionalSize());
	}
}

void UCommonButtonInternalBase::SetMaxDesiredWidth(int32 InMaxWidth)
{
	MaxWidth = InMaxWidth;
	if (MyBox.IsValid())
	{
		MyBox->SetMaxDesiredWidth(InMaxWidth > 0 ? InMaxWidth : FOptionalSize());
	}
}

TSharedRef<SWidget> UCommonButtonInternalBase::RebuildWidget()
{
	MyButton = MyCommonButton = SNew(SCommonButton)
		.OnClicked(BIND_UOBJECT_DELEGATE(FOnClicked, SlateHandleClickedOverride))
		.OnDoubleClicked(BIND_UOBJECT_DELEGATE(FOnClicked, SlateHandleDoubleClicked))
		.OnPressed(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandlePressedOverride))
		.OnReleased(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleReleasedOverride))
		.OnSlateButtonDragDetected(BIND_UOBJECT_DELEGATE(FOnDragDetected, SlateHandleDragDetectedOverride))
		.OnSlateButtonDragEnter(BIND_UOBJECT_DELEGATE(FOnDragEnter, SlateHandleDragEnterOverride))
		.OnSlateButtonDragLeave(BIND_UOBJECT_DELEGATE(FOnDragLeave, SlateHandleDragLeaveOverride))
		.OnSlateButtonDragOver(BIND_UOBJECT_DELEGATE(FOnDragOver, SlateHandleDragOverOverride))
		.OnSlateButtonDrop(BIND_UOBJECT_DELEGATE(FOnDrop, SlateHandleDrop))
		.ButtonStyle(&GetStyle())
		.ClickMethod(GetClickMethod())
		.TouchMethod(GetTouchMethod())
		.IsFocusable(GetIsFocusable())
		.IsButtonEnabled(bButtonEnabled)
		.IsInteractionEnabled(bInteractionEnabled)
		.OnReceivedFocus(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleOnReceivedFocus))
		.OnLostFocus(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleOnLostFocus));

	MyBox = SNew(SBox)
		.MinDesiredWidth(MinWidth)
		.MinDesiredHeight(MinHeight)
		.MaxDesiredWidth(MaxWidth > 0 ? MaxWidth : FOptionalSize())
		.MaxDesiredHeight(MaxHeight > 0 ? MaxHeight : FOptionalSize())
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			MyCommonButton.ToSharedRef()
		];

	if (GetChildrenCount() > 0)
	{
		Cast<UButtonSlot>(GetContentSlot())->BuildSlot(MyCommonButton.ToSharedRef());
	}

	return MyBox.ToSharedRef();
}

void UCommonButtonInternalBase::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	MyCommonButton.Reset();
	MyBox.Reset();
}

FReply UCommonButtonInternalBase::SlateHandleClickedOverride()
{
	return Super::SlateHandleClicked();
}

void UCommonButtonInternalBase::SlateHandlePressedOverride()
{
	Super::SlateHandlePressed();
}

void UCommonButtonInternalBase::SlateHandleReleasedOverride()
{
	Super::SlateHandleReleased();
}

FReply UCommonButtonInternalBase::SlateHandleDoubleClicked()
{
	FReply Reply = FReply::Unhandled();
	if (HandleDoubleClicked.IsBound())
	{
		Reply = HandleDoubleClicked.Execute();
	}

	if (OnDoubleClicked.IsBound())
	{
		OnDoubleClicked.Broadcast();
		Reply = FReply::Handled();
	}

	return Reply;
}

/** Begin Drag Drop
*	These overrides are currently identical to the slate handlers found in UButton.  Adding these stubs for ease of access/ override functionality
*/
FReply UCommonButtonInternalBase::SlateHandleDragDetectedOverride(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{
	if (OnButtonDragDetected.IsBound())
	{
		return OnButtonDragDetected.Execute(MyGeometry, PointerEvent);
	}
	return FReply::Unhandled();
}

void UCommonButtonInternalBase::SlateHandleDragEnterOverride(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	OnButtonDragEnter.ExecuteIfBound(MyGeometry, DragDropEvent);
}

void UCommonButtonInternalBase::SlateHandleDragLeaveOverride(const FDragDropEvent& DragDropEvent)
{
	OnButtonDragLeave.ExecuteIfBound(DragDropEvent);
}

FReply UCommonButtonInternalBase::SlateHandleDragOverOverride(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnButtonDragOver.IsBound())
	{
		return OnButtonDragOver.Execute(MyGeometry, DragDropEvent);
	}
	return FReply::Unhandled();
}

FReply UCommonButtonInternalBase::SlateHandleDropOverride(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnButtonDrop.IsBound())
	{
		return OnButtonDrop.Execute(MyGeometry, DragDropEvent);
	}
	return FReply::Unhandled();
}
//	End Drag Drop

//////////////////////////////////////////////////////////////////////////
// UCommonButtonBase
//////////////////////////////////////////////////////////////////////////

UCommonButtonBase::UCommonButtonBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MinWidth(0)
	, MinHeight(0)
	, MaxWidth(0)
	, MaxHeight(0)
	, bApplyAlphaOnDisable(true)
	, bLocked(false)
	, bSelectable(false)
	, bShouldSelectUponReceivingFocus(false)
	, bToggleable(false)
	, bTriggerClickedAfterSelection(false)
	, bDisplayInputActionWhenNotInteractable(true)
	, bShouldUseFallbackDefaultInputAction(true)
	, bRequiresHold(false)
	, bSimulateHoverOnTouchInput(true)
	, bSelected(false)
	, bButtonEnabled(true)
	, bInteractionEnabled(true)
	, bNavigateToNextWidgetOnDisable(false)
	, HoldTime(0.f)
	, HoldRollbackTime(0.f)
	, CurrentHoldTime(0.f)
	, CurrentHoldProgress(0.f)
{
	SetIsFocusable(true);
}

void UCommonButtonBase::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();
	GetCachedWidget()->AddMetadata<FCommonButtonMetaData>(MakeShared<FCommonButtonMetaData>(*this));	
}

void UCommonButtonBase::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// We will remove this once existing content is fixed up. Since previously the native CDO was actually the default style, this code will attempt to set the style on assets that were once using this default
	if (!Style && !bStyleNoLongerNeedsConversion && !IsRunningDedicatedServer())
	{
		UCommonUIEditorSettings& Settings = ICommonUIModule::GetEditorSettings();
		Settings.ConditionalPostLoad();
		Style = Settings.GetTemplateButtonStyle();
	}
	bStyleNoLongerNeedsConversion = true;
#endif
}

void UCommonButtonBase::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	RefreshDimensions();
	BuildStyles();
}

#if WITH_EDITOR
void UCommonButtonBase::OnCreationFromPalette()
{
	bStyleNoLongerNeedsConversion = true;
	if (!Style)
	{
		Style = ICommonUIModule::GetEditorSettings().GetTemplateButtonStyle();
	}
	if (!HoldData && ICommonInputModule::GetSettings().GetDefaultHoldData())
	{
		HoldData = ICommonInputModule::GetSettings().GetDefaultHoldData();
	}
	Super::OnCreationFromPalette();
}

const FText UCommonButtonBase::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif // WITH_EDITOR

bool UCommonButtonBase::Initialize()
{
	const bool bInitializedThisCall = Super::Initialize();

	if (bInitializedThisCall)
	{
		UCommonButtonInternalBase* RootButtonRaw = ConstructInternalButton();

		RootButtonRaw->SetClickMethod(ClickMethod);
		RootButtonRaw->SetTouchMethod(TouchMethod);
		RootButtonRaw->SetPressMethod(PressMethod);
		//Force the RootButton to not be focusable if it has a DesiredFocusWidgetName set which was stealing the focus and preventing DesiredFocusWidget from getting the FocusReceived event.
		RootButtonRaw->SetButtonFocusable(GetDesiredFocusWidgetName().IsNone() && IsFocusable());
		RootButtonRaw->SetButtonEnabled(bButtonEnabled);
		RootButtonRaw->SetInteractionEnabled(bInteractionEnabled);
		RootButton = RootButtonRaw;

		if (WidgetTree->RootWidget)
		{
			UButtonSlot* NewSlot = Cast<UButtonSlot>(RootButtonRaw->AddChild(WidgetTree->RootWidget));
			NewSlot->SetPadding(FMargin());
			NewSlot->SetHorizontalAlignment(EHorizontalAlignment::HAlign_Fill);
			NewSlot->SetVerticalAlignment(EVerticalAlignment::VAlign_Fill);
			WidgetTree->RootWidget = RootButtonRaw;

			RootButton->OnClicked.AddUniqueDynamic(this, &UCommonButtonBase::HandleButtonClicked);
			RootButton->HandleDoubleClicked.BindUObject(this, &UCommonButtonBase::HandleButtonDoubleClicked);
			RootButton->OnReceivedFocus.BindUObject(this, &UCommonButtonBase::HandleFocusReceived);
			RootButton->OnLostFocus.BindUObject(this, &UCommonButtonBase::HandleFocusLost);
			RootButton->OnPressed.AddUniqueDynamic(this, &UCommonButtonBase::HandleButtonPressed);
			RootButton->OnReleased.AddUniqueDynamic(this, &UCommonButtonBase::HandleButtonReleased);
			RootButton->OnButtonDragDetected.BindUObject(this, &UCommonButtonBase::HandleButtonDragDetected);
			RootButton->OnButtonDragEnter.BindUObject(this, &UCommonButtonBase::HandleButtonDragEnter);
			RootButton->OnButtonDragLeave.BindUObject(this, &UCommonButtonBase::HandleButtonDragLeave);
			RootButton->OnButtonDragOver.BindUObject(this, &UCommonButtonBase::HandleButtonDragOver);
			RootButton->OnButtonDrop.BindUObject(this, &UCommonButtonBase::HandleButtonDrop);
		}
	}

	return bInitializedThisCall;
}

UCommonButtonInternalBase* UCommonButtonBase::ConstructInternalButton()
{
	return WidgetTree->ConstructWidget<UCommonButtonInternalBase>(UCommonButtonInternalBase::StaticClass(), FName(TEXT("InternalRootButtonBase")));
}

void UCommonButtonBase::NativeConstruct()
{
	if (!HoldData && ICommonInputModule::GetSettings().GetDefaultHoldData())
	{
		HoldData = ICommonInputModule::GetSettings().GetDefaultHoldData();
	}

	BindTriggeringInputActionToClick();
	BindInputMethodChangedDelegate();
	UpdateInputActionWidget();

	Super::NativeConstruct();
}

void UCommonButtonBase::NativeDestruct()
{
	Super::NativeDestruct();

	UnbindTriggeringInputActionToClick();
	UnbindInputMethodChangedDelegate();

	if (HoldTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(HoldTickerHandle);
		HoldTickerHandle = nullptr;
	}
	if (HoldProgressRollbackTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(HoldProgressRollbackTickerHandle);
		HoldProgressRollbackTickerHandle = nullptr;
	}
}

void UCommonButtonBase::SetIsEnabled(bool bInIsEnabled)
{
	bool bValueChanged = bButtonEnabled != bInIsEnabled;

	// Change the underlying enabled bool but do not broadcast because we don't want to propagate it to the underlying SWidget
	bool bOldBroadcastState = bShouldBroadcastState;
	bShouldBroadcastState = false;
	if (bInIsEnabled)
	{
		Super::SetIsEnabled(bInIsEnabled);
		EnableButton();
	}
	else
	{
		Super::SetIsEnabled(bInIsEnabled);
		DisableButton();
	}
	bShouldBroadcastState = bOldBroadcastState;

	if (bValueChanged)
	{
		// Note: State is disabled, so we broadcast !bIsEnabled
		BroadcastBinaryPostStateChange(UWidgetDisabledStateRegistration::Bit, !bInIsEnabled);

		HandleImplicitFocusLost();
	}
}


void UCommonButtonBase::SetVisibility(ESlateVisibility InVisibility)
{
	bool bValueChanged = InVisibility != GetVisibility();
	
	Super::SetVisibility(InVisibility);

	if (bValueChanged)
	{
		HandleImplicitFocusLost();
	}
}


bool UCommonButtonBase::NativeIsInteractable() const
{
	// If it's enabled, it's "interactable" from a UMG perspective. 
	// For now this is how we generate friction on the analog cursor, which we still want for disabled buttons since they have tooltips.
	return GetIsEnabled();
}

void UCommonButtonBase::BindInputMethodChangedDelegate()
{
	UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();
	if (CommonInputSubsystem)
	{
		CommonInputSubsystem->OnInputMethodChangedNative.AddUObject(this, &UCommonButtonBase::OnInputMethodChanged);
		UpdateHoldData(CommonInputSubsystem->GetDefaultInputType());
	}
}

void UCommonButtonBase::UnbindInputMethodChangedDelegate()
{
	UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();
	if (CommonInputSubsystem)
	{
		CommonInputSubsystem->OnInputMethodChangedNative.RemoveAll(this);
	}
}

void UCommonButtonBase::OnInputMethodChanged(ECommonInputType CurrentInputType)
{
	UpdateInputActionWidget();
	UpdateHoldData(CurrentInputType);
	HoldReset();
	NativeOnActionProgress(0.f);
	BP_OnInputMethodChanged(CurrentInputType);

	if (TriggeringBindingHandle.IsValid())
	{
		TriggeringBindingHandle.ResetHold();
	}
}

bool UCommonButtonBase::IsHoverSimulationOnTouchAvailable() const
{
	return UE::CommonUI::Private::EnableSimulateHoverOnTouchInput != 0;
}

bool UCommonButtonBase::ShouldProcessHoverEvent(EHoverEventSource HoverReason)
{
	if (const UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(GetOwningLocalPlayer()))
	{
		if (CommonInputSubsystem->GetCurrentInputType() == ECommonInputType::Touch)
		{
			if (HoverReason == EHoverEventSource::SimulationForTouch)
			{
				// disabled mode: blocks internally simulated Hover events
				if (UE::CommonUI::Private::EnableSimulateHoverOnTouchInput == 0)
				{
					return false;
				}

				// legacy mode: blocks internally simulated Hover events when bSimulateHoverOnTouchInput is false
				if (UE::CommonUI::Private::EnableSimulateHoverOnTouchInput == 2 && !bSimulateHoverOnTouchInput)
				{
					return false;
				}
			}

			// strict mode: blocks ALL Hover events when bSimulateHoverOnTouchInput is false
			if (UE::CommonUI::Private::EnableSimulateHoverOnTouchInput == 1 && !bSimulateHoverOnTouchInput)
			{
				return false;
			}

			// If we do not explicitly want to block hover events on touch for that button, it should get processed.
			return true;
		}
	}

	// On all input methods except touch, process all events except the ones simulated for touch.
	return HoverReason != EHoverEventSource::SimulationForTouch;
}

void UCommonButtonBase::UpdateHoldData(ECommonInputType CurrentInputType)
{
	if (HoldData && bRequiresHold)
	{
		UCommonUIHoldData* CommonUIHoldBehaviorValues = HoldData.GetDefaultObject();
		if (CommonUIHoldBehaviorValues)
		{
			switch (CurrentInputType)
			{
			case ECommonInputType::MouseAndKeyboard:
				HoldTime = CommonUIHoldBehaviorValues->KeyboardAndMouse.HoldTime;
				HoldRollbackTime = CommonUIHoldBehaviorValues->KeyboardAndMouse.HoldRollbackTime;
				break;
			case ECommonInputType::Gamepad:
				HoldTime = CommonUIHoldBehaviorValues->Gamepad.HoldTime;
				HoldRollbackTime = CommonUIHoldBehaviorValues->Gamepad.HoldRollbackTime;
				break;
			case ECommonInputType::Touch:
				HoldTime = CommonUIHoldBehaviorValues->Touch.HoldTime;
				HoldRollbackTime = CommonUIHoldBehaviorValues->Touch.HoldRollbackTime;
				break;
			default:
				break;
			}
		}
	}
}

void UCommonButtonBase::BindTriggeringInputActionToClick()
{
	if (!TriggeredInputAction.IsNull())
	{
		return;
	}

	if (CommonUI::IsEnhancedInputSupportEnabled() && TriggeringEnhancedInputAction && !TriggeringBindingHandle.IsValid())
	{
		FBindUIActionArgs BindArgs(TriggeringEnhancedInputAction, false, FSimpleDelegate::CreateUObject(this, &UCommonButtonBase::HandleTriggeringActionCommited));
		BindArgs.OnHoldActionProgressed.BindUObject(this, &UCommonButtonBase::NativeOnActionProgress);
		BindArgs.OnHoldActionPressed.BindUObject(this, &UCommonButtonBase::NativeOnPressed);
		BindArgs.OnHoldActionReleased.BindUObject(this, &UCommonButtonBase::NativeOnReleased);
		BindArgs.bIsPersistent = bIsPersistentBinding;

		BindArgs.InputMode = InputModeOverride;

		TriggeringBindingHandle = RegisterUIActionBinding(BindArgs);
	}
	else if (!TriggeringInputAction.IsNull() && !TriggeringBindingHandle.IsValid())
	{
		FBindUIActionArgs BindArgs(TriggeringInputAction, false, FSimpleDelegate::CreateUObject(this, &UCommonButtonBase::HandleTriggeringActionCommited));
		BindArgs.OnHoldActionProgressed.BindUObject(this, &UCommonButtonBase::NativeOnActionProgress);
		BindArgs.OnHoldActionPressed.BindUObject(this, &UCommonButtonBase::NativeOnPressed);
		BindArgs.OnHoldActionReleased.BindUObject(this, &UCommonButtonBase::NativeOnReleased);
		BindArgs.bIsPersistent = bIsPersistentBinding;
		BindArgs.bForceHold = GetConvertInputActionToHold();

		BindArgs.InputMode = InputModeOverride;

		TriggeringBindingHandle = RegisterUIActionBinding(BindArgs);
	}
}

void UCommonButtonBase::UnbindTriggeringInputActionToClick()
{	
	if (!TriggeredInputAction.IsNull())
	{
		return;
	}

	if (TriggeringBindingHandle.IsValid())
	{
		TriggeringBindingHandle.Unregister();
	}

	CurrentHoldTime = 0.f;
	CurrentHoldProgress = 0.f;
}

void UCommonButtonBase::HandleTriggeringActionCommited(bool& bPassthrough)
{
	HandleTriggeringActionCommited();
}

void UCommonButtonBase::HandleTriggeringActionCommited()
{
	if (IsInteractionEnabled())
	{
		// Because this path doesn't go through SButton::Press(), the sound needs to be played from here.
		FSlateApplication::Get().PlaySound(NormalStyle.PressedSlateSound);
		BP_OnInputActionTriggered();
	}
	HandleButtonClicked();
}

void UCommonButtonBase::DisableButtonWithReason(const FText& DisabledReason)
{
	DisabledTooltipText = DisabledReason;
	SetIsEnabled(false);
}

void UCommonButtonBase::SetIsInteractionEnabled(bool bInIsInteractionEnabled)
{
	if (bInteractionEnabled == bInIsInteractionEnabled)
	{
		return;
	}

	const bool bWasHovered = IsHovered();

	bInteractionEnabled = bInIsInteractionEnabled;

	if (RootButton.IsValid())
	{
		if (bInteractionEnabled)
		{
			// If this is a selected and not-toggleable button, don't enable root button interaction
			if (!GetSelected() || bToggleable)
			{
				RootButton->SetInteractionEnabled(true);
			}

			if (bApplyAlphaOnDisable)
			{
				FLinearColor ButtonColor = RootButton->GetColorAndOpacity();
				ButtonColor.A = 1.f;
				RootButton->SetColorAndOpacity(ButtonColor);
			}
		}
		else
		{
			RootButton->SetInteractionEnabled(false);

			if (bApplyAlphaOnDisable)
			{
				FLinearColor ButtonColor = RootButton->GetColorAndOpacity();
				ButtonColor.A = 0.5f;
				RootButton->SetColorAndOpacity(ButtonColor);
			}
		}
	}

	UpdateInputActionWidgetVisibility();

	if (ShouldProcessHoverEvent(EHoverEventSource::InteractabilityChanged))
	{
		// If the hover state changed due to an interactability change, trigger internal logic accordingly.
		const bool bIsHoveredNow = IsHovered();
		if (bWasHovered != bIsHoveredNow)
		{
			if (bIsHoveredNow)
			{
				NativeOnHovered();
			}
			else
			{
				NativeOnUnhovered();
			}
		}
	}

	SetButtonStyle();
}

void UCommonButtonBase::SetHideInputAction(bool bInHideInputAction)
{
	bHideInputAction = bInHideInputAction;

	UpdateInputActionWidgetVisibility();
}

void UCommonButtonBase::SetHideInputActionWithKeyboard(bool bInHideInputActionWithKeyboard)
{
	bHideInputActionWithKeyboard = bInHideInputActionWithKeyboard;

	UpdateInputActionWidgetVisibility();
}

bool UCommonButtonBase::IsInteractionEnabled() const
{
	ESlateVisibility Vis = GetVisibility(); // hidden or collapsed should have 'bInteractionEnabled' set false, but sometimes they don't :(
	return GetIsEnabled() && bButtonEnabled && bInteractionEnabled && (Vis != ESlateVisibility::Collapsed) && (Vis != ESlateVisibility::Hidden);
}

bool UCommonButtonBase::IsHovered() const
{
	return RootButton.IsValid() && RootButton->IsHovered();
}

bool UCommonButtonBase::IsPressed() const
{
	return RootButton.IsValid() && RootButton->IsPressed();
}

void UCommonButtonBase::SetClickMethod(EButtonClickMethod::Type InClickMethod)
{
	ClickMethod = InClickMethod;
	if (RootButton.IsValid())
	{
		RootButton->SetClickMethod(ClickMethod);
	}
}

void UCommonButtonBase::SetTouchMethod(EButtonTouchMethod::Type InTouchMethod)
{
	TouchMethod = InTouchMethod;
	if (RootButton.IsValid())
	{
		RootButton->SetTouchMethod(InTouchMethod);
	}
}

void UCommonButtonBase::SetPressMethod(EButtonPressMethod::Type InPressMethod)
{
	PressMethod = InPressMethod;
	if (RootButton.IsValid())
	{
		RootButton->SetPressMethod(InPressMethod);
	}
}

void UCommonButtonBase::SetIsSelectable(bool bInIsSelectable)
{
	if (bInIsSelectable != bSelectable)
	{
		bSelectable = bInIsSelectable;

		if (bSelected && !bInIsSelectable)
		{
			SetSelectedInternal(false);
		}
	}
}

void UCommonButtonBase::SetIsInteractableWhenSelected(bool bInInteractableWhenSelected)
{
	if (bInInteractableWhenSelected != bInteractableWhenSelected)
	{
		bInteractableWhenSelected = bInInteractableWhenSelected;
		if (GetSelected() && !bToggleable)
		{
			SetIsInteractionEnabled(bInInteractableWhenSelected);
		}
	}
}

bool UCommonButtonBase::GetConvertInputActionToHold()
{
	return bRequiresHold;
}

void UCommonButtonBase::NativeOnActionProgress(float HeldPercent)
{
	if (InputActionWidget)
	{
		InputActionWidget->OnActionProgress(HeldPercent);
	}
	OnActionProgress(HeldPercent);
	CurrentHoldProgress = HeldPercent;
}

bool UCommonButtonBase::NativeOnHoldProgress(float DeltaTime)
{
	if (HoldTime > UE_SMALL_NUMBER)
	{
		CurrentHoldTime += FMath::Clamp(DeltaTime, 0.f, HoldTime);
		CurrentHoldProgress = FMath::Clamp(CurrentHoldTime / HoldTime, 0.f, 1.f);
		NativeOnActionProgress(CurrentHoldProgress);
		if (CurrentHoldProgress >= 1.f)
		{
			HandleTriggeringActionCommited();
			HoldReset();
			return false;
		}

		return true;
	}
	HoldReset();
	return false;
}

bool UCommonButtonBase::NativeOnHoldProgressRollback(float DeltaTime)
{
	if (HoldTime > UE_SMALL_NUMBER && HoldRollbackTime > UE_SMALL_NUMBER)
	{
		const float HoldRollbackMultiplier = HoldTime / HoldRollbackTime;
		CurrentHoldTime = FMath::Clamp(CurrentHoldTime - (DeltaTime * HoldRollbackMultiplier), 0.f, HoldRollbackTime);
		CurrentHoldProgress = FMath::Clamp(CurrentHoldTime / HoldTime, 0.f, 1.f);
		NativeOnActionProgress(CurrentHoldProgress);
		if (CurrentHoldProgress <= 0.f)
		{
			FTSTicker::RemoveTicker(HoldProgressRollbackTickerHandle);
			HoldProgressRollbackTickerHandle = nullptr;

			return false;
		}

		return true;
	}
	HoldReset();

	return false;
}

void UCommonButtonBase::HoldReset()
{
	if (HoldTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(HoldTickerHandle);
		HoldTickerHandle = nullptr;
	}
	if (HoldProgressRollbackTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(HoldProgressRollbackTickerHandle);
		HoldProgressRollbackTickerHandle = nullptr;
	}
	CurrentHoldTime = 0.f;
	CurrentHoldProgress = 0.f;
}

void UCommonButtonBase::NativeOnActionComplete()
{
	if (InputActionWidget)
	{
		InputActionWidget->OnActionComplete();
	}
	OnActionComplete();
}

void UCommonButtonBase::SetIsToggleable(bool bInIsToggleable)
{
	bToggleable = bInIsToggleable;

	// Update interactability.
	if (RootButton.IsValid())
	{
		if (!GetSelected() || bToggleable)
		{
			RootButton->SetInteractionEnabled(bInteractionEnabled);
		}
		else if (GetSelected() && !bToggleable)
		{
			RootButton->SetInteractionEnabled(bInteractableWhenSelected);
		}
	}

	UpdateInputActionWidgetVisibility();
}

void UCommonButtonBase::SetShouldUseFallbackDefaultInputAction(bool bInShouldUseFallbackDefaultInputAction)
{
	bShouldUseFallbackDefaultInputAction = bInShouldUseFallbackDefaultInputAction;

	UpdateInputActionWidget();
}

void UCommonButtonBase::SetIsSelected(bool InSelected, bool bGiveClickFeedback)
{
	const bool bWasHovered = IsHovered();

	if (bSelectable && bSelected != InSelected)
	{
		if (!InSelected && bToggleable)
		{
			SetSelectedInternal(false);
		}
		else if (InSelected)
		{
			// Only allow a sound if we weren't just clicked
			SetSelectedInternal(true, bGiveClickFeedback);
		}
	}

	if (ShouldProcessHoverEvent(EHoverEventSource::SelectionChanged))
	{
		// If the hover state changed due to a selection change, trigger internal logic accordingly.
		const bool bIsHoveredNow = IsHovered();
		if (bWasHovered != bIsHoveredNow)
		{
			if (bIsHoveredNow)
			{
				NativeOnHovered();
			}
			else
			{
				NativeOnUnhovered();
			}
		}
	}
}

void UCommonButtonBase::SetIsLocked(bool bInIsLocked)
{
	bool bValueChanged = bInIsLocked != bLocked;

	if (bValueChanged)
	{
		bLocked = bInIsLocked;

		SetButtonStyle();

		BP_OnLockedChanged(bLocked);

		BroadcastBinaryPostStateChange(UWidgetLockedStateRegistration::Bit, bLocked);
	}
}

void UCommonButtonBase::SetSelectedInternal(bool bInSelected, bool bAllowSound /*= true*/, bool bBroadcast /*= true*/)
{
	bool bValueChanged = bInSelected != bSelected;

	bSelected = bInSelected;

	SetButtonStyle();

	if (bSelected)
	{
		NativeOnSelected(bBroadcast);
		if (!bToggleable && IsInteractable())
		{
			// If the button isn't toggleable, then disable interaction with the root button while selected
			// The prevents us getting unnecessary click noises and events
			if (RootButton.IsValid())
			{
				RootButton->SetInteractionEnabled(bInteractableWhenSelected);
			}
		}

		if (bAllowSound)
		{
			// Selection was not triggered by a button click, so play the click sound
			FSlateApplication::Get().PlaySound(NormalStyle.PressedSlateSound);
		}
	}
	else
	{
		// Once deselected, restore the root button interactivity to the desired state
		if (RootButton.IsValid())
		{
			RootButton->SetInteractionEnabled(bInteractionEnabled);
		}

		NativeOnDeselected(bBroadcast);
	}

	UpdateInputActionWidgetVisibility();

	if (bValueChanged)
	{
		BroadcastBinaryPostStateChange(UWidgetSelectedStateRegistration::Bit, bSelected);
	}
}

void UCommonButtonBase::RefreshDimensions()
{
	RefreshDimensions(MinWidth, MinHeight, MaxWidth, MaxHeight);
}

void UCommonButtonBase::RefreshDimensions(const int32 InMinWidth, const int32 InMinHeight, const int32 InMaxWidth, const int32 InMaxHeight)
{
	if (RootButton.IsValid())
	{
		const UCommonButtonStyle* const StyleCDO = GetStyleCDO();
		RootButton->SetMinDesiredWidth(FMath::Max(InMinWidth, StyleCDO ? StyleCDO->MinWidth : 0));
		RootButton->SetMinDesiredHeight(FMath::Max(InMinHeight, StyleCDO ? StyleCDO->MinHeight : 0));
		
		if (!StyleCDO)
		{
			RootButton->SetMaxDesiredWidth(InMaxWidth);
			RootButton->SetMaxDesiredHeight(InMaxHeight);
		}
		else
		{
			if (InMaxWidth > 0 && StyleCDO->MaxWidth > 0)
			{
				RootButton->SetMaxDesiredWidth(FMath::Min(InMaxWidth, StyleCDO->MaxWidth));
			}
			else
			{
				RootButton->SetMaxDesiredWidth(FMath::Max(InMaxWidth, StyleCDO->MaxWidth));
			}

			if (InMaxHeight > 0 && StyleCDO->MaxHeight > 0)
			{
				RootButton->SetMaxDesiredHeight(FMath::Min(InMaxHeight, StyleCDO->MaxHeight));
			}
			else
			{
				RootButton->SetMaxDesiredHeight(FMath::Max(InMaxHeight, StyleCDO->MaxHeight));
			}
		}
	}
}

void UCommonButtonBase::NativeOnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (!InMouseEvent.IsTouchEvent())
	{
		Super::NativeOnMouseEnter(InGeometry, InMouseEvent);

		if (GetIsEnabled() && bInteractionEnabled)
		{
			if (ShouldProcessHoverEvent(EHoverEventSource::MouseEvent))
			{
				NativeOnHovered();
			}
		}
	}
}

void UCommonButtonBase::NativeOnMouseLeave(const FPointerEvent& InMouseEvent)
{
	if (!InMouseEvent.IsTouchEvent())
	{
		Super::NativeOnMouseLeave(InMouseEvent);

		if (GetIsEnabled() && bInteractionEnabled)
		{
			if (ShouldProcessHoverEvent(EHoverEventSource::MouseEvent))
			{
				NativeOnUnhovered();
			}
		}
	}
}

bool UCommonButtonBase::GetSelected() const
{
	return bSelected;
}

bool UCommonButtonBase::GetLocked() const
{
	return bLocked;
}

void UCommonButtonBase::ClearSelection()
{
	SetSelectedInternal( false, false );
}

void UCommonButtonBase::SetShouldSelectUponReceivingFocus(bool bInShouldSelectUponReceivingFocus)
{
	if (ensure(bSelectable || !bInShouldSelectUponReceivingFocus))
	{
		bShouldSelectUponReceivingFocus = bInShouldSelectUponReceivingFocus;
	}
}

bool UCommonButtonBase::GetShouldSelectUponReceivingFocus() const
{
	return bShouldSelectUponReceivingFocus;
}

void UCommonButtonBase::SetStyle(TSubclassOf<UCommonButtonStyle> InStyle)
{
	if (InStyle && Style != InStyle)
	{
		Style = InStyle;
		BuildStyles();
	}
}

UCommonButtonStyle* UCommonButtonBase::GetStyle() const
{
	return const_cast<UCommonButtonStyle*>(GetStyleCDO());
}

const UCommonButtonStyle* UCommonButtonBase::GetStyleCDO() const
{
	if (Style)
	{
		if (const UCommonButtonStyle* CommonButtonStyle = Cast<UCommonButtonStyle>(Style->GetDefaultObject(false)))
		{
			return CommonButtonStyle;
		}
	}
	return nullptr;
}

void UCommonButtonBase::GetCurrentButtonPadding(FMargin& OutButtonPadding) const
{
	if (const UCommonButtonStyle* CommonButtonStyle = GetStyleCDO())
	{
		CommonButtonStyle->GetButtonPadding( OutButtonPadding);
	}
}

void UCommonButtonBase::GetCurrentCustomPadding(FMargin& OutCustomPadding) const
{
	if (const UCommonButtonStyle* CommonButtonStyle = GetStyleCDO())
	{
		CommonButtonStyle->GetCustomPadding(OutCustomPadding);
	}
}

UCommonTextStyle* UCommonButtonBase::GetCurrentTextStyle() const
{
	if (const UCommonButtonStyle* CommonButtonStyle = GetStyleCDO())
	{
		UCommonTextStyle* CurrentTextStyle = nullptr;
		if (!bButtonEnabled)
		{
			CurrentTextStyle = CommonButtonStyle->GetDisabledTextStyle();
		}
		else if (bSelected)
		{
			if (IsHovered())
			{
				CurrentTextStyle = CommonButtonStyle->GetSelectedHoveredTextStyle();
			}
			if (CurrentTextStyle == nullptr)
			{
				CurrentTextStyle = CommonButtonStyle->GetSelectedTextStyle();
			}
		}

		if (CurrentTextStyle == nullptr)
		{
			if (IsHovered())
			{
				CurrentTextStyle = CommonButtonStyle->GetNormalHoveredTextStyle();
			}
			if (CurrentTextStyle == nullptr)
			{
				CurrentTextStyle = CommonButtonStyle->GetNormalTextStyle();
			}
		}
		return CurrentTextStyle;
	}
	return nullptr;
}

TSubclassOf<UCommonTextStyle> UCommonButtonBase::GetCurrentTextStyleClass() const
{
	if (UCommonTextStyle* CurrentTextStyle = GetCurrentTextStyle())
	{
		return CurrentTextStyle->GetClass();
	}
	return nullptr;
}

void UCommonButtonBase::SetMinDimensions(int32 InMinWidth, int32 InMinHeight)
{
	InMinWidth = FMath::Max(InMinWidth, 0);
	InMinHeight = FMath::Max(InMinHeight, 0);

	if (UE::CommonUI::Private::bDisableDimensionValueSetInDesignTime && IsDesignTime())
	{
		if (RootButton.IsValid())
		{
			RefreshDimensions(InMinWidth, InMinHeight, RootButton->GetMaxWidth(), RootButton->GetMaxHeight());
		}
	}
	else
	{
		MinWidth = InMinWidth;
		MinHeight = InMinHeight;

		RefreshDimensions();
	}
}

void UCommonButtonBase::SetMaxDimensions(int32 InMaxWidth, int32 InMaxHeight)
{
	InMaxWidth = FMath::Max(InMaxWidth, RootButton.IsValid() ? RootButton->GetMinWidth() : 0);
	InMaxHeight = FMath::Max(InMaxHeight, RootButton.IsValid() ? RootButton->GetMinHeight() : 0);

	if (UE::CommonUI::Private::bDisableDimensionValueSetInDesignTime && IsDesignTime())
	{
		if (RootButton.IsValid())
		{
			RefreshDimensions(RootButton->GetMinWidth(), RootButton->GetMinHeight(), InMaxWidth, InMaxHeight);
		}
	}
	else
	{
		MaxWidth = InMaxWidth;
		MaxHeight = InMaxHeight;

		RefreshDimensions();
	}
}

void UCommonButtonBase::SetActionDisplayName(const FText& InDisplayName)
{
	if (!TriggeringBindingHandle.IsValid())
	{
		return;
	}

	if (!TriggeringBindingHandle.GetDisplayName().EqualTo(InDisplayName))
	{
		TriggeringBindingHandle.SetDisplayName(InDisplayName);
	}
}

void UCommonButtonBase::SetTriggeredInputAction(const FDataTableRowHandle &InputActionRow)
{
	UnbindTriggeringInputActionToClick();

	TriggeringInputAction = {};
	TriggeringEnhancedInputAction = nullptr;
	TriggeredInputAction = InputActionRow;
	UpdateInputActionWidget();

	OnTriggeredInputActionChanged(InputActionRow);
}

void UCommonButtonBase::SetTriggeringInputAction(const FDataTableRowHandle & InputActionRow)
{
	if (TriggeringInputAction != InputActionRow)
	{
		UnbindTriggeringInputActionToClick();

		TriggeredInputAction = {};
		TriggeringEnhancedInputAction = nullptr;
		TriggeringInputAction = InputActionRow;

		if (!IsDesignTime())
		{
			BindTriggeringInputActionToClick();
		}

		// Update the Input action widget whenever the triggering input action changes
		UpdateInputActionWidget();

		OnTriggeringInputActionChanged(InputActionRow);
	}
}

void UCommonButtonBase::SetTriggeringEnhancedInputAction(UInputAction* InInputAction)
{
	if (CommonUI::IsEnhancedInputSupportEnabled() && TriggeringEnhancedInputAction != InInputAction)
	{
		UnbindTriggeringInputActionToClick();

		TriggeredInputAction = {};
		TriggeringInputAction = {};
		TriggeringEnhancedInputAction = InInputAction;

		if (!IsDesignTime())
		{
			BindTriggeringInputActionToClick();
		}

		// Update the Input action widget whenever the triggering input action changes
		UpdateInputActionWidget();

		OnTriggeringEnhancedInputActionChanged(InInputAction);
	}
}

bool UCommonButtonBase::GetInputAction(FDataTableRowHandle &InputActionRow) const
{
	bool bBothActionsSet = !TriggeringInputAction.IsNull() && !TriggeredInputAction.IsNull();
	bool bNoActionSet = TriggeringInputAction.IsNull() && TriggeredInputAction.IsNull();

	if (bBothActionsSet || bNoActionSet)
	{
		return false;
	}

	if (!TriggeringInputAction.IsNull())
	{
		InputActionRow = TriggeringInputAction;
		return true;
	}
	else
	{
		InputActionRow = TriggeredInputAction;
		return true;
	}
}

UInputAction* UCommonButtonBase::GetEnhancedInputAction() const
{
	return TriggeringEnhancedInputAction;
}

UMaterialInstanceDynamic* UCommonButtonBase::GetSingleMaterialStyleMID() const
{
	return SingleMaterialStyleMID;
}

void UCommonButtonBase::ExecuteTriggeredInput()
{
}

void UCommonButtonBase::UpdateInputActionWidget()
{
	// Update the input action state of the input action widget contextually based on the current state of the button
	if (GetGameInstance() && InputActionWidget)
	{
		bool bIsEnhancedInputSupportEnabled = CommonUI::IsEnhancedInputSupportEnabled();

		// Prefer visualizing the triggering enhanced input action before all else
		if (bIsEnhancedInputSupportEnabled && TriggeringEnhancedInputAction)
		{
			InputActionWidget->SetEnhancedInputAction(TriggeringEnhancedInputAction);
		}
		// Prefer visualizing the triggering input action next
		else if (!TriggeringInputAction.IsNull())
		{
			InputActionWidget->SetInputAction(TriggeringInputAction);
		}
		// Fallback to visualizing the triggered input action, if it's available
		else if (!TriggeredInputAction.IsNull())
		{
			InputActionWidget->SetInputAction(TriggeredInputAction);
		}
		// Visualize the default click action when neither input action is bound and when the widget is enabled and hovered
		else if (bShouldUseFallbackDefaultInputAction && bButtonEnabled && IsHovered())
		{
			UInputAction* DefaultEnhancedClickAction = bIsEnhancedInputSupportEnabled ? ICommonInputModule::GetSettings().GetEnhancedInputClickAction() : nullptr;
			if (DefaultEnhancedClickAction)
			{
				InputActionWidget->SetEnhancedInputAction(DefaultEnhancedClickAction);
			}
			else
			{
				InputActionWidget->SetInputAction(ICommonInputModule::GetSettings().GetDefaultClickAction());
			}
		}
		else
		{
			if (bIsEnhancedInputSupportEnabled)
			{
				InputActionWidget->SetEnhancedInputAction(nullptr);
			}

			FDataTableRowHandle EmptyStateHandle;
			InputActionWidget->SetInputAction(EmptyStateHandle);
		}

		UpdateInputActionWidgetVisibility();
	}
}

void UCommonButtonBase::HandleButtonClicked()
{
	// Since the button enabled state is part of UCommonButtonBase, UButton::OnClicked can be fired while this button is not interactable.
	// Guard against this case.
	if (IsInteractionEnabled())
	{
		// @TODO: Current click rejection method relies on click hold time, this can be refined. See NativeOnHoldProgress.
		// Also gamepad can indirectly trigger this method, so don't guard against pressed
		if (bRequiresHold && CurrentHoldProgress < 1.f)
		{
			return;
		}

		if (bTriggerClickedAfterSelection)
		{
			SetIsSelected(!bSelected, false);
			NativeOnClicked();
		}
		else
		{
			NativeOnClicked();
			SetIsSelected(!bSelected, false);
		}

		ExecuteTriggeredInput();
		HoldReset();
	}
}

FReply UCommonButtonBase::HandleButtonDoubleClicked()
{
	bStopDoubleClickPropagation = false;
	NativeOnDoubleClicked();
	return bStopDoubleClickPropagation ? FReply::Handled() : FReply::Unhandled();
}

void UCommonButtonBase::HandleFocusReceived()
{
	if (bShouldSelectUponReceivingFocus && !GetSelected())
	{
		SetIsSelected(true, false);
	}
	OnFocusReceived().Broadcast();

	if (CanSafelyRouteCall())
	{
		BP_OnFocusReceived();

		if (OnButtonBaseFocused.IsBound())
		{
			OnButtonBaseFocused.Broadcast(this);
		}
	}
}

void UCommonButtonBase::HandleFocusLost()
{
	OnFocusLost().Broadcast();

	if (CanSafelyRouteCall())
	{
		BP_OnFocusLost();

		if (OnButtonBaseUnfocused.IsBound())
		{
			OnButtonBaseUnfocused.Broadcast(this);
		}
	}
}

void UCommonButtonBase::HandleButtonPressed()
{
	NativeOnPressed();

	UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();
	
	if (ShouldProcessHoverEvent(EHoverEventSource::SimulationForTouch))
	{
		NativeOnHovered();
	}

	if (bRequiresHold && HoldTime > 0.f)
	{
		// Note: Fires once per frame FTSTicker::AddTicker has a delay param if desired
		HoldTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCommonButtonBase::NativeOnHoldProgress));
		if (HoldProgressRollbackTickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(HoldProgressRollbackTickerHandle);
			HoldProgressRollbackTickerHandle = nullptr;
		}
	}
	if (TriggeringBindingHandle.IsValid())
	{
		TriggeringBindingHandle.ResetHold();
	}
}

void UCommonButtonBase::HandleButtonReleased()
{
	NativeOnReleased();

	UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();
	
	if (ShouldProcessHoverEvent(EHoverEventSource::SimulationForTouch))
	{
		// Simulate hover events when using touch input
		NativeOnUnhovered();
	}

	if (bRequiresHold && HoldTime > 0.f)
	{
		if (HoldRollbackTime <= UE_SMALL_NUMBER)
		{
			HoldReset();
		}
		else
		{
			// Begin hold progress rollback
			HoldProgressRollbackTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCommonButtonBase::NativeOnHoldProgressRollback));

			FTSTicker::RemoveTicker(HoldTickerHandle);
			HoldTickerHandle = nullptr;
		}
	}
}

COMMONUI_API FReply UCommonButtonBase::HandleButtonDragDetected(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{
	//	This seems strange... If NativeOnDragDetected has been overridden and creates a valid operation, should I reply Handled?
	UDragDropOperation* Operation = nullptr;
	NativeOnDragDetected(MyGeometry, PointerEvent, Operation);

	if (OnCommonButtonDragDetected().IsBound())
	{
		return OnCommonButtonDragDetected().Execute(MyGeometry, PointerEvent);
	}

	return FReply::Unhandled();
}

COMMONUI_API void UCommonButtonBase::HandleButtonDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	
	if (NativeOp.IsValid())
	{
		NativeOnDragEnter(MyGeometry, DragDropEvent, NativeOp->GetOperation());
	}

	OnCommonButtonDragEnter().ExecuteIfBound(MyGeometry, DragDropEvent);
}

COMMONUI_API void UCommonButtonBase::HandleButtonDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	
	if (NativeOp.IsValid())
	{
		NativeOnDragLeave(DragDropEvent, NativeOp->GetOperation());
	}

	OnCommonButtonDragLeave().ExecuteIfBound(DragDropEvent);
}

COMMONUI_API FReply UCommonButtonBase::HandleButtonDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if (NativeOp.IsValid())
	{
		if (NativeOnDragOver(MyGeometry, DragDropEvent, NativeOp->GetOperation()))
		{
			return FReply::Handled();
		}
	}

	if (OnCommonButtonDragOver().IsBound())
	{
		return OnCommonButtonDragOver().Execute(MyGeometry, DragDropEvent);
	}

	return FReply::Unhandled();
}

COMMONUI_API FReply UCommonButtonBase::HandleButtonDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if (NativeOp.IsValid())
	{
		if (NativeOnDrop(MyGeometry, DragDropEvent, NativeOp->GetOperation()))
		{
			return FReply::Handled();	
		}
	}

	if (OnCommonButtonDrop().IsBound())
	{
		return OnCommonButtonDrop().Execute(MyGeometry, DragDropEvent);
	}

	return FReply::Unhandled();
}

FReply UCommonButtonBase::NativeOnFocusReceived(const FGeometry& InGeometry, const FFocusEvent& InFocusEvent)
{
	FReply Reply = Super::NativeOnFocusReceived(InGeometry, InFocusEvent);

	if (!Reply.IsEventHandled() && RootButton.IsValid() && RootButton->GetCommonButton().IsValid())
	{
		Reply = FReply::Handled().SetUserFocus(RootButton->GetCommonButton().ToSharedRef(), InFocusEvent.GetCause());
	}

	return Reply;
}

void UCommonButtonBase::NativeOnSelected(bool bBroadcast)
{
	BP_OnSelected();

	if (bBroadcast)
	{
		OnIsSelectedChanged().Broadcast(true);
		OnSelectedChangedBase.Broadcast(this, true);
		OnButtonBaseSelected.Broadcast(this);
	}
	NativeOnCurrentTextStyleChanged();
}

void UCommonButtonBase::NativeOnDeselected(bool bBroadcast)
{
	BP_OnDeselected();

	if (bBroadcast)
	{
		OnIsSelectedChanged().Broadcast(false);
		OnSelectedChangedBase.Broadcast(this, false);
		OnButtonBaseUnselected.Broadcast(this);
	}
	NativeOnCurrentTextStyleChanged();
}

void UCommonButtonBase::NativeOnHovered()
{
	if (!ShouldProcessHoverEvent(EHoverEventSource::Unknown))
	{
		return;
	}

	BP_OnHovered();
	OnHovered().Broadcast();

	if (OnButtonBaseHovered.IsBound())
	{
		OnButtonBaseHovered.Broadcast(this);
	}

	Invalidate(EInvalidateWidgetReason::Layout);

	NativeOnCurrentTextStyleChanged();
	UpdateInputActionWidget();

	BroadcastBinaryPostStateChange(UWidgetHoveredStateRegistration::Bit, true);
}

void UCommonButtonBase::NativeOnUnhovered()
{
	if (!ShouldProcessHoverEvent(EHoverEventSource::Unknown))
	{
		return;
	}

	BP_OnUnhovered();
	OnUnhovered().Broadcast();

	if (OnButtonBaseUnhovered.IsBound())
	{
		OnButtonBaseUnhovered.Broadcast(this);
	}

	Invalidate(EInvalidateWidgetReason::Layout);

	NativeOnCurrentTextStyleChanged();
	UpdateInputActionWidget();

	BroadcastBinaryPostStateChange(UWidgetHoveredStateRegistration::Bit, false);
}

void UCommonButtonBase::NativeOnClicked()
{
	if (!GetLocked())
	{
		BP_OnClicked();
		OnClicked().Broadcast();
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::ClickEvent);
		if (OnButtonBaseClicked.IsBound())
		{
			OnButtonBaseClicked.Broadcast(this);
		}

		FString ButtonName, ABTestName, ExtraData;
		if (GetButtonAnalyticInfo(ButtonName, ABTestName, ExtraData))
		{
			UCommonUISubsystemBase* CommonUISubsystem = GetUISubsystem();
			if (GetGameInstance())
			{
				check(CommonUISubsystem);

				CommonUISubsystem->FireEvent_ButtonClicked(ButtonName, ABTestName, ExtraData);
			}
		}
	}
	else
	{
		BP_OnLockClicked();
		OnLockClicked().Broadcast();
		if (OnButtonBaseLockClicked.IsBound())
		{
			OnButtonBaseLockClicked.Broadcast(this);
		}
	}
}

void UCommonButtonBase::NativeOnDoubleClicked()
{
	if (!GetLocked())
	{
		BP_OnDoubleClicked();
		OnDoubleClicked().Broadcast();
		if (OnButtonBaseDoubleClicked.IsBound())
		{
			OnButtonBaseDoubleClicked.Broadcast(this);
		}
	}
	else
	{
		BP_OnLockDoubleClicked();
		OnLockDoubleClicked().Broadcast();
		if (OnButtonBaseLockDoubleClicked.IsBound())
		{
			OnButtonBaseLockDoubleClicked.Broadcast(this);
		}
	}
}

void UCommonButtonBase::StopDoubleClickPropagation()
{
	bStopDoubleClickPropagation = true;
}

void UCommonButtonBase::NativeOnPressed()
{
	HoldReset();
	BP_OnPressed();
	OnPressed().Broadcast();
	BroadcastBinaryPostStateChange(UWidgetPressedStateRegistration::Bit, true);
}

void UCommonButtonBase::NativeOnReleased()
{
	BP_OnReleased();
	OnReleased().Broadcast();
	BroadcastBinaryPostStateChange(UWidgetPressedStateRegistration::Bit, false);
}

void UCommonButtonBase::NativeOnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, UDragDropOperation*& OutOperation)
{
	if (OnButtonBaseDragDetected.IsBound())
	{
		OnButtonBaseDragDetected.Broadcast(this, InGeometry, OutOperation);
	}
	Super::NativeOnDragDetected(InGeometry, InMouseEvent, OutOperation);
}

void UCommonButtonBase::NativeOnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation)
{
	if (OnButtonBaseDragEnter.IsBound())
	{
		OnButtonBaseDragEnter.Broadcast(this, InGeometry, InOperation);
	}
	Super::NativeOnDragEnter(InGeometry, InDragDropEvent, InOperation);
}

void UCommonButtonBase::NativeOnDragLeave(const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation)
{
	if (OnButtonBaseDragLeave.IsBound())
	{
		OnButtonBaseDragLeave.Broadcast(this, InOperation);
	}
	Super::NativeOnDragLeave(InDragDropEvent, InOperation);
}

bool UCommonButtonBase::NativeOnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation)
{
	if (OnButtonBaseDragOver.IsBound())
	{
		OnButtonBaseDragOver.Broadcast(this,InGeometry, InOperation);
	}
	return Super::NativeOnDragOver(InGeometry, InDragDropEvent, InOperation);
}

bool UCommonButtonBase::NativeOnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation)
{
	if (OnButtonBaseDrop.IsBound())
	{
		OnButtonBaseDrop.Broadcast(this, InGeometry, InOperation);
	}
	return Super::NativeOnDrop(InGeometry, InDragDropEvent, InOperation);
}

void UCommonButtonBase::NativeOnEnabled()
{
	BP_OnEnabled();
	NativeOnCurrentTextStyleChanged();
}

void UCommonButtonBase::NativeOnDisabled()
{
	BP_OnDisabled();
	NativeOnCurrentTextStyleChanged();
}

bool UCommonButtonBase::GetButtonAnalyticInfo(FString& ButtonName, FString& ABTestName, FString& ExtraData) const 
{
	GetName(ButtonName);
	ABTestName = TEXT("None");
	ExtraData = TEXT("None");

	return true;
}

void UCommonButtonBase::NativeOnCurrentTextStyleChanged()
{
	OnCurrentTextStyleChanged();
}

void UCommonButtonBase::BuildStyles()
{
	if (const UCommonButtonStyle* CommonButtonStyle = GetStyleCDO())
	{
		const FMargin& ButtonPadding = CommonButtonStyle->ButtonPadding;
		const FSlateBrush& DisabledBrush = CommonButtonStyle->Disabled;

		FSlateBrush DynamicSingleMaterialBrush;
		if (CommonButtonStyle->bSingleMaterial)
		{
			DynamicSingleMaterialBrush = CommonButtonStyle->SingleMaterialBrush;

			// Create dynamic instance of material if possible.
			UMaterialInterface* const BaseMaterial = Cast<UMaterialInterface>(DynamicSingleMaterialBrush.GetResourceObject());
			SingleMaterialStyleMID = BaseMaterial ? UMaterialInstanceDynamic::Create(BaseMaterial, this) : nullptr;
			if (SingleMaterialStyleMID)
			{
				DynamicSingleMaterialBrush.SetResourceObject(SingleMaterialStyleMID);
			}
		}
		else
		{
			SingleMaterialStyleMID = nullptr;
		}
		bool bHasPressedSlateSoundOverride = PressedSlateSoundOverride.GetResourceObject() != nullptr;
		bool bHasClickedSlateSoundOverride = ClickedSlateSoundOverride.GetResourceObject() != nullptr;
		bool bHasHoveredSlateSoundOverride = HoveredSlateSoundOverride.GetResourceObject() != nullptr;

		NormalStyle.Normal = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : CommonButtonStyle->NormalBase;
		NormalStyle.Hovered = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : CommonButtonStyle->NormalHovered;
		NormalStyle.Pressed = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : CommonButtonStyle->NormalPressed;
		NormalStyle.Disabled = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : DisabledBrush;
		NormalStyle.NormalPadding = ButtonPadding;
		NormalStyle.PressedPadding = ButtonPadding;

		// Sets the sound overrides for the Normal state
		NormalStyle.PressedSlateSound = bHasPressedSlateSoundOverride ? PressedSlateSoundOverride : CommonButtonStyle->PressedSlateSound;
		NormalStyle.ClickedSlateSound = bHasClickedSlateSoundOverride ? ClickedSlateSoundOverride : CommonButtonStyle->ClickedSlateSound;
		NormalStyle.HoveredSlateSound = bHasHoveredSlateSoundOverride ? HoveredSlateSoundOverride : CommonButtonStyle->HoveredSlateSound;

		SelectedStyle.Normal = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : CommonButtonStyle->SelectedBase;
		SelectedStyle.Hovered = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : CommonButtonStyle->SelectedHovered;
		SelectedStyle.Pressed = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : CommonButtonStyle->SelectedPressed;
		SelectedStyle.Disabled = CommonButtonStyle->bSingleMaterial ? DynamicSingleMaterialBrush : DisabledBrush;
		SelectedStyle.NormalPadding = ButtonPadding;
		SelectedStyle.PressedPadding = ButtonPadding;

		DisabledStyle = NormalStyle;

		/**
		 * Selected State Sound overrides
		 * If there is no Selected state sound override, the Normal state's sound will be used.
		 * This sound may come from either the button style or the sound override in Blueprints.
		 */
		if (SelectedPressedSlateSoundOverride.GetResourceObject())
		{
			SelectedStyle.PressedSlateSound = SelectedPressedSlateSoundOverride;
		}
		else
		{
			SelectedStyle.PressedSlateSound =
				bHasPressedSlateSoundOverride || !CommonButtonStyle->SelectedPressedSlateSound
				? NormalStyle.PressedSlateSound
				: CommonButtonStyle->SelectedPressedSlateSound.Sound;
		}
		
		if (SelectedClickedSlateSoundOverride.GetResourceObject())
		{
			SelectedStyle.ClickedSlateSound = SelectedClickedSlateSoundOverride;
		}
		else
		{
			SelectedStyle.ClickedSlateSound =
				bHasClickedSlateSoundOverride || !CommonButtonStyle->SelectedClickedSlateSound
				? NormalStyle.ClickedSlateSound
				: CommonButtonStyle->SelectedClickedSlateSound.Sound;
		}

		if (SelectedHoveredSlateSoundOverride.GetResourceObject())
		{
			SelectedStyle.HoveredSlateSound = SelectedHoveredSlateSoundOverride;
		}
		else
		{
			SelectedStyle.HoveredSlateSound =
				bHasHoveredSlateSoundOverride || !CommonButtonStyle->SelectedHoveredSlateSound
				? NormalStyle.HoveredSlateSound
				: CommonButtonStyle->SelectedHoveredSlateSound.Sound;
		}

		// Locked State Sound overrides
		LockedStyle = NormalStyle;
		if (CommonButtonStyle->LockedPressedSlateSound || LockedPressedSlateSoundOverride.GetResourceObject())
		{
			LockedStyle.PressedSlateSound =
				LockedPressedSlateSoundOverride.GetResourceObject()
				? LockedPressedSlateSoundOverride
				: CommonButtonStyle->LockedPressedSlateSound.Sound;
		}
		
		if (CommonButtonStyle->LockedClickedSlateSound || LockedClickedSlateSoundOverride.GetResourceObject())
		{
			LockedStyle.ClickedSlateSound =
				LockedClickedSlateSoundOverride.GetResourceObject()
				? LockedClickedSlateSoundOverride
				: CommonButtonStyle->LockedClickedSlateSound.Sound;
		}
		
		if (CommonButtonStyle->LockedHoveredSlateSound || LockedHoveredSlateSoundOverride.GetResourceObject())
		{
			LockedStyle.HoveredSlateSound =
				LockedHoveredSlateSoundOverride.GetResourceObject()
				? LockedHoveredSlateSoundOverride
				: CommonButtonStyle->LockedHoveredSlateSound.Sound;
		}

		SetButtonStyle();

		RefreshDimensions();
	}
}

void UCommonButtonBase::SetButtonStyle()
{
	if (UButton* ButtonPtr = RootButton.Get())
	{
		const FButtonStyle* UseStyle;
		if (bLocked)
		{
			UseStyle = &LockedStyle;
		}
		else if (bSelected)
		{
			UseStyle = &SelectedStyle;
		}
		else if (bButtonEnabled)
		{
			UseStyle = &NormalStyle;
		}
		else
		{
			UseStyle = &DisabledStyle;
		}
		ButtonPtr->SetStyle(*UseStyle);
		NativeOnCurrentTextStyleChanged();
	}
}

void UCommonButtonBase::SetInputActionProgressMaterial(const FSlateBrush& InProgressMaterialBrush, const FName& InProgressMaterialParam)
{
	if (InputActionWidget)
	{
		InputActionWidget->SetProgressMaterial(InProgressMaterialBrush, InProgressMaterialParam);
	}
}

void UCommonButtonBase::SetPressedSoundOverride(USoundBase* Sound)
{
	if (PressedSlateSoundOverride.GetResourceObject() != Sound)
	{
		PressedSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetClickedSoundOverride(USoundBase* Sound)
{
	if (ClickedSlateSoundOverride.GetResourceObject() != Sound)
	{
		ClickedSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetHoveredSoundOverride(USoundBase* Sound)
{
	if (HoveredSlateSoundOverride.GetResourceObject() != Sound)
	{
		HoveredSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetSelectedPressedSoundOverride(USoundBase* Sound)
{
	if (SelectedPressedSlateSoundOverride.GetResourceObject() != Sound)
	{
		SelectedPressedSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetSelectedClickedSoundOverride(USoundBase* Sound)
{
	if (SelectedClickedSlateSoundOverride.GetResourceObject() != Sound)
	{
		SelectedClickedSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetSelectedHoveredSoundOverride(USoundBase* Sound)
{
	if (SelectedHoveredSlateSoundOverride.GetResourceObject() != Sound)
	{
		SelectedHoveredSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetLockedPressedSoundOverride(USoundBase* Sound)
{
	if (LockedPressedSlateSoundOverride.GetResourceObject() != Sound)
	{
		LockedPressedSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetLockedClickedSoundOverride(USoundBase* Sound)
{
	if (LockedClickedSlateSoundOverride.GetResourceObject() != Sound)
	{
		LockedClickedSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetLockedHoveredSoundOverride(USoundBase* Sound)
{
	if (LockedHoveredSlateSoundOverride.GetResourceObject() != Sound)
	{
		LockedHoveredSlateSoundOverride.SetResourceObject(Sound);
		BuildStyles();
	}
}

void UCommonButtonBase::SetPressedSlateSoundOverride(const FSlateSound& InPressedSlateSoundOverride)
{
	const bool bBuildStyles = InPressedSlateSoundOverride.GetResourceObject() != PressedSlateSoundOverride.GetResourceObject();
	PressedSlateSoundOverride = InPressedSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetClickedSlateSoundOverride(const FSlateSound& InClickedSlateSoundOverride)
{
	const bool bBuildStyles = InClickedSlateSoundOverride.GetResourceObject() != ClickedSlateSoundOverride.GetResourceObject();
	ClickedSlateSoundOverride = InClickedSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetHoveredSlateSoundOverride(const FSlateSound& InHoveredSlateSoundOverride)
{
	const bool bBuildStyles = InHoveredSlateSoundOverride.GetResourceObject() != HoveredSlateSoundOverride.GetResourceObject();
	HoveredSlateSoundOverride = InHoveredSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetSelectedPressedSlateSoundOverride(const FSlateSound& InSelectedPressedSlateSoundOverride)
{
	const bool bBuildStyles = InSelectedPressedSlateSoundOverride.GetResourceObject() != SelectedPressedSlateSoundOverride.GetResourceObject();
	SelectedPressedSlateSoundOverride = InSelectedPressedSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetSelectedClickedSlateSoundOverride(const FSlateSound& InSelectedClickedSlateSoundOverride)
{
	const bool bBuildStyles = InSelectedClickedSlateSoundOverride.GetResourceObject() != SelectedClickedSlateSoundOverride.GetResourceObject();
	SelectedClickedSlateSoundOverride = InSelectedClickedSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetSelectedHoveredSlateSoundOverride(const FSlateSound& InSelectedHoveredSlateSoundOverride)
{
	const bool bBuildStyles = InSelectedHoveredSlateSoundOverride.GetResourceObject() != SelectedHoveredSlateSoundOverride.GetResourceObject();
	SelectedHoveredSlateSoundOverride = InSelectedHoveredSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetLockedPressedSlateSoundOverride(const FSlateSound& InLockedPressedSlateSoundOverride)
{
	const bool bBuildStyles = InLockedPressedSlateSoundOverride.GetResourceObject() != LockedPressedSlateSoundOverride.GetResourceObject();
	LockedPressedSlateSoundOverride = InLockedPressedSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetLockedClickedSlateSoundOverride(const FSlateSound& InLockedClickedSlateSoundOverride)
{
	const bool bBuildStyles = InLockedClickedSlateSoundOverride.GetResourceObject() != LockedClickedSlateSoundOverride.GetResourceObject();
	LockedClickedSlateSoundOverride = InLockedClickedSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::SetLockedHoveredSlateSoundOverride(const FSlateSound& InLockedHoveredSlateSoundOverride)
{
	const bool bBuildStyles = InLockedHoveredSlateSoundOverride.GetResourceObject() != LockedHoveredSlateSoundOverride.GetResourceObject();
	LockedHoveredSlateSoundOverride = InLockedHoveredSlateSoundOverride;
	if (bBuildStyles)
	{
		BuildStyles();
	}
}

void UCommonButtonBase::UpdateInputActionWidgetVisibility()
{
	if (InputActionWidget)
	{
		bool bHidden = false;

		UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem();

		if (bHideInputAction)
		{
			bHidden = true;
		}
		else if (CommonInputSubsystem && bHideInputActionWithKeyboard && CommonInputSubsystem->GetCurrentInputType() != ECommonInputType::Gamepad)
		{
			bHidden = true;
		}
		else if (bSelected)
		{
			if (!bToggleable)
			{
				if (!bDisplayInputActionWhenNotInteractable && !bInteractableWhenSelected)
				{
					bHidden = true;
				}
			}
		}
		else
		{
			if (!bDisplayInputActionWhenNotInteractable && !bInteractionEnabled)
			{
				bHidden = true;
			}
		}

		InputActionWidget->SetHidden(bHidden);
	}
}

void UCommonButtonBase::EnableButton()
{
	if (!bButtonEnabled)
	{
		bButtonEnabled = true;
		if (RootButton.IsValid())
		{
			RootButton->SetButtonEnabled(true);
		}

		SetButtonStyle();

		NativeOnEnabled();

		if (InputActionWidget)
		{
			UpdateInputActionWidget();
			InputActionWidget->SetIsEnabled(bButtonEnabled);
		}
	}
}

void UCommonButtonBase::DisableButton()
{
	if (bButtonEnabled)
	{
		bButtonEnabled = false;
		if (RootButton.IsValid())
		{
			RootButton->SetButtonEnabled(false);
		}

		SetButtonStyle();

		NativeOnDisabled();

		if (InputActionWidget)
		{
			UpdateInputActionWidget();
			InputActionWidget->SetIsEnabled(bButtonEnabled);
		}
	}
}

void UCommonButtonBase::SetRequiresHold(bool bInRequiresHold)
{
	const bool bPrevRequiresHold = bRequiresHold;
	bRequiresHold = bInRequiresHold;
	if (const UCommonInputSubsystem* CommonInputSubsystem = GetInputSubsystem())
	{
		UpdateHoldData(CommonInputSubsystem->GetCurrentInputType());
	}

	if (bPrevRequiresHold != bRequiresHold)
	{
		BP_OnRequiresHoldChanged();
	}
}

COMMONUI_API void UCommonButtonBase::SetAllowDragDrop(bool bInAllowDragDrop)
{
	if (RootButton.IsValid())
	{
		RootButton->SetAllowDragDrop(bInAllowDragDrop);
	}
}

void UCommonButtonBase::SetIsFocusable(bool bInIsFocusable)
{
	UUserWidget::SetIsFocusable(bInIsFocusable);

	if (RootButton.IsValid())
	{
		RootButton->SetButtonFocusable(bInIsFocusable);
	}
}

bool UCommonButtonBase::GetIsFocusable() const
{
	return IsFocusable();
}

void UCommonButtonBase::HandleImplicitFocusLost()
{
	// Note: This is workaround to avoid users from invalidating focus state
	// From code, users are able to disable/hide widgets that have the users focus. 
	// If the widget was disabled, and is interacted with (e.g. Clicked), Slate will attempt to restore user focus. If the widget is hidden, the focus state is lost 
	// This results in a bad focused widget state as keyboard and controller events will attempt to tunnel through the focus path via FReply::RouteAlongFocusPath
	// To avoid this, ensure users are focused on enabled widgets by navigating to next available focusable widget if the disabled widget is currently in focus. 
	if (!bNavigateToNextWidgetOnDisable)
	{
		return;
	}

	if (GetIsEnabled() && IsVisible())
	{
		return;
	}

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (!SafeWidget.IsValid() || !RootButton.IsValid())
	{
		return;
	}

	const UCommonButtonBase& ThisRef = *this;

	FSlateApplication::Get().ForEachUser([SafeWidget, &ThisRef](FSlateUser& User)
	{
		if (User.IsWidgetInFocusPath(SafeWidget.ToSharedRef()))
		{
			// The SCommonButton child is not disabled, and may be visible, wherein the Owning UCommonButtonBase is disabled. 
			// Using Navigation::Next will first descend the widget hierarchy and focus on the SCommonButton regardless of the outer SObjectWidget state. 
			// To avoid this, use the top level directional navigation from the SObjectWidget level
			const TArray<EUINavigation> Directions = { EUINavigation::Right, EUINavigation::Down,  EUINavigation::Left,  EUINavigation::Up };
			if (FSlateApplication::Get().NavigateFromWidget(User.GetUserIndex(), SafeWidget, Directions) == EUINavigation::Invalid)
			{
				// Fallback if failed to navigate to any other widgets. Reset the focus state
				if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(ThisRef))
				{
					ActionRouter->RefreshActiveRootFocus();
				}
			}
		}
	});
}

FName UWidgetLockedStateRegistration::GetStateName() const
{
	return StateName;
};

bool UWidgetLockedStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const
{
	if (const UCommonButtonBase* CommonButton = Cast<UCommonButtonBase>(InWidget))
	{
		return CommonButton->GetLocked();
	}

	return false;
}

void UWidgetLockedStateRegistration::InitializeStaticBitfields() const
{
	Bit = FWidgetStateBitfield(GetStateName());
}
