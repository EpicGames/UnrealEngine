// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonButtonTypes.h"

#include "CommonUISettings.h"
#include "Framework/Application/SlateApplication.h"
#include "ICommonUIModule.h"
#include "Input/Reply.h"

//////////////////////////////////////////////////////////////////////////
// SCommonButton
//////////////////////////////////////////////////////////////////////////

FReply SCommonButton::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return IsInteractable() ? SButton::OnMouseButtonDown(MyGeometry, MouseEvent) : FReply::Handled();
}

FReply SCommonButton::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent)
{
	if (!IsInteractable())
	{
		return FReply::Handled();
	}

	if (OnDoubleClicked.IsBound())
	{
		FReply Reply = OnDoubleClicked.Execute();
		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}

	// The default button behavior 'ignores' double click, which means you get,
	// down -> up, double-click -> up
	// which can make the input feel like it's being lost when players are clicking
	// a button over and over.  So if your button does not handle the double click
	// specifically, we'll treat the double click as a mouse up, and do whatever
	// we would normally do based on button configuration.
	FReply Reply = OnMouseButtonDown(InMyGeometry, InPointerEvent);
	if (Reply.IsEventHandled())
	{
		return Reply;
	}

	return SButton::OnMouseButtonDoubleClick(InMyGeometry, InPointerEvent);
}

FReply SCommonButton::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& InPointerEvent)
{
	FReply Reply = FReply::Handled();
	if (!IsInteractable())
	{
		if (HasMouseCapture())
		{
			// It's conceivable that interaction was disabled while this button had mouse capture
			// If that's the case, we want to release it (without acknowledging the click)
			Release();
			Reply.ReleaseMouseCapture();
		}
	}
	else
	{
		Reply = SButton::OnMouseButtonUp(MyGeometry, InPointerEvent);
	}

	return Reply;
}

void SCommonButton::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& InPointerEvent)
{
	if (!InPointerEvent.IsTouchEvent())
	{
		const bool bWasHovered = IsHovered();

		bHovered = true;
		SetHover(bHovered && bIsInteractionEnabled);
		SButton::OnMouseEnter(MyGeometry, InPointerEvent);

		// SButton won't be able to correctly detect hover changes since we manually set hover, do our own detection
		if (!bWasHovered && IsHovered())
		{
			ExecuteHoverStateChanged(true);
		}
	}
}

void SCommonButton::OnMouseLeave(const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.IsTouchEvent())
	{
		if (HasMouseCapture())
		{
			Release();
		}
	}
	else
	{
		const bool bWasHovered = IsHovered();

		bHovered = false;
		SetHover(false);
		SButton::OnMouseLeave(InPointerEvent);

		// SButton won't be able to correctly detect hover changes since we manually set hover, do our own detection
		if (bWasHovered && !IsHovered())
		{
			ExecuteHoverStateChanged(true);
		}
	}
}

FReply SCommonButton::OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	FReply Reply = FReply::Handled();

	if (HasMouseCapture())
	{
		if (!MyGeometry.IsUnderLocation(InTouchEvent.GetScreenSpacePosition()))
		{
			Release();
			Reply.ReleaseMouseCapture();
		}
	}
	else
	{
		Reply = SButton::OnTouchMoved(MyGeometry, InTouchEvent);
	}

	return Reply;
}

FReply SCommonButton::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (ICommonUIModule::GetSettings().GetCommonButtonAcceptKeyHandling() == ECommonButtonAcceptKeyHandling::Ignore
		&& FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		return FReply::Unhandled();
	}
	return SButton::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SCommonButton::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (ICommonUIModule::GetSettings().GetCommonButtonAcceptKeyHandling() == ECommonButtonAcceptKeyHandling::Ignore
		&& FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		return FReply::Unhandled();
	}
	return SButton::OnKeyUp(MyGeometry, InKeyEvent);
}

void SCommonButton::Press()
{
	if (IsInteractable())
	{
		SButton::Press();
	}
}

void SCommonButton::SetIsButtonEnabled(bool bInIsButtonEnabled)
{
	bIsButtonEnabled = bInIsButtonEnabled;
}

void SCommonButton::SetIsButtonFocusable(bool bInIsButtonFocusable)
{
	SetIsFocusable(bInIsButtonFocusable);
}

void SCommonButton::SetIsInteractionEnabled(bool bInIsInteractionEnabled)
{
	if (bIsInteractionEnabled == bInIsInteractionEnabled)
	{
		return;
	}

	const bool bWasHovered = IsHovered();

	bIsInteractionEnabled = bInIsInteractionEnabled;

	// If the hover state changed due to an intractability change, trigger external logic accordingly.
	const bool bIsHoveredNow = bHovered && bInIsInteractionEnabled;
	if (bWasHovered != bIsHoveredNow)
	{
		SetHover(bIsHoveredNow);
		ExecuteHoverStateChanged(false);
	}
}

bool SCommonButton::IsInteractable() const
{
	return bIsButtonEnabled && bIsInteractionEnabled;
}

int32 SCommonButton::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	bool bEnabled = bParentEnabled && bIsButtonEnabled;
	return SButton::OnPaint(Args, AllottedGeometry, MyClippingRect, OutDrawElements, LayerId, InWidgetStyle, bEnabled);
}
