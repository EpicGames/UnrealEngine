// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SButton.h"

#define UE_API COMMONUI_API

//////////////////////////////////////////////////////////////////////////
// SCommonButton
//////////////////////////////////////////////////////////////////////////
/**
 * Lets us disable clicking on a button without disabling hit-testing
 * Needed because NativeOnMouseEnter is not received by disabled widgets,
 * but that also disables our anchored tooltips.
 */
class SCommonButton : public SButton
{
public:
	SLATE_BEGIN_ARGS(SCommonButton)
		: _Content()
		, _HAlign(HAlign_Fill)
		, _VAlign(VAlign_Fill)
		, _OnSlateButtonDragDetected()
		, _OnSlateButtonDragEnter()
		, _OnSlateButtonDragLeave()
		, _OnSlateButtonDragOver()
		, _OnSlateButtonDrop()
		, _ClickMethod(EButtonClickMethod::DownAndUp)
		, _TouchMethod(EButtonTouchMethod::DownAndUp)
		, _PressMethod(EButtonPressMethod::DownAndUp)
		, _IsFocusable(true)
		, _IsInteractionEnabled(true)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
		SLATE_ARGUMENT(EHorizontalAlignment, HAlign)
		SLATE_ARGUMENT(EVerticalAlignment, VAlign)
		SLATE_EVENT(FOnDragDetected, OnSlateButtonDragDetected)
		SLATE_EVENT(FOnDragEnter, OnSlateButtonDragEnter)
		SLATE_EVENT(FOnDragLeave, OnSlateButtonDragLeave)
		SLATE_EVENT(FOnDragOver, OnSlateButtonDragOver)
		SLATE_EVENT(FOnDrop, OnSlateButtonDrop)
		SLATE_EVENT(FOnClicked, OnClicked)
		SLATE_EVENT(FOnClicked, OnDoubleClicked)
		SLATE_EVENT(FSimpleDelegate, OnPressed)
		SLATE_EVENT(FSimpleDelegate, OnReleased)
		SLATE_ARGUMENT(EButtonClickMethod::Type, ClickMethod)
		SLATE_ARGUMENT(EButtonTouchMethod::Type, TouchMethod)
		SLATE_ARGUMENT(EButtonPressMethod::Type, PressMethod)
		SLATE_ARGUMENT(bool, IsFocusable)
		SLATE_EVENT(FSimpleDelegate, OnReceivedFocus)
		SLATE_EVENT(FSimpleDelegate, OnLostFocus)

		/** Is interaction enabled? */
		SLATE_ARGUMENT(bool, IsButtonEnabled)
		SLATE_ARGUMENT(bool, IsInteractionEnabled)
		SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnDoubleClicked = InArgs._OnDoubleClicked;

		SButton::Construct(SButton::FArguments()
			.ButtonStyle(InArgs._ButtonStyle)
			.HAlign(InArgs._HAlign)
			.VAlign(InArgs._VAlign)
			.OnSlateButtonDragDetected(InArgs._OnSlateButtonDragDetected)
			.OnSlateButtonDragOver(InArgs._OnSlateButtonDragOver)
			.OnSlateButtonDragEnter(InArgs._OnSlateButtonDragEnter)
			.OnSlateButtonDragLeave(InArgs._OnSlateButtonDragLeave)
			.OnSlateButtonDrop(InArgs._OnSlateButtonDrop)
			.ClickMethod(InArgs._ClickMethod)
			.TouchMethod(InArgs._TouchMethod)
			.PressMethod(InArgs._PressMethod)
			.OnClicked(InArgs._OnClicked)
			.OnPressed(InArgs._OnPressed)
			.OnReleased(InArgs._OnReleased)
			.IsFocusable(InArgs._IsFocusable)
			.OnReceivedFocus(InArgs._OnReceivedFocus)
			.OnLostFocus(InArgs._OnLostFocus)
			.Content()
			[
				InArgs._Content.Widget
			]);

		SetCanTick(false);
		// Set the hover state to indicate that we want to override the default behavior
		SetHover(false);

		bIsButtonEnabled = InArgs._IsButtonEnabled;
		bIsInteractionEnabled = InArgs._IsInteractionEnabled;
		bHovered = false;
	}

	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	UE_API virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UE_API void SetIsButtonEnabled(bool bInIsButtonEnabled);

	UE_API void SetIsButtonFocusable(bool bInIsButtonFocusable);

	UE_API void SetIsInteractionEnabled(bool bInIsInteractionEnabled);

	UE_API bool IsInteractable() const;

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

protected:
	/** Press the button */
	UE_API virtual void Press() override;

private:
	FOnClicked OnDoubleClicked;

	/** True if the button is enabled */
	bool bIsButtonEnabled;

	/** True if clicking is enabled, to allow for things like double click */
	bool bIsInteractionEnabled;

	/** True if mouse over the widget */
	bool bHovered;
};

#undef UE_API
