// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonCustomNavigation.h"
#include "CommonWidgetPaletteCategories.h"

#include "Types/NavigationMetaData.h"
#include "Widgets/Layout/SBorder.h"
#include "Components/BorderSlot.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonCustomNavigation)

class SCustomNavBorder : public SBorder
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnSimpleNavigationEvent, EUINavigation);

	SLATE_BEGIN_ARGS(SCustomNavBorder)
		: _Content()
	{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FOnSimpleNavigationEvent, OnHandleNavigation)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SBorder::Construct(SBorder::FArguments()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(0)
			.Content()
			[
				InArgs._Content.Widget
			]);

		OnHandleNavigation = InArgs._OnHandleNavigation;

#if UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA
		if (OnHandleNavigation.IsBound())
		{
			AddMetadata(MakeShared<FSimulatedNavigationMetaData>(EUINavigationRule::Explicit));
		}
#endif
	}

private:

	virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override
	{
		if (OnHandleNavigation.IsBound())
		{
			if (OnHandleNavigation.Execute(InNavigationEvent.GetNavigationType()))
			{
				// We handled the navigation so tell slate to do nothing.
				return FNavigationReply::Explicit(nullptr);
			}
		}
		return FNavigationReply::Escape();
	}

	FOnSimpleNavigationEvent OnHandleNavigation;
};

UCommonCustomNavigation::UCommonCustomNavigation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetPadding(FMargin(0.f));
}

TSharedRef<SWidget> UCommonCustomNavigation::RebuildWidget()
{
	MyBorder = SNew(SCustomNavBorder)
		.OnHandleNavigation(BIND_UOBJECT_DELEGATE(SCustomNavBorder::FOnSimpleNavigationEvent, OnNavigation));

	if (GetChildrenCount() > 0)
	{
		Cast<UBorderSlot>(GetContentSlot())->BuildSlot(MyBorder.ToSharedRef());
	}

	return MyBorder.ToSharedRef();
}

bool UCommonCustomNavigation::OnNavigation(EUINavigation NavigationType)
{
	if (OnNavigationEvent.IsBound())
	{
		return OnNavigationEvent.Execute(NavigationType);
	}

	// Do nothing
	return false;
}

#if WITH_EDITOR
const FText UCommonCustomNavigation::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif // WITH_EDITOR

