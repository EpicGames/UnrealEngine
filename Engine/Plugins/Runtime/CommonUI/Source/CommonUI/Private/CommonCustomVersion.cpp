// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FCommonCustomVersion::GUID(0x925477b, 0x763d4001, 0x9d91d673, 0xb75b411);

// Register the custom version with core
FCustomVersionRegistration GRegisterCommonUICustomVersion(FCommonCustomVersion::GUID, FCommonCustomVersion::LatestVersion, TEXT("CommonUIVer"));
