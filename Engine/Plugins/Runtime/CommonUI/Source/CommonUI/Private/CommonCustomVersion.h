// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


struct FGuid;

// Custom serialization version for all packages containing CommonUI asset types
struct FCommonCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		// Removing IsPercentage from the Common Numeric Text block in favor of an enum we can further extend.
		RemovingIsPercentage = 1,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FCommonCustomVersion() {}
};
