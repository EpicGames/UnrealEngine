// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonDateTimeTextBlock.h"
#include "CommonWidgetPaletteCategories.h"
#include "Engine/World.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonDateTimeTextBlock)

DECLARE_CYCLE_STAT(TEXT("CommonDateTimeTextBlock UpdateUnderlyingText"), STAT_CommonDateTimeTextBlock_UpdateUnderlyingText, STATGROUP_UI);


UCommonDateTimeTextBlock::UCommonDateTimeTextBlock(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LastDaysCount(-1)
	, LastHoursCount(-1)
	, bUseCountdownCompletionText(false)
{
}

void UCommonDateTimeTextBlock::TimerTick()
{
	UpdateUnderlyingText();
}

#if WITH_EDITOR
const FText UCommonDateTimeTextBlock::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif // WITH_EDITOR

void UCommonDateTimeTextBlock::SynchronizeProperties()
{
	UpdateUnderlyingText();

	Super::SynchronizeProperties();
}

void UCommonDateTimeTextBlock::SetDateTimeValue(const FDateTime InDateTime, bool InShowAsCountdown, float InRefreshDelay)
{
	UWorld* const World = GetWorld();
	if (ensure(World))
	{
		if (TimerTickHandle.IsValid())
		{
			World->GetTimerManager().ClearTimer(TimerTickHandle);
		}

		DateTime = InDateTime;
		bShowAsCountdown = InShowAsCountdown;

		if (bShowAsCountdown)
		{
			LastTimerTickTime = World->GetTimeSeconds();
			World->GetTimerManager().SetTimer(TimerTickHandle, this, &ThisClass::TimerTick, InRefreshDelay, true);
		}

		UpdateUnderlyingText();
	}
}

void UCommonDateTimeTextBlock::SetTimespanValue(const FTimespan InTimespan)
{
	SetDateTimeValue(GetTimespanStartingTime() + InTimespan, true);
}

void UCommonDateTimeTextBlock::SetCountDownCompletionText(const FText InCompletionText)
{
	bUseCountdownCompletionText = true;
	CountdownCompletionText = InCompletionText;
	UpdateUnderlyingText();
}

FDateTime UCommonDateTimeTextBlock::GetDateTime() const
{
	return DateTime;
}

void UCommonDateTimeTextBlock::UpdateUnderlyingText()
{
	SCOPE_CYCLE_COUNTER(STAT_CommonDateTimeTextBlock_UpdateUnderlyingText);

	TOptional<FText> TextToSet = TOptional<FText>();
	if (bShowAsCountdown)
	{
		FTimespan Remaining = DateTime - GetTimespanStartingTime();
		if (TimerTickHandle.IsValid() && ShouldClearTimer(Remaining))
		{
			UWorld* const World = GetWorld();
			if (World)
			{
				World->GetTimerManager().ClearTimer(TimerTickHandle);
			}
		}

		if (Remaining.GetTotalSeconds() < 1.0)
		{
			OnTimeCountDownCompletion().Broadcast();
			
			if(bUseCountdownCompletionText)
			{
				TextToSet = CountdownCompletionText;
			}
		}
		else
		{
			TextToSet = FormatTimespan(Remaining);
		}

		LastDaysCount = Remaining.GetDays();
		LastHoursCount = Remaining.GetHours();
	}
	else
	{
		TextToSet = FormatDateTime(DateTime);
	}

	if (TextToSet.IsSet())
	{
		SetText(TextToSet.GetValue());
	}
}

FDateTime UCommonDateTimeTextBlock::GetTimespanStartingTime() const
{
	return FDateTime::Now();
}

bool UCommonDateTimeTextBlock::ShouldClearTimer(const FTimespan& TimeRemaining) const
{
	return TimeRemaining.GetTotalSeconds() < 1.0;
}

TOptional<FText> UCommonDateTimeTextBlock::FormatTimespan(const FTimespan& InTimespan) const
{
	TOptional<FText> TextToSet = TOptional<FText>();

	if (CustomTimespanFormat.IsEmpty())
	{
		if (InTimespan.GetTotalSeconds() < 1.0)
		{
			TextToSet = FText::AsTimespan(InTimespan);
		}
		else
		{
			const int32 NewDaysCount = InTimespan.GetDays();
			if (NewDaysCount > 2)
			{
				if (NewDaysCount != GetLastDaysCount())
				{
					FText TimespanFormatPattern = NSLOCTEXT("CommonDateTimeTextBlock", "DaysFormatText", "{Days} {Days}|plural(one=Day, other=Days)");
					FFormatNamedArguments TimeArguments;
					TimeArguments.Add(TEXT("Days"), NewDaysCount);
					TextToSet = FText::Format(TimespanFormatPattern, TimeArguments);
				}
			}
			else
			{
				const int32 NewHoursCount = (int32)InTimespan.GetTotalHours();
				if (NewHoursCount > 12)
				{
					if (NewHoursCount != GetLastHoursCount())
					{
						FText TimespanFormatPattern = NSLOCTEXT("CommonDateTimeTextBlock", "HoursFormatText", "{Hours} {Hours}|plural(one=Hour, other=Hours)");
						FFormatNamedArguments TimeArguments;
						TimeArguments.Add(TEXT("Hours"), NewHoursCount);
						TextToSet = FText::Format(TimespanFormatPattern, TimeArguments);
					}
				}
				else
				{
					TextToSet = FText::AsTimespan(InTimespan);
				}
			}
		}
	}
	else
	{
		FNumberFormattingOptions NumberFormatOptions;
		NumberFormatOptions.MinimumIntegralDigits = bCustomTimespanLeadingZeros ? 2 : 1;

		FFormatNamedArguments TimeArguments;
		TimeArguments.Add(TEXT("Days"), FText::AsNumber(InTimespan.GetDays(), &NumberFormatOptions));
		TimeArguments.Add(TEXT("Hours"), FText::AsNumber(InTimespan.GetHours(), &NumberFormatOptions));
		TimeArguments.Add(TEXT("Minutes"), FText::AsNumber(InTimespan.GetMinutes(), &NumberFormatOptions));
		TimeArguments.Add(TEXT("Seconds"), FText::AsNumber(InTimespan.GetSeconds(), &NumberFormatOptions));
		TextToSet = FText::Format(CustomTimespanFormat, TimeArguments);
	}

	return TextToSet;
}

TOptional<FText> UCommonDateTimeTextBlock::FormatDateTime(const FDateTime& InDateTime) const
{
	return FText::AsDateTime(InDateTime);
}
