// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonGameViewportClient.h"
#include "Engine/Console.h"
#include "Engine/GameInstance.h"
#include "Engine/LocalPlayer.h"
#include "Input/CursorReply.h"
#include "InputKeyEventArgs.h"

#if WITH_EDITOR
#endif // WITH_EDITOR

#include "Input/CommonUIActionRouterBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonGameViewportClient)

#define LOCTEXT_NAMESPACE ""

static const FName NAME_Typing = FName(TEXT("Typing"));
static const FName NAME_Open = FName(TEXT("Open"));

UCommonGameViewportClient::UCommonGameViewportClient(FVTableHelper& Helper) : Super(Helper)
{
}

bool UCommonGameViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	FInputKeyEventArgs EventArgs = InEventArgs;

	if (IsKeyPriorityAboveUI(EventArgs))
	{
		return true;
	}

	// Check override before UI
	if (OnOverrideInputKey().IsBound())
	{
		if (OnOverrideInputKey().Execute(EventArgs))
		{
			return true;
		}
	}

	// The input is fair game for handling - the UI gets first dibs
#if ALLOW_CONSOLE
	if (ViewportConsole && !ViewportConsole->ConsoleState.IsEqual(NAME_Typing) && !ViewportConsole->ConsoleState.IsEqual(NAME_Open))
#endif
	{		
		FReply Result = FReply::Unhandled();
		if (!OnRerouteInput().ExecuteIfBound(EventArgs.InputDevice, EventArgs.Key, EventArgs.Event, Result))
		{
			HandleRerouteInput(EventArgs.InputDevice, EventArgs.Key, EventArgs.Event, Result);
		}

		if (Result.IsEventHandled())
		{
			return true;
		}
	}

	return Super::InputKey(EventArgs);
}

bool UCommonGameViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	FReply RerouteResult = FReply::Unhandled();

	if (!OnRerouteAxis().ExecuteIfBound(Args.InputDevice, Args.Key, Args.AmountDepressed, RerouteResult))
	{
		HandleRerouteAxis(Args.InputDevice, Args.Key, Args.AmountDepressed, RerouteResult);
	}

	if (RerouteResult.IsEventHandled())
	{
		return true;
	}
	return Super::InputAxis(Args);
}

bool UCommonGameViewportClient::InputTouch(FViewport* const InViewport, const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation,
	const float Force, const uint64 Timestamp)
{
#if ALLOW_CONSOLE
	if (ViewportConsole != nullptr && (ViewportConsole->ConsoleState != NAME_Typing) && (ViewportConsole->ConsoleState != NAME_Open))
#endif
	{
		FReply Result = FReply::Unhandled();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!GetRerouteTouchDelegate().ExecuteIfBound(TouchId, Type, TouchLocation, Result) &&
			!OnRerouteTouchInput().ExecuteIfBound(TouchId.GetDeviceId(), TouchId.GetIndex(), Type, TouchLocation, Result))
		{
			HandleRerouteTouch(TouchId, Type, TouchLocation, Result);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (Result.IsEventHandled())
		{
			return true;
		}
	}

	return Super::InputTouch(InViewport, TouchId, Type, TouchLocation, Force, Timestamp);
}

void UCommonGameViewportClient::MouseMove(FViewport* InViewport, int32 X, int32 Y)
{
	if (ViewportConsole)
	{
		ViewportConsole->MouseMove(InViewport, X, Y);
	}
}

void UCommonGameViewportClient::CapturedMouseMove(FViewport* InViewport, int32 X, int32 Y)
{
	if (ViewportConsole)
	{
		ViewportConsole->CapturedMouseMove(InViewport, X, Y);
	}
}

void UCommonGameViewportClient::HandleRerouteInput(FInputDeviceId DeviceId, FKey Key, EInputEvent EventType, FReply& Reply)
{
	FPlatformUserId OwningPlatformUser = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(DeviceId);
	ULocalPlayer* LocalPlayer = GameInstance->FindLocalPlayerFromPlatformUserId(OwningPlatformUser);
	Reply = FReply::Unhandled();

	if (LocalPlayer)
	{
		UCommonUIActionRouterBase* ActionRouter = LocalPlayer->GetSubsystem<UCommonUIActionRouterBase>();
		if (ensure(ActionRouter))
		{
			ERouteUIInputResult InputResult = ActionRouter->ProcessInput(Key, EventType);
			if (InputResult == ERouteUIInputResult::BlockGameInput)
			{
				// We need to set the reply as handled otherwise the input won't actually be blocked from reaching the viewport.
				Reply = FReply::Handled();
				// Notify interested parties that we blocked the input.
				OnRerouteBlockedInput().ExecuteIfBound(DeviceId, Key, EventType, Reply);
			}
			else if (InputResult == ERouteUIInputResult::Handled)
			{
				Reply = FReply::Handled();
			}
		}
	}	
}

void UCommonGameViewportClient::HandleRerouteAxis(FInputDeviceId DeviceId, FKey Key, float Delta, FReply& Reply)
{
	// Get the ownign platform user for this input device and their local player
	FPlatformUserId OwningPlatformUser = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(DeviceId);
	ULocalPlayer* LocalPlayer = GameInstance->FindLocalPlayerFromPlatformUserId(OwningPlatformUser);
	
	Reply = FReply::Unhandled();

	if (LocalPlayer)
	{
		UCommonUIActionRouterBase* ActionRouter = LocalPlayer->GetSubsystem<UCommonUIActionRouterBase>();
		if (ensure(ActionRouter))
		{
			// We don't actually use axis inputs that reach the game viewport UI land for anything, we just want block them reaching the game when they shouldn't
			if (!ActionRouter->CanProcessNormalGameInput())
			{
				Reply = FReply::Handled();
			}
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UCommonGameViewportClient::HandleRerouteTouch(FInputDeviceId DeviceId, uint32 TouchId, ETouchType::Type TouchType, const FVector2D& TouchLocation, FReply& Reply)
{
	return HandleRerouteTouch(FTouchId(DeviceId, static_cast<ETouchIndex::Type>(TouchId)), TouchType, TouchLocation, Reply);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UCommonGameViewportClient::HandleRerouteTouch(FTouchId TouchId, ETouchType::Type TouchType, const FVector2D& TouchLocation, FReply& Reply)
{
	ULocalPlayer* LocalPlayer = GameInstance->FindLocalPlayerFromDeviceId(TouchId.GetDeviceId());
	Reply = FReply::Unhandled();

	if (LocalPlayer && TouchId.GetIndex() < EKeys::NUM_TOUCH_KEYS)
	{
		FKey KeyPressed = EKeys::TouchKeys[TouchId.GetIndex()];
		if (KeyPressed.IsValid())
		{
			UCommonUIActionRouterBase* ActionRouter = LocalPlayer->GetSubsystem<UCommonUIActionRouterBase>();
			if (ensure(ActionRouter))
			{
				EInputEvent SimilarInputEvent = IE_MAX;
				switch (TouchType)
				{
				case ETouchType::Began:
					SimilarInputEvent = IE_Pressed;
					break;
				case ETouchType::Ended:
					SimilarInputEvent = IE_Released;
					break;
				default:
					SimilarInputEvent = IE_Repeat;
					break;
				}

				if (ActionRouter->ProcessInput(KeyPressed, SimilarInputEvent) != ERouteUIInputResult::Unhandled)
				{
					Reply = FReply::Handled();
				}
			}
		}
	}
}

bool UCommonGameViewportClient::IsKeyPriorityAboveUI(const FInputKeyEventArgs& EventArgs)
{
#if ALLOW_CONSOLE
	// First priority goes to the viewport console regardless any state or setting
	if (ViewportConsole && ViewportConsole->InputKey(EventArgs.InputDevice, EventArgs.Key, EventArgs.Event, EventArgs.AmountDepressed, EventArgs.IsGamepad()))
	{
		return true;
	}
#endif

	// We'll also treat toggling fullscreen as a system-level sort of input that isn't affected by input filtering
	if (TryToggleFullscreenOnInputKey(EventArgs.Key, EventArgs.Event))
	{
		return true;
	}

	return false;
}

TOptional<TSharedRef<SWidget>> UCommonGameViewportClient::MapCursor(FViewport* InViewport, const FCursorReply& CursorReply)
{
	// When the virtual pointer is active, UI widgets may capture OnCursorQuery and return Default
	// before FSceneViewport gets to return Custom. Redirect any non-None cursor to Custom so the
	// registered virtual pointer widget is always resolved by the base class lookup.
	if (bUseVirtualPointerCursor && CursorReply.GetCursorType() != EMouseCursor::None)
	{
		return Super::MapCursor(InViewport, FCursorReply::Cursor(EMouseCursor::Custom));
	}
	return Super::MapCursor(InViewport, CursorReply);
}

#undef LOCTEXT_NAMESPACE

