// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonHardwareVisibilityBorder.h"
#include "CommonUIPrivate.h"

#include "CommonUIVisibilitySubsystem.h"
#include "CommonUISettings.h"
#include "ICommonUIModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonHardwareVisibilityBorder)

UCommonHardwareVisibilityBorder::UCommonHardwareVisibilityBorder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VisibleType(ESlateVisibility::SelfHitTestInvisible)
	, HiddenType(ESlateVisibility::Collapsed)
{
	SetPadding(FMargin(0.f,0.f,0.f,0.f));
}

void UCommonHardwareVisibilityBorder::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();
	UpdateVisibility();
	ListenToInputMethodChanged();
}

void UCommonHardwareVisibilityBorder::UpdateVisibility(UCommonUIVisibilitySubsystem* VisSystem)
{
	if (!IsDesignTime())
	{
		bool bVisibleForFeatures = true;

		if (!VisibilityQuery.IsEmpty())
		{
			const UCommonUIVisibilitySubsystem* CommonInputSubsystem = VisSystem ? VisSystem : UCommonUIVisibilitySubsystem::Get(GetOwningLocalPlayer());
			if (CommonInputSubsystem)
			{
				bVisibleForFeatures = VisibilityQuery.Matches(CommonInputSubsystem->GetVisibilityTags());
			}
			else
			{
				UE_LOGF(LogCommonUI, Verbose, "[%ls] -> ULocalPlayer not available, using PlatformTraits instead", *GetName());

				// If UCommonUIVisibilitySubsystem is unavailable use the hardware traits from UCommonUISettings
				const FGameplayTagContainer& HardwareTags = ICommonUIModule::GetSettings().GetPlatformTraits();
				bVisibleForFeatures = VisibilityQuery.Matches(HardwareTags);
			}
		}

		SetVisibility(bVisibleForFeatures ? VisibleType : HiddenType);
	}
}

void UCommonHardwareVisibilityBorder::ListenToInputMethodChanged()
{
	if (IsDesignTime())
	{
		return;
	}

	if (UCommonUIVisibilitySubsystem* CommonInputSubsystem = UCommonUIVisibilitySubsystem::Get(GetOwningLocalPlayer()))
	{
		CommonInputSubsystem->OnVisibilityTagsChanged.RemoveAll(this);
		CommonInputSubsystem->OnVisibilityTagsChanged.AddUObject(this, &ThisClass::HandleInputMethodChanged);
	}
}

void UCommonHardwareVisibilityBorder::HandleInputMethodChanged(UCommonUIVisibilitySubsystem* VisSystem)
{
	UpdateVisibility(VisSystem);
}

