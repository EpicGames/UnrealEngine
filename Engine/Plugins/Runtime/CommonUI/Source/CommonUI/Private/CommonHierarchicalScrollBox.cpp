// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonHierarchicalScrollBox.h"
#include "Components/ScrollBoxSlot.h"
#include "SCommonHierarchicalScrollBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonHierarchicalScrollBox)

/////////////////////////////////////////////////////
// UCommonHierarchicalScrollBox

UCommonHierarchicalScrollBox::UCommonHierarchicalScrollBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetAnimateWheelScrolling(true);
}

TSharedRef<SWidget> UCommonHierarchicalScrollBox::RebuildWidget()
{
	MyScrollBox = SNew(SCommonHierarchicalScrollBox)
		.Style(&GetWidgetStyle())
		.ScrollBarStyle(&GetWidgetBarStyle())
		.Orientation(GetOrientation())
		.ConsumeMouseWheel(GetConsumeMouseWheel())
		.NavigationDestination(GetNavigationDestination())
		.NavigationScrollPadding(GetNavigationScrollPadding())
		.OnUserScrolled(BIND_UOBJECT_DELEGATE(FOnUserScrolled, SlateHandleUserScrolled))
		.AnimateWheelScrolling(IsAnimateWheelScrolling());

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UScrollBoxSlot* TypedSlot = Cast<UScrollBoxSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyScrollBox.ToSharedRef());
		}
	}
	
	return MyScrollBox.ToSharedRef();
}
