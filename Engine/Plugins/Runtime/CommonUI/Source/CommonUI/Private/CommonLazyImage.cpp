// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonLazyImage.h"
#include "CommonWidgetPaletteCategories.h"

#include "Engine/Texture2D.h"
#include "ICommonUIModule.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "CommonUISettings.h"
#include "Engine/Texture2DDynamic.h"
#include "Widgets/Images/SImage.h"

#if WITH_EDITOR
#include "TextureCompiler.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonLazyImage)

UCommonLazyImage::UCommonLazyImage(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	LoadingBackgroundBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	LoadingThrobberBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
}

TSharedRef<SWidget> UCommonLazyImage::RebuildWidget()
{
	MyLoadGuard = SNew(SLoadGuard)
		.Throbber(LoadingThrobberBrush.DrawAs != ESlateBrushDrawType::NoDrawType ? SNew(SImage).Image(&LoadingThrobberBrush) : TSharedPtr<SWidget>(nullptr))
		.GuardBackgroundBrush(&LoadingBackgroundBrush)
		.OnLoadingStateChanged_UObject(this, &UCommonLazyImage::HandleLoadGuardStateChanged)
		[
			RebuildImageWidget()
		];

	if (ensure(MyImage))
	{
		// The image itself shouldn't be hit-testable as that can override this LazyImage's intended visibility
		MyImage->SetVisibility(EVisibility::SelfHitTestInvisible);
	}

	return MyLoadGuard.ToSharedRef();
}

void UCommonLazyImage::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();

	SetIsLoading(StreamingHandle.IsValid() && StreamingHandle->IsLoadingInProgress());
}

void UCommonLazyImage::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetThrobber(LoadingThrobberBrush.DrawAs != ESlateBrushDrawType::NoDrawType ? SNew(SImage).Image(&LoadingThrobberBrush) : TSharedPtr<SWidget>(nullptr));
	}

#if WITH_EDITOR
	if (IsDesignTime())
	{
		SetIsLoading(bShowLoading);
	}
#endif
}

void UCommonLazyImage::CancelImageStreaming()
{
	Super::CancelImageStreaming();

	SetIsLoading(false);
}

void UCommonLazyImage::OnImageStreamingStarted(TSoftObjectPtr<UObject> SoftObject)
{
	Super::OnImageStreamingStarted(SoftObject);

	SetIsLoading(true);
}

void UCommonLazyImage::OnImageStreamingComplete(TSoftObjectPtr<UObject> LoadedSoftObject)
{
	Super::OnImageStreamingComplete(LoadedSoftObject);

	SetIsLoading(false);
}

TSharedRef<SWidget> UCommonLazyImage::RebuildImageWidget()
{
	// Default to creating the image as UImage does
	return Super::RebuildWidget();
}

bool UCommonLazyImage::IsLoading() const
{
	return MyLoadGuard.IsValid() && MyLoadGuard->IsLoading();
}

void UCommonLazyImage::SetMaterialTextureParamName(FName TextureParamName)
{
	if (Cast<UMaterialInterface>(GetBrush().GetResourceObject()) != nullptr)
	{
		MaterialTextureParamName = TextureParamName;
	}
}

void UCommonLazyImage::SetBrush(const FSlateBrush& InBrush)
{
	Super::SetBrush(InBrush);
	MaterialTextureParamName = NAME_None;
}

void UCommonLazyImage::SetBrushFromAsset(USlateBrushAsset* Asset)
{
	Super::SetBrushFromAsset(Asset);
	MaterialTextureParamName = NAME_None;
}

void UCommonLazyImage::SetBrushFromTexture(UTexture2D* Texture, bool bMatchSize /*= false*/)
{
	SetBrushObjectInternal(Texture, bMatchSize);
}

void UCommonLazyImage::SetBrushFromTextureDynamic(UTexture2DDynamic* Texture, bool bMatchSize /*= false*/)
{
	SetBrushObjectInternal(Texture, bMatchSize);
}

void UCommonLazyImage::SetBrushFromMaterial(UMaterialInterface* Material)
{
	SetBrushObjectInternal(Material);
}

void UCommonLazyImage::SetBrushFromLazyTexture(const TSoftObjectPtr<UTexture2D>& LazyTexture, bool bMatchSize)
{
	if (!LazyTexture.IsNull())
	{
		SetBrushFromSoftTexture(LazyTexture, bMatchSize);
	}
	else
	{
		ShowDefaultImage();
	}
}

void UCommonLazyImage::SetBrushFromLazyMaterial(const TSoftObjectPtr<UMaterialInterface>& LazyMaterial)
{
	if (!LazyMaterial.IsNull())
	{
		SetBrushFromSoftMaterial(LazyMaterial);
	}
	else
	{
		ShowDefaultImage();
	}
}

void UCommonLazyImage::SetBrushFromLazyDisplayAsset(const TSoftObjectPtr<UObject>& LazyObject, bool bMatchTextureSize)
{
	if (!LazyObject.IsNull())
	{
		TWeakObjectPtr<UCommonLazyImage> WeakThis(this); // using weak ptr in case 'this' has gone out of scope by the time this lambda is called

		RequestAsyncLoad(LazyObject,
			[WeakThis, LazyObject, bMatchTextureSize]() {
				if (UImage* StrongThis = WeakThis.Get())
				{
					ensureMsgf(LazyObject.Get(), TEXT("Failed to load %s"), *LazyObject.ToSoftObjectPath().ToString());
					if (UTexture2D* AsTexture = Cast<UTexture2D>(LazyObject.Get()))
					{
						StrongThis->SetBrushFromTexture(AsTexture, bMatchTextureSize);
					}
					else if (TScriptInterface<ISlateTextureAtlasInterface> AsAtlasInterface = LazyObject.Get())
					{
						StrongThis->SetBrushFromAtlasInterface(AsAtlasInterface, bMatchTextureSize);
					}
					else if (UMaterialInterface* AsMaterial = Cast<UMaterialInterface>(LazyObject.Get()))
					{
						StrongThis->SetBrushFromMaterial(AsMaterial);
					}
				}
			}
		);
	}
	else
	{
		ShowDefaultImage();
	}
}

#if WITH_EDITOR
const FText UCommonLazyImage::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

bool UCommonLazyImage::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty->GetFName().IsEqual(GET_MEMBER_NAME_CHECKED(UCommonLazyImage, MaterialTextureParamName)))
	{
		// The param name is only relevant when the brush uses a material
		return Cast<UMaterialInterface>(GetBrush().GetResourceObject()) != nullptr;
	}
	return Super::CanEditChange(InProperty);
}

#endif

void UCommonLazyImage::SetIsLoading(bool bIsLoading)
{
	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetForceShowSpinner(bIsLoading);
	}
}

void UCommonLazyImage::HandleLoadGuardStateChanged(bool bIsLoading)
{
	OnLoadingStateChangedEvent.Broadcast(bIsLoading);
	BP_OnLoadingStateChanged.Broadcast(bIsLoading);
}

void UCommonLazyImage::ShowDefaultImage()
{
	UObject* DefaultBrushImage = ICommonUIModule::GetSettings().GetDefaultImageResourceObject();
	if (UMaterialInterface* AsMaterial = Cast<UMaterialInterface>(DefaultBrushImage))
	{
		SetBrushObjectInternal(AsMaterial);
	}
	else
	{
		SetBrushObjectInternal(Cast<UTexture2D>(DefaultBrushImage));
	}
}

void UCommonLazyImage::SetBrushObjectInternal(UMaterialInterface* Material)
{
	// This overrides our original material, so we won't try to set any params on it automatically anymore
	MaterialTextureParamName = NAME_None;
	UImage::SetBrushFromMaterial(Material);
}

void UCommonLazyImage::SetBrushObjectInternal(UTexture* Texture, bool bMatchSize /*= false*/)
{
	bool bAppliedTextureToMaterial = false;
	if (!MaterialTextureParamName.IsNone())
	{
		if (UMaterialInstanceDynamic* BrushMID = GetDynamicMaterial())
		{
			bAppliedTextureToMaterial = true;
			BrushMID->SetTextureParameterValue(MaterialTextureParamName, Texture);
			if (bMatchSize)
			{
				FSlateBrush CurrentBrush = GetBrush();
				if (UTexture2DDynamic* AsDynamicTexture = Cast<UTexture2DDynamic>(Texture))
				{
					CurrentBrush.ImageSize.X = AsDynamicTexture->SizeX;
					CurrentBrush.ImageSize.Y = AsDynamicTexture->SizeY;
				}
				else if (UTexture2D* AsTexture2D = Cast<UTexture2D>(Texture))
				{
#if WITH_EDITOR
					FTextureCompilingManager::Get().FinishCompilation({ Texture });
#endif
					CurrentBrush.ImageSize.X = AsTexture2D->GetSizeX();
					CurrentBrush.ImageSize.Y = AsTexture2D->GetSizeY();
				}
				Super::SetBrush(CurrentBrush);
			}
		}
	}

	// If we aren't setting the texture as a material param, just set it the usual way
	if (!bAppliedTextureToMaterial)
	{
		if (UTexture2DDynamic* AsDynamicTexture = Cast<UTexture2DDynamic>(Texture))
		{
			UImage::SetBrushFromTextureDynamic(AsDynamicTexture);
		}
		else
		{
			UImage::SetBrushFromTexture(Cast<UTexture2D>(Texture), bMatchSize);
		}
	}
}

void UCommonLazyImage::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyLoadGuard.Reset();
}
