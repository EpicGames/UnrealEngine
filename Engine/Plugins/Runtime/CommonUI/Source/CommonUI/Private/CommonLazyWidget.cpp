// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonLazyWidget.h"
#include "CommonActivatableWidget.h"
#include "CommonWidgetPaletteCategories.h"
#include "Engine/AssetManager.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonLazyWidget)

UCommonLazyWidget::UCommonLazyWidget(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	LoadingBackgroundBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	LoadingThrobberBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UCommonLazyWidget::SetLazyContent(const TSoftClassPtr<UUserWidget> SoftWidget)
{
	SetLazyContentInternal(SoftWidget, {});
}

void UCommonLazyWidget::SetLazyContentWithCallback(const TSoftClassPtr<UUserWidget> SoftWidget, const FOnWidgetCreated& OnCreatedCallback)
{
	SetLazyContentInternal<UUserWidget>(SoftWidget, [OnCreatedCallback](UUserWidget& Widget) { OnCreatedCallback.ExecuteIfBound(&Widget); });
}

void UCommonLazyWidget::LoadLazyContent()
{
	LoadLazyContent({});
}

TSharedRef<SWidget> UCommonLazyWidget::RebuildWidget()
{
	MyLoadGuard = SNew(SLoadGuard)
		.Throbber(LoadingThrobberBrush.DrawAs != ESlateBrushDrawType::NoDrawType ? SNew(SImage).Image(&LoadingThrobberBrush) : TSharedPtr<SWidget>(nullptr))
		.GuardBackgroundBrush(&LoadingBackgroundBrush)
		.OnLoadingStateChanged_UObject(this, &ThisClass::HandleLoadGuardStateChanged)
		[
			Content ? Content->TakeWidget() : SNullWidget::NullWidget
		];

	return MyLoadGuard.ToSharedRef();
}

void UCommonLazyWidget::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();

	if (StreamingHandle.IsValid() && StreamingHandle->IsLoadingInProgress())
	{
		MyLoadGuard->SetForceShowSpinner(true);
	}
}

void UCommonLazyWidget::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetThrobber(LoadingThrobberBrush.DrawAs != ESlateBrushDrawType::NoDrawType ? SNew(SImage).Image(&LoadingThrobberBrush) : TSharedPtr<SWidget>(nullptr));

#if WITH_EDITOR
		if (IsDesignTime())
		{
			if (!WidgetClass.IsNull())
			{
				SetLazyContent(WidgetClass);
			}
			else
			{
				MyLoadGuard->SetForceShowSpinner(true);
			}
		}
#endif
	}
}

void UCommonLazyWidget::SetForceShowSpinner(bool bShowLoading)
{
	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetForceShowSpinner(bShowLoading);
	}
}

void UCommonLazyWidget::CancelStreaming()
{
	if (StreamingHandle.IsValid())
	{
		StreamingHandle->CancelHandle();
		StreamingHandle.Reset();
	}

	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetForceShowSpinner(false);
	}
}

void UCommonLazyWidget::OnStreamingStarted(TSoftClassPtr<UObject> SoftObject)
{
	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetForceShowSpinner(true);
	}
}

void UCommonLazyWidget::OnStreamingComplete(TSoftClassPtr<UObject> LoadedSoftObject)
{
	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetForceShowSpinner(false);
	}
}

bool UCommonLazyWidget::IsLoading() const
{
	return MyLoadGuard.IsValid() && MyLoadGuard->IsLoading();
}

void UCommonLazyWidget::SetLoadedContent(UUserWidget* InContent)
{
	const bool bPerformActivateDeactivate = !IsDesignTime();

	if (bPerformActivateDeactivate)
	{
		if (UCommonActivatableWidget* OutgoingActivatable = Cast<UCommonActivatableWidget>(Content))
		{
			OutgoingActivatable->DeactivateWidget();
		}
	}

	Content = InContent;

	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetContent(InContent ? InContent->TakeWidget() : SNullWidget::NullWidget);
	}

	if (bPerformActivateDeactivate)
	{
		if (UCommonActivatableWidget* IncomingActivatable = Cast<UCommonActivatableWidget>(Content))
		{
			IncomingActivatable->ActivateWidget();
		}
	}

	OnContentChangedEvent.Broadcast(Content);
}

void UCommonLazyWidget::RequestAsyncLoad(TSoftClassPtr<UObject> SoftObject, TFunction<void()>&& Callback)
{
	RequestAsyncLoad(SoftObject, FStreamableDelegate::CreateLambda(MoveTemp(Callback)));
}

void UCommonLazyWidget::RequestAsyncLoad(TSoftClassPtr<UObject> SoftObject, FStreamableDelegate DelegateToCall)
{
	CancelStreaming();

	if (UObject* StrongObject = SoftObject.Get())
	{
		DelegateToCall.ExecuteIfBound();
		return;  // No streaming was needed, complete immediately.
	}

	OnStreamingStarted(SoftObject);

	TWeakObjectPtr<UCommonLazyWidget> WeakThis(this);
	StreamingObjectPath = SoftObject.ToSoftObjectPath();
	StreamingHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(
		StreamingObjectPath,
		[WeakThis, DelegateToCall, SoftObject]() {
			if (ThisClass* StrongThis = WeakThis.Get())
			{
				// If the object paths don't match, then this delegate was interrupted, but had already been queued for a callback
				// so ignore everything and abort.
				if (StrongThis->StreamingObjectPath != SoftObject.ToSoftObjectPath())
				{
					return; // Abort!
				}

				// Call the delegate to do whatever is needed, probably set the new image.
				DelegateToCall.ExecuteIfBound();

				// Note that the streaming has completed.
				StrongThis->OnStreamingComplete(SoftObject);
			}
		},
		FStreamableManager::AsyncLoadHighPriority);
}

#if WITH_EDITOR
const FText UCommonLazyWidget::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif

void UCommonLazyWidget::HandleLoadGuardStateChanged(bool bIsLoading)
{
	OnLoadingStateChangedEvent.Broadcast(bIsLoading);
	BP_OnLoadingStateChanged.Broadcast(bIsLoading);
}

void UCommonLazyWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyLoadGuard.Reset();
}
