// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonListView.h"
#include "CommonWidgetPaletteCategories.h"
#include "SCommonButtonTableRow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonListView)

//////////////////////////////////////////////////////////////////////////
// UCommonListView
//////////////////////////////////////////////////////////////////////////

UCommonListView::UCommonListView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetVisibilityInternal(ESlateVisibility::Visible);
	bEnableScrollAnimation = true;
}

void UCommonListView::SetEntrySpacing(float InEntrySpacing)
{
	SetHorizontalEntrySpacing(InEntrySpacing);
	SetVerticalEntrySpacing(InEntrySpacing);
}

#if WITH_EDITOR
const FText UCommonListView::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif

TSharedRef<STableViewBase> UCommonListView::RebuildListWidget()
{
	return ConstructListView<SCommonListView>();
}

UUserWidget& UCommonListView::OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (DesiredEntryClass->IsChildOf<UCommonButtonBase>())
	{
		return GenerateTypedEntry<UUserWidget, SCommonButtonTableRow<UObject*>>(DesiredEntryClass, OwnerTable);
	}
	return GenerateTypedEntry(DesiredEntryClass, OwnerTable);
}
