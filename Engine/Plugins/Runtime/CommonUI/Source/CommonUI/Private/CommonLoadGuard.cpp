// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonLoadGuard.h"
#include "Components/SizeBoxSlot.h"
#include "CommonWidgetPaletteCategories.h"
#include "CommonUIObjectVersion.h"
#include "Engine/AssetManager.h"
#include "Widgets/SOverlay.h"
#include "Engine/StreamableManager.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"
#include "ICommonUIModule.h"
#include "CommonUISettings.h"
#include "CommonUIEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonLoadGuard)

//////////////////////////////////////////////////////////////////////////
// SLoadGuard
//////////////////////////////////////////////////////////////////////////

SLoadGuard::SLoadGuard()
{
	SetCanTick(false);
}

void SLoadGuard::Construct(const FArguments& InArgs)
{
	FTextBlockStyle TextBlockStyle;
	const bool bTextStyleSet = (InArgs._GuardTextStyle != nullptr);
	if (bTextStyleSet)
	{
		TSubclassOf<UCommonTextStyle> SourceStyle = InArgs._GuardTextStyle;
		if (const UCommonTextStyle* StyleCDO = GetDefault<UCommonTextStyle>(SourceStyle))
		{
			StyleCDO->ToTextBlockStyle(TextBlockStyle);
		}
	}

	OnLoadingStateChanged = InArgs._OnLoadingStateChanged;

	TSharedRef<SWidget> Throbber = InArgs._Throbber.IsValid() ? InArgs._Throbber.ToSharedRef() : SNew(SImage).Image(&ICommonUIModule::GetSettings().GetDefaultThrobberBrush());

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(ContentBorder, SBorder)
			.Padding(0.f)
			.BorderImage(nullptr)
			.Visibility(EVisibility::SelfHitTestInvisible)
			[
				InArgs._Content.Widget
			]
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(GuardBorder, SBorder)
			.Visibility(EVisibility::Collapsed)
			.HAlign(InArgs._ThrobberHAlign)
			.VAlign(VAlign_Center)
			.BorderImage(InArgs._GuardBackgroundBrush)
			.Padding(0.f)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				.StretchDirection(EStretchDirection::DownOnly)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Expose(ThrobberSlot)
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(0.f, 0.f, InArgs._GuardText.IsEmpty() ? 0.f : 18.f, 0.f)
					[
						Throbber
					]

					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SAssignNew(GuardTextBlock, STextBlock)
						.Text(InArgs._GuardText)
					]
				]
			]
		]
	];


	if (bTextStyleSet)
	{
		GuardTextBlock->SetTextStyle(&TextBlockStyle);
	}

	SetForceShowSpinner(false);
}

FVector2D SLoadGuard::ComputeDesiredSize(float) const
{
	// The actual content of the guard is what we want driving our size - the throbber portion's desired size is irrelevant
	return ContentBorder->GetDesiredSize();
}

void SLoadGuard::SetForceShowSpinner(bool bInForceShowSpinner)
{
	bForceShowSpinner = bInForceShowSpinner;
	UpdateLoadingAppearance();
}

void SLoadGuard::SetContent(const TSharedRef<SWidget>& InContent)
{
	ContentBorder->SetContent(InContent);
}

void SLoadGuard::SetThrobberHAlign(EHorizontalAlignment InHAlign)
{
	GuardBorder->SetHAlign(InHAlign);
}

void SLoadGuard::SetThrobber(const TSharedPtr<SWidget>& InThrobber)
{
	if (ThrobberSlot)
	{
		ThrobberSlot->AttachWidget(InThrobber.IsValid() ? InThrobber.ToSharedRef() : SNew(SImage).Image(&ICommonUIModule::GetSettings().GetDefaultThrobberBrush()));
	}
}

void SLoadGuard::SetGuardText(const FText& InText)
{
	GuardTextBlock->SetText(InText);
}

void SLoadGuard::SetGuardTextStyle(const FTextBlockStyle& InGuardTextStyle)
{
	GuardTextBlock->SetTextStyle(&InGuardTextStyle);
}

void SLoadGuard::SetGuardBackgroundBrush(const FSlateBrush* InGuardBackground)
{
	GuardBorder->SetBorderImage(InGuardBackground);
}

void SLoadGuard::UpdateLoadingAppearance()
{
	if (bIsShowingSpinner && !StreamingHandle.IsValid() && !bForceShowSpinner)
	{
		// Showing the spinner, but we aren't loading anything and it's not forced, so display the content
		bIsShowingSpinner = false;
		OnLoadingStateChanged.ExecuteIfBound(false);
		ContentBorder->SetVisibility(EVisibility::SelfHitTestInvisible);
		GuardBorder->SetVisibility(EVisibility::Collapsed);
		Invalidate(EInvalidateWidgetReason::Layout);
	}
	else if (!bIsShowingSpinner && (StreamingHandle.IsValid() || bForceShowSpinner))
	{
		// We're loading something or the spinner is forced, so show it
		bIsShowingSpinner = true;
		OnLoadingStateChanged.ExecuteIfBound(true);
		ContentBorder->SetVisibility(EVisibility::Collapsed);
		GuardBorder->SetVisibility(EVisibility::SelfHitTestInvisible);
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SLoadGuard::GuardAndLoadAsset(const TSoftObjectPtr<UObject>& InLazyAsset, FOnLoadGuardAssetLoaded OnAssetLoaded)
{
	const bool bIsChangingAsset = LazyAsset != InLazyAsset;

	// Cancel previous streaming operation if we're changing assets.
	if (bIsChangingAsset)
	{
		if (StreamingHandle.IsValid())
		{
			StreamingHandle->CancelHandle();
			StreamingHandle.Reset();
		}
	}

	LazyAsset = InLazyAsset;

	// If the asset is already loaded, trigger the callback directly.
	if (InLazyAsset.IsValid())
	{
		OnAssetLoaded.ExecuteIfBound(InLazyAsset.Get());
	}
	// Otherwise, if we're changing assets and there's something to load, initiate the load.
	else if (bIsChangingAsset && !InLazyAsset.IsNull())
	{
		TWeakPtr<SLoadGuard> LocalWeakThis = SharedThis(this);
		StreamingHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(InLazyAsset.ToSoftObjectPath(),
			[this, LocalWeakThis, OnAssetLoaded]()
		{
			if (LocalWeakThis.IsValid() && LazyAsset.IsValid())
			{
				StreamingHandle.Reset();
				UpdateLoadingAppearance();

				OnAssetLoaded.ExecuteIfBound(LazyAsset.Get());
			}
		}, FStreamableManager::AsyncLoadHighPriority);
	}

	UpdateLoadingAppearance();
}

//////////////////////////////////////////////////////////////////////////
// ULoadGuardSlot
//////////////////////////////////////////////////////////////////////////

void ULoadGuardSlot::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (LoadGuard.IsValid())
	{
		TSharedRef<SBorder> ContentBorder = LoadGuard.Pin()->GetContentBorder();
		ContentBorder->SetPadding(Padding);
		ContentBorder->SetHAlign(HorizontalAlignment);
		ContentBorder->SetVAlign(VerticalAlignment);
	}
}

void ULoadGuardSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	LoadGuard.Reset();
}

void ULoadGuardSlot::BuildSlot(TSharedRef<SLoadGuard> InLoadGuard)
{
	LoadGuard = InLoadGuard;

	TSharedRef<SBorder> ContentBorder = InLoadGuard->GetContentBorder();
	ContentBorder->SetPadding(Padding);
	ContentBorder->SetHAlign(HorizontalAlignment);
	ContentBorder->SetVAlign(VerticalAlignment);

	ContentBorder->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

void ULoadGuardSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if (LoadGuard.IsValid())
	{
		LoadGuard.Pin()->GetContentBorder()->SetPadding(InPadding);
	}
}

void ULoadGuardSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if (LoadGuard.IsValid())
	{
		LoadGuard.Pin()->GetContentBorder()->SetHAlign(InHorizontalAlignment);
	}
}

void ULoadGuardSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if (LoadGuard.IsValid())
	{
		LoadGuard.Pin()->GetContentBorder()->SetVAlign(InVerticalAlignment);
	}
}

//////////////////////////////////////////////////////////////////////////
// UCommonLoadGuard
//////////////////////////////////////////////////////////////////////////

UCommonLoadGuard::UCommonLoadGuard(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, ThrobberAlignment(EHorizontalAlignment::HAlign_Center)
	, LoadingText(NSLOCTEXT("LoadGuard", "Loading", "Loading..."))
{
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);

	// Default to not showing the loading BG or throbber brushes - it's an opt-in kind of thing
	LoadingBackgroundBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	LoadingThrobberBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
}

void UCommonLoadGuard::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyLoadGuard.Reset();
}

void UCommonLoadGuard::SetLoadingText(const FText& InLoadingText)
{
	LoadingText = InLoadingText;
	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetGuardText(LoadingText);
	}
}

void UCommonLoadGuard::SetIsLoading(bool bInIsLoading)
{
	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetForceShowSpinner(bInIsLoading);
	}
}

bool UCommonLoadGuard::IsLoading() const
{
	return MyLoadGuard.IsValid() && MyLoadGuard->IsLoading();
}

void UCommonLoadGuard::BP_GuardAndLoadAsset(const TSoftObjectPtr<UObject>& InLazyAsset, const FOnAssetLoaded& OnAssetLoaded)
{
	GuardAndLoadAsset<UObject>(InLazyAsset, 
		[OnAssetLoaded] (UObject* Asset) 
		{
			OnAssetLoaded.ExecuteIfBound(Asset); 
		});
}

void UCommonLoadGuard::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(ECommonUIObjectVersion::Guid);
}

void UCommonLoadGuard::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(ECommonUIObjectVersion::Guid) < ECommonUIObjectVersion::CreatedLoadGuardSlot)
	{
		//Convert existing slot to a LoadGuardSlot
		if (UPanelSlot* PanelSlot = GetContentSlot())
		{
			ULoadGuardSlot* LoadGuardSlot = NewObject<ULoadGuardSlot>(this);
			LoadGuardSlot->Content = PanelSlot->Content;
			LoadGuardSlot->Content->Slot = LoadGuardSlot;
			LoadGuardSlot->Parent = this;

			// Copy properties from the old SizeBoxSlot
			if (USizeBoxSlot* AsSizeBoxSlot = Cast<USizeBoxSlot>(PanelSlot))
			{
				LoadGuardSlot->SetPadding(AsSizeBoxSlot->GetPadding());
				LoadGuardSlot->SetHorizontalAlignment(AsSizeBoxSlot->GetHorizontalAlignment());
				LoadGuardSlot->SetVerticalAlignment(AsSizeBoxSlot->GetVerticalAlignment());
			}
			Slots[0] = LoadGuardSlot;
		}
	}

#if WITH_EDITOR
	// Needed for backwards compatibility of old style updating
	if (!TextStyle && !bStyleNoLongerNeedsConversion && !IsRunningDedicatedServer())
	{
		UCommonUIEditorSettings& Settings = ICommonUIModule::GetEditorSettings();
		Settings.ConditionalPostLoad();
		TextStyle = Settings.GetTemplateTextStyle();
	}

	bStyleNoLongerNeedsConversion = true;
#endif
}

TSharedRef<SWidget> UCommonLoadGuard::RebuildWidget()
{
	MyLoadGuard = SNew(SLoadGuard)
		.GuardText(LoadingText)
		.GuardTextStyle(TextStyle)
		.GuardBackgroundBrush(&LoadingBackgroundBrush)
		.Throbber(LoadingThrobberBrush.DrawAs != ESlateBrushDrawType::NoDrawType ? SNew(SImage).Image(&LoadingThrobberBrush) : TSharedPtr<SWidget>(nullptr))
		.ThrobberHAlign(ThrobberAlignment)
		.OnLoadingStateChanged_UObject(this, &UCommonLoadGuard::HandleLoadingStateChanged);

	if (GetChildrenCount() > 0)
	{
		Cast<ULoadGuardSlot>(GetContentSlot())->BuildSlot(MyLoadGuard.ToSharedRef());
	}

	return MyLoadGuard.ToSharedRef();
}

void UCommonLoadGuard::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyLoadGuard.IsValid())
	{
		if (const UCommonTextStyle* StyleCDO = TextStyle ? GetDefault<UCommonTextStyle>(TextStyle) : nullptr)
		{
			FTextBlockStyle GuardTextStyle;
			StyleCDO->ToTextBlockStyle(GuardTextStyle);
			MyLoadGuard->SetGuardTextStyle(GuardTextStyle);
		}

		MyLoadGuard->SetGuardText(LoadingText);
		MyLoadGuard->SetThrobber(LoadingThrobberBrush.DrawAs != ESlateBrushDrawType::NoDrawType ? SNew(SImage).Image(&LoadingThrobberBrush) : TSharedPtr<SWidget>(nullptr));
		MyLoadGuard->SetThrobberHAlign(ThrobberAlignment);
		MyLoadGuard->SetGuardBackgroundBrush(&LoadingBackgroundBrush);

#if WITH_EDITORONLY_DATA
		MyLoadGuard->SetForceShowSpinner(bShowLoading);
#endif
	}
}

void UCommonLoadGuard::OnSlotAdded(UPanelSlot* NewSlot)
{
	if (MyLoadGuard.IsValid())
	{
		Cast<ULoadGuardSlot>(NewSlot)->BuildSlot(MyLoadGuard.ToSharedRef());
	}
}

void UCommonLoadGuard::OnSlotRemoved(UPanelSlot* OldSlot)
{
	if (MyLoadGuard.IsValid())
	{
		MyLoadGuard->SetContent(SNullWidget::NullWidget);
	}
}

#if WITH_EDITOR
void UCommonLoadGuard::OnCreationFromPalette()
{
	bStyleNoLongerNeedsConversion = true;
	if (!TextStyle)
	{
		TextStyle = ICommonUIModule::GetEditorSettings().GetTemplateTextStyle();
	}
	Super::OnCreationFromPalette();
}

const FText UCommonLoadGuard::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif

void UCommonLoadGuard::HandleLoadingStateChanged(bool bIsLoading)
{
	OnLoadingStateChanged().Broadcast(bIsLoading);
}

