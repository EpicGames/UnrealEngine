// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonNumericTextBlock.h"
#include "CommonWidgetPaletteCategories.h"
#include "Engine/World.h"
#include "TimerManager.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Math/UnitConversion.h"
#include "CommonCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonNumericTextBlock)

UCommonNumericTextBlock::UCommonNumericTextBlock(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CurrentNumericValue(0.0f)
	, NumericType(ECommonNumericType::Number)
	, EaseOutInterpolationExponent(1.5f)
	, InterpolationUpdateInterval(0.0f)
	, PostInterpolationShrinkDuration(0.25f)
	, PerformSizeInterpolation(true)
	, IsPercentage_DEPRECATED(false)
	, CurrentInterpolationState(EInterpolationState::None)
{
}

void UCommonNumericTextBlock::Tick(float DeltaSeconds)
{
	// Should never start a tick while not interpolating.
	if (CurrentInterpolationState == EInterpolationState::None)
	{
		// It's possible to get in this state when we are changing World and have a pending ClearTimer action, which will be caught in OnTimerTick.
		return;
	}

	// If we aren't in a valid tree then we shouldn't tick
	if (!GetCachedWidget().IsValid())
	{
		return;
	}

	// Calculate current state's duration.
	float CurrentStateDuration;
	switch (CurrentInterpolationState)
	{
	case EInterpolationState::NumericInterpolation:
		CurrentStateDuration = NumericInterpolationState.Duration;
		break;
	case EInterpolationState::SizeInterpolation:
		CurrentStateDuration = SizeInterpolationState.Duration;
		break;
	default:
		CurrentStateDuration = 0.0f;
		break;
	}

	// Update current state's elapsed duration, tracking any overage to pass on to the next state (if relevant).
	const float CurrentStateOverage = FMath::Max(0.0f, InterpolationState.ElapsedStateDuration + DeltaSeconds - CurrentStateDuration);
	InterpolationState.ElapsedStateDuration = FMath::Min(InterpolationState.ElapsedStateDuration + DeltaSeconds, CurrentStateDuration);

	const EInterpolationState PreTickInterpolationState = CurrentInterpolationState;

	// Run state-specific tick logic.
	switch (CurrentInterpolationState)
	{
	case EInterpolationState::NumericInterpolation:
		UpdateNumericInterpolation();
		break;
	case EInterpolationState::SizeInterpolation:
		UpdateSizeInterpolation();
		break;
	}

	// Pass the state duration overage on to the new state, if any, if possible.
	if (CurrentInterpolationState != PreTickInterpolationState)
	{
		InterpolationState.ElapsedStateDuration = CurrentStateOverage;
		switch (CurrentInterpolationState)
		{
		case EInterpolationState::NumericInterpolation:
			UpdateNumericInterpolation();
			break;
		case EInterpolationState::SizeInterpolation:
			UpdateSizeInterpolation();
			break;
		}
	}
}

void UCommonNumericTextBlock::OnTimerTick()
{
	// TimerTick requires the world to calculate the actual time delta between ticks.
	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	// Calculate time delta.
	const float CurrentTickTime = World->GetTimeSeconds();
	const float DeltaSeconds = CurrentTickTime - LastTimerTickTime;

	static const float MinimumUpdateInterval = 0.05f;
	const float UpdateRate = FMath::Max(InterpolationUpdateInterval, MinimumUpdateInterval);

	if (DeltaSeconds > UpdateRate)
	{
		// Leverage actual tick as time-delta-based workhorse function.
		Tick(DeltaSeconds);

		// Update tick time so future calls to TimerTick behave properly.
		LastTimerTickTime = CurrentTickTime;

		// Cancel timer if no longer interpolating.
		if (CurrentInterpolationState == EInterpolationState::None)
		{
			World->GetTimerManager().ClearTimer(TimerTickHandle);
			return;
		}
	}

	// If we're not done interpolating, try again next frame.
	TimerTickHandle = World->GetTimerManager().SetTimerForNextTick(this, &ThisClass::OnTimerTick);
}

#if WITH_EDITOR

bool UCommonNumericTextBlock::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// this should be move in a customizaer
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTextBlock, MinDesiredWidth))
		{
			bIsEditable = PerformSizeInterpolation != true;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return bIsEditable;
}

const FText UCommonNumericTextBlock::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif // WITH_EDITOR

void UCommonNumericTextBlock::SynchronizeProperties()
{
	UpdateUnderlyingText();

	Super::SynchronizeProperties();
}

float UCommonNumericTextBlock::GetTargetValue() const
{
	return IsInterpolatingNumericValue() ? NumericInterpolationState.DestinationValue : CurrentNumericValue;
}

void UCommonNumericTextBlock::SetCurrentValue(const float NewValue)
{
	// Cancel any ongoing interpolation.
	CancelInterpolation();

	// Set and update current numeric value.
	CurrentNumericValue = NewValue;
	UpdateUnderlyingText();
}

void UCommonNumericTextBlock::InterpolateToValue(const float InTargetValue, float InMaximumInterpolationDuration /*= 3.0f*/, float InMinimumChangeRate /*= 1.0f*/, float InOutroOffset /*= 0.0f*/)
{
	const bool bWasTimerActive = TimerTickHandle.IsValid();

	// Cancel any ongoing interpolation.
	CancelInterpolation();

	// Parameter validation and sanitization.
	ensureAlways(InMaximumInterpolationDuration > 0.0f);
	ensureAlways(InOutroOffset <= InMaximumInterpolationDuration);
	ensureAlways(InMinimumChangeRate >= 0.0f);
	InMaximumInterpolationDuration = FMath::Max(0.0f, InMaximumInterpolationDuration);
	InOutroOffset = FMath::Clamp(InOutroOffset, 0.0f, InMaximumInterpolationDuration);
	InMinimumChangeRate = FMath::Max(0.0f, InMinimumChangeRate);

	// No interpolation necessary if at target value already.
	if (InTargetValue == CurrentNumericValue)
	{
		return;
	}

	// Enter numeric interpolation state.
	const float InterpolationDuration = CalculateInterpolationDuration(InTargetValue, InMinimumChangeRate, InMaximumInterpolationDuration);
	const float OutroOffset = FMath::Clamp(InOutroOffset, 0.0f, InterpolationDuration);
	EnterNumericInterpolation(CurrentNumericValue, InTargetValue, InterpolationDuration, OutroOffset);

	// Start timer for interval ticking if an interval is specified.
	UWorld* const World = GetWorld();
	if (World && CurrentInterpolationState != EInterpolationState::None)
	{
		// Only update the last tick time if the timer wasn't already active.
		if (!bWasTimerActive)
		{
			LastTimerTickTime = World->GetTimeSeconds();
		}

		// Re-kick the timer.
		TimerTickHandle = World->GetTimerManager().SetTimerForNextTick(this, &ThisClass::OnTimerTick);
	}
}

float UCommonNumericTextBlock::CalculateInterpolationDuration(const float InTargetValue, const float InMinimumChangeRate, const float InMaximumInterpolationDuration) const
{
	// Calculate the properly signed minimum change rate, based on the total delta to be interpolated over.
	const float SignedMinimumChangeRate = InMinimumChangeRate * FMath::Sign(InTargetValue - CurrentNumericValue);

	// Calculate the necessary interpolation duration that is less than or equal to the maximum specified AND changes the numeric value by at least the specified minimum delta each second.
	// Due to the interpolation formula used, we can only solve for the maximum duration based on the minimum change rate if the rate is smaller than the total delta.
	if (FMath::Abs(SignedMinimumChangeRate) < FMath::Abs(InTargetValue - CurrentNumericValue))
	{
		// What LERP alpha value will give a result with a delta of the minimum change rate?
		// System of Equations:
		//		Result = CurrentNumericValue + MinimumLERPAlpha * (InTargetValue - CurrentNumericValue)
		//		Result = CurrentNumericValue + SignedChangeRate;
		// Solution for Minimum Alpha:
		//		CurrentNumericValue + SignedChangeRate = CurrentNumericValue + MinimumLERPAlpha * (InTargetValue - CurrentNumericValue)
		//		CurrentNumericValue + SignedChangeRate = CurrentNumericValue + MinimumLERPAlpha * (InTargetValue - CurrentNumericValue)
		//		SignedChangeRate = MinimumLERPAlpha * (InTargetValue - CurrentNumericValue)
		//		SignedChangeRate / (InTargetValue - CurrentNumericValue) = MinimumLERPAlpha
		//		MinimumLERPAlpha = SignedChangeRate / (InTargetValue - CurrentNumericValue)
		// What duration will cause EaseOutInterp to use the minimum LERP alpha value at 1 second?
		// System of Equations:
		//		MinimumLERPAlpha = 1.0 - Pow(1.0 - EaseOutAlpha, EaseOutInterpolationExponent)
		//		EaseOutAlpha = 1.0 / Duration
		//		MinimumLERPAlpha = SignedChangeRate / (InTargetValue - CurrentNumericValue)
		// Solution for Duration:
		//		SignedChangeRate / (InTargetValue - CurrentNumericValue) = 1.0 - Pow(1.0 - 1.0 / Duration, EaseOutInterpolationExponent)
		//		SignedChangeRate / (InTargetValue - CurrentNumericValue) - 1.0 = -Pow(1.0 - 1.0 / Duration, EaseOutInterpolationExponent)
		//		-SignedChangeRate / (InTargetValue - CurrentNumericValue) + 1.0 = Pow(1.0 - 1.0 / Duration, EaseOutInterpolationExponent)
		//		Root(-SignedChangeRate / (InTargetValue - CurrentNumericValue) + 1.0, EaseOutInterpolationExponent) = 1.0 - 1.0 / Duration
		//		Root(-SignedChangeRate / (InTargetValue - CurrentNumericValue) + 1.0, EaseOutInterpolationExponent) - 1.0 = -1.0 / Duration
		//		-Root(-SignedChangeRate / (InTargetValue - CurrentNumericValue) + 1.0, EaseOutInterpolationExponent) + 1.0 = 1.0 / Duration
		//		(-Root(-SignedChangeRate / (InTargetValue - CurrentNumericValue) + 1.0, EaseOutInterpolationExponent) + 1.0) * Duration = 1.0
		//		Duration = 1.0 / (-Root(-SignedChangeRate / (InTargetValue - CurrentNumericValue) + 1.0, EaseOutInterpolationExponent) + 1.0)
		const float DurationBasedOnMinimumChangeRate = 1.0 / (-FMath::Pow(-SignedMinimumChangeRate / (InTargetValue - CurrentNumericValue) + 1.0, 1.0 / EaseOutInterpolationExponent) + 1.0);

		return FMath::Clamp(DurationBasedOnMinimumChangeRate, 0.0f, InMaximumInterpolationDuration);
	}
	// Otherwise, fall back on a simple proportion of the total delta over the minimum change rate. IE: The duration it would take to cover the total delta at the specified rate if linearly interpolating.
	else
	{
		return FMath::Clamp((InTargetValue - CurrentNumericValue) / SignedMinimumChangeRate, 0.0f, InMaximumInterpolationDuration);
	}
}

bool UCommonNumericTextBlock::IsInterpolatingNumericValue() const
{
	return CurrentInterpolationState == EInterpolationState::NumericInterpolation;
}


void UCommonNumericTextBlock::SetNumericType(ECommonNumericType InNumericType)
{
	NumericType = InNumericType;
	UpdateUnderlyingText();
}

FNumberFormattingOptions UCommonNumericTextBlock::MakeNumberFormattingOptions() const
{
	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.RoundingMode = FormattingSpecification.RoundingMode;
	NumberFormattingOptions.AlwaysSign = FormattingSpecification.AlwaysSign;
	NumberFormattingOptions.UseGrouping = FormattingSpecification.UseGrouping;
	NumberFormattingOptions.MinimumIntegralDigits = FormattingSpecification.MinimumIntegralDigits;
	NumberFormattingOptions.MaximumIntegralDigits = FormattingSpecification.MaximumIntegralDigits;
	NumberFormattingOptions.MinimumFractionalDigits = FormattingSpecification.MinimumFractionalDigits;
	NumberFormattingOptions.MaximumFractionalDigits = FormattingSpecification.MaximumFractionalDigits;
	return NumberFormattingOptions;
}

FText UCommonNumericTextBlock::FormatText(float InCurrentNumericValue, const FNumberFormattingOptions& InNumberFormattingOptions) const
{
	switch (NumericType)
	{
		case ECommonNumericType::Number:
			return FText::AsNumber(InCurrentNumericValue, &InNumberFormattingOptions);
		case ECommonNumericType::Percentage:
			return FText::AsPercent(InCurrentNumericValue, &InNumberFormattingOptions);
		case ECommonNumericType::Seconds:
		{
			FTimespan CurrentSeconds = FTimespan::FromSeconds(InCurrentNumericValue);
			return FText::AsTimespan(CurrentSeconds);
		}
		case ECommonNumericType::Distance:
		{
			FNumericUnit<float> DisplayUnit = FUnitConversion::QuantizeUnitsToBestFit<float>(InCurrentNumericValue, EUnit::Centimeters);

			return FText::Format(FText::FromString(TEXT("{0} {1}")),
				FText::AsNumber(DisplayUnit.Value, &InNumberFormattingOptions),
				FText::FromString(FUnitConversion::GetUnitDisplayString(DisplayUnit.Units)));
		}
	}

	return FText::GetEmpty();
}

void UCommonNumericTextBlock::UpdateUnderlyingText()
{
	const FNumberFormattingOptions& NumberFormattingOptions = MakeNumberFormattingOptions();
	const FText NumeralText = FormatText(CurrentNumericValue, NumberFormattingOptions);
	SetText(NumeralText);
}

void UCommonNumericTextBlock::EnterNumericInterpolation(const float InitialValue, const float FinalValue, const float Duration, const float OutroOffset)
{
	OnInterpolationStartedEvent.Broadcast(this);

	CurrentInterpolationState = EInterpolationState::NumericInterpolation;

	// Initialize interpolation state.
	InterpolationState.ElapsedStateDuration = 0.0f;
	NumericInterpolationState.SourceValue = InitialValue;
	NumericInterpolationState.DestinationValue = FinalValue;
	NumericInterpolationState.OutroOffset = OutroOffset;
	NumericInterpolationState.HasTriggeredOutro = false;
	NumericInterpolationState.Duration = Duration;

	// Initial tick.
	UpdateNumericInterpolation();
}

void UCommonNumericTextBlock::UpdateNumericInterpolation()
{
	if (PerformSizeInterpolation)
	{
		// Update text block's minimum desired width so as not to shrink. Should be done before setting new text, as we don't want to shrink width from a previous update.
		SetMinDesiredWidth(GetDesiredSize().X);
	}

	const float LastNumericValue = CurrentNumericValue;

	// Set current numeric value based on "ease out" interpolation towards target numeric value.
	const float Alpha = FMath::Clamp(InterpolationState.ElapsedStateDuration / NumericInterpolationState.Duration, 0.0f, 1.0f);
	CurrentNumericValue = FMath::InterpEaseOut(NumericInterpolationState.SourceValue, NumericInterpolationState.DestinationValue, Alpha, EaseOutInterpolationExponent);
	UpdateUnderlyingText();

	OnInterpolationUpdatedEvent.Broadcast(this, LastNumericValue, CurrentNumericValue);

	// Trigger outro event if not yet triggered and we're at or past the outro point.
	const bool ShouldHaveTriggeredOutro = InterpolationState.ElapsedStateDuration >= NumericInterpolationState.Duration - NumericInterpolationState.OutroOffset;
	if (!NumericInterpolationState.HasTriggeredOutro && ShouldHaveTriggeredOutro)
	{
		NumericInterpolationState.HasTriggeredOutro = true;
		OnOutroEvent.Broadcast(this);
	}

	// If interpolation duration has elapsed...
	const bool HasNumericInterpolationDurationElapsed = InterpolationState.ElapsedStateDuration >= NumericInterpolationState.Duration;
	if (HasNumericInterpolationDurationElapsed)
	{
		ExitNumericInterpolation(true);

		if (PerformSizeInterpolation)
		{
			EnterSizeInterpolation(PostInterpolationShrinkDuration);
		}
	}
}

void UCommonNumericTextBlock::ExitNumericInterpolation(const bool HasCompleted)
{
	CurrentInterpolationState = EInterpolationState::None;

	OnInterpolationEndedEvent.Broadcast(this, HasCompleted);
}

void UCommonNumericTextBlock::EnterSizeInterpolation(const float Duration)
{
	CurrentInterpolationState = EInterpolationState::SizeInterpolation;

	// Initialize interpolation state.
	InterpolationState.ElapsedStateDuration = 0.0f;
	SizeInterpolationState.InitialWidth = GetDesiredSize().X;
	SizeInterpolationState.Duration = Duration;

	// Initial tick.
	UpdateSizeInterpolation();
}

void UCommonNumericTextBlock::UpdateSizeInterpolation()
{
	// Calculate the width the text block would have if not for the forced minimum.
	SetMinDesiredWidth(0);
	const TSharedRef<STextBlock> UnderlyingWidget = StaticCastSharedRef<STextBlock>(TakeWidget());
	const int32 TargetDesiredWidth = UnderlyingWidget->ComputeDesiredSize(FSlateApplication::Get().GetApplicationScale()).X;

	// Set text block's minimum desired width based on "ease out" interpolation.
	const float Alpha = FMath::Clamp(InterpolationState.ElapsedStateDuration / SizeInterpolationState.Duration, 0.0f, 1.0f);
	SetMinDesiredWidth(FMath::InterpEaseOut(SizeInterpolationState.InitialWidth, TargetDesiredWidth, Alpha, EaseOutInterpolationExponent));

	// If the shrink duration has elapsed...
	const bool HasSizeInterpolationDurationElapsed = InterpolationState.ElapsedStateDuration >= SizeInterpolationState.Duration;
	if (HasSizeInterpolationDurationElapsed)
	{
		ExitSizeInterpolation();
	}
}

void UCommonNumericTextBlock::ExitSizeInterpolation()
{
	// Clear minimum desired width.
	SetMinDesiredWidth(0);

	CurrentInterpolationState = EInterpolationState::None;
}

void UCommonNumericTextBlock::CancelInterpolation()
{
	switch (CurrentInterpolationState)
	{
	case EInterpolationState::NumericInterpolation:
		ExitNumericInterpolation();

		if (PerformSizeInterpolation)
		{
			// Enter and exit size interpolation for consistency of order of events.
			EnterSizeInterpolation(0.0f);
			ExitSizeInterpolation();
		}
		break;
	case EInterpolationState::SizeInterpolation:
		ExitSizeInterpolation();
		break;
	}

	// Cancel timer if no longer interpolating.
	if (CurrentInterpolationState == EInterpolationState::None && TimerTickHandle.IsValid())
	{
		if(UWorld* const World = GetWorld())
		{
			World->GetTimerManager().ClearTimer(TimerTickHandle);
		}
	}
}

void UCommonNumericTextBlock::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCommonCustomVersion::GUID);
}

void UCommonNumericTextBlock::PostLoad()
{
	Super::PostLoad();

	const int32 PaperVer = GetLinkerCustomVersion(FCommonCustomVersion::GUID);

	if (PaperVer < FCommonCustomVersion::RemovingIsPercentage)
	{
		NumericType = IsPercentage_DEPRECATED ? ECommonNumericType::Percentage : ECommonNumericType::Number;
	}
}
