// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonPoolableWidgetInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonPoolableWidgetInterface)

