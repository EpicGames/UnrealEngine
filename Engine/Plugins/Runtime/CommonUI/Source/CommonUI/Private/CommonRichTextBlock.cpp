// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonRichTextBlock.h"
#include "ICommonUIModule.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/STextScroller.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#include "Framework/Text/SlateTextRun.h"

#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"
#include "Misc/DefaultValueHelper.h"
#include "Fonts/FontMeasure.h"
#include "CommonUIUtils.h"
#include "CommonWidgetPaletteCategories.h"
#include "CommonUIEditorSettings.h"
#include "CommonUIRichTextData.h"
#include "Framework/Text/RichTextMarkupProcessing.h"
#include "Framework/Application/SlateApplication.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonRichTextBlock)

namespace UE::CommonUI::CommonRichTextBlock
{
	int32 bApplyMobileScaleOnTextRuns = 0;
	FAutoConsoleVariableRef CVarApplyMobileScaleOnTextRuns(
		TEXT("CommonRichTextBlock.CVarApplyMobileScaleOnTextRuns"),
		bApplyMobileScaleOnTextRuns,
		TEXT("Enable the mobile scale on the text when it gets created. It is off by default because it can result its scale being applied twice in tagged text.\n0: Disable (default), 1: Enable"),
		ECVF_Default);
}

namespace SupportedMarkupKeys
{
	const FString Id = TEXT("id");
	
	// Text-specific
	const FString Text = TEXT("text");

	const FString Case = TEXT("case");
	const FString Case_Upper = TEXT("upper");
	const FString Case_Lower = TEXT("lower");

	const FString Color = TEXT("color");
	const FString Opacity = TEXT("opacity");
	const FString FontFace = TEXT("fontface");
	const FString Size = TEXT("size");
	const FString Material = TEXT("mat");

	// Input
	const FString Input = TEXT("input");
	const FString Action = TEXT("action");
	const FString Axis = TEXT("axis");
	const FString Key = TEXT("key");
	const FString Borderless = TEXT("borderless");
}

class FCommonRichTextDecorator : public ITextDecorator
{
public:
	static TSharedRef<FCommonRichTextDecorator> CreateInstance(UCommonRichTextBlock& InOwner)
	{
		return MakeShareable(new FCommonRichTextDecorator(InOwner));
	}
	
	virtual ~FCommonRichTextDecorator() {}

	virtual bool Supports(const FTextRunParseResults& RunParseResult, const FString& Text) const override
	{
		// Basic decorator supports inline text styling
		return RunParseResult.Name == SupportedMarkupKeys::Text;
	}

	virtual TSharedRef<ISlateRun> Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef<FString>& InOutModelText, const ISlateStyle* Style) override final
	{
		FTextRange ModelRange;
		ModelRange.BeginIndex = InOutModelText->Len();

		FTextRunInfo RunInfo(RunParseResult.Name, FText::FromString(OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.EndIndex - RunParseResult.ContentRange.BeginIndex)));
		for (const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
		{
			RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid(Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
		}

		FTextBlockStyle TextStyle = Owner->GetCurrentDefaultTextStyle();
		for (const auto& TagValuePair : RunInfo.MetaData)
		{
			const FString& Tag = TagValuePair.Key;
			const FString& Value = TagValuePair.Value;
			if (Tag == SupportedMarkupKeys::Color)
			{
				TextStyle.SetColorAndOpacity(FLinearColor::FromSRGBColor(FColor::FromHex(Value)));
			}
			else if (Tag == SupportedMarkupKeys::FontFace)
			{
				TextStyle.SetTypefaceFontName(*Value);
			}
			else if (Tag == SupportedMarkupKeys::Size)
			{
				float FontSize;
				LexFromString(FontSize, *Value);
				TextStyle.SetFontSize(FontSize);
			}
			else if (Tag == SupportedMarkupKeys::Case)
			{
				if (Value == SupportedMarkupKeys::Case_Lower)
				{
					RunInfo.Content = RunInfo.Content.ToLower();
				}
				else if (Value == SupportedMarkupKeys::Case_Upper)
				{
					RunInfo.Content = RunInfo.Content.ToUpper();
				}
			}
			else if (Tag == SupportedMarkupKeys::Material)
			{
				if (Value.IsEmpty())
				{
					TextStyle.Font.FontMaterial = nullptr;
				}
			}
		}

		if (CommonUIUtils::ShouldScaleText() && UE::CommonUI::CommonRichTextBlock::bApplyMobileScaleOnTextRuns)
		{
			float Scale = 1.0f;
			if (CommonUIUtils::ShouldDisplayMobileUISizes())
			{
				Scale = Owner->GetMobileTextBlockScale();
			}
			else if (CommonUIUtils::UISmallTextScaling > 0.0f)
			{
				if (TextStyle.Font.Size <= CommonUIUtils::UISmallTextScalingThresholdMax)
				{
					Scale = CommonUIUtils::UISmallTextScaling;
				}
			}
			else
			{
				checkNoEntry();
			}

			TextStyle.SetFontSize(TextStyle.Font.Size * Scale);
		}

		TSharedPtr<ISlateRun> SlateRun;
		TSharedPtr<SWidget> DecoratorWidget = CreateDecoratorWidget(RunInfo, TextStyle);
		if (DecoratorWidget.IsValid())
		{
			*InOutModelText += TEXT('\u200B'); // Zero-Width Breaking Space
			ModelRange.EndIndex = InOutModelText->Len();

			// Calculate the baseline of the text within the owning rich text
			// Requested on demand as the font may not be loaded right now
			const FSlateFontInfo Font = TextStyle.Font;
			const float ShadowOffsetY = FMath::Min(0.0f, TextStyle.ShadowOffset.Y);

			TAttribute<int16> GetBaseline = TAttribute<int16>::CreateLambda([Font, ShadowOffsetY]()
			{
				const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				return FontMeasure->GetBaseline(Font) - ShadowOffsetY;
			});

			FSlateWidgetRun::FWidgetRunInfo WidgetRunInfo(DecoratorWidget.ToSharedRef(), GetBaseline);
			SlateRun = FSlateWidgetRun::Create(TextLayout, RunInfo, InOutModelText, WidgetRunInfo, ModelRange);
		}
		else
		{
			*InOutModelText += RunInfo.Content.ToString();
			ModelRange.EndIndex = InOutModelText->Len();
			SlateRun = FSlateTextRun::Create(RunInfo, InOutModelText, TextStyle, ModelRange);
		}

		return SlateRun.ToSharedRef();
	}

protected:
	FCommonRichTextDecorator(UCommonRichTextBlock& InOwner)
		: Owner(&InOwner)
	{}

	virtual TSharedPtr<SWidget> CreateDecoratorWidget(const FTextRunInfo& RunInfo, const FTextBlockStyle& DefaultTextStyle) const
	{
		return TSharedPtr<SWidget>();
	}

	TWeakObjectPtr<UCommonRichTextBlock> Owner;
};

struct FInlineIconEntry
{
	FText Name;
	FVector2D ImageSize;
	TSoftObjectPtr<UObject> ResourceObject;

	FInlineIconEntry() {}
	FInlineIconEntry(const FText& InName, const FVector2D& InImageSize, const TSoftObjectPtr<UObject>& InResourceObject)
		: Name(InName)
		, ImageSize(InImageSize)
		, ResourceObject(InResourceObject)
	{}
};

class SCommonInlineIcon : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SCommonInlineIcon)
		: _DisplayMode(ERichTextInlineIconDisplayMode::IconAndText)
		, _IconScale(0.75f)
		, _bTintIcon(false)
		, _TransformPolicy(ETextTransformPolicy::None)
	{}
		SLATE_ARGUMENT(ERichTextInlineIconDisplayMode, DisplayMode)
		SLATE_ARGUMENT(float, IconScale)
		SLATE_ARGUMENT(bool, bTintIcon)
		SLATE_ARGUMENT(ETextTransformPolicy, TransformPolicy)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const FInlineIconEntry& Entry, const FTextBlockStyle& TextStyle)
	{
		TSharedPtr<SWidget> ContentWidget;

		switch (InArgs._DisplayMode)
		{
		case ERichTextInlineIconDisplayMode::IconOnly:
			ContentWidget = CreateIconWidget(Entry, TextStyle, InArgs._IconScale, InArgs._bTintIcon);
			break;
		case ERichTextInlineIconDisplayMode::TextOnly:
			ContentWidget = CreateTextWidget(Entry, TextStyle, InArgs._TransformPolicy);
			break;
		case ERichTextInlineIconDisplayMode::IconAndText:
			ContentWidget = 
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 8.f, 0.f)
				[
					CreateIconWidget(Entry, TextStyle, InArgs._IconScale, InArgs._bTintIcon)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					CreateTextWidget(Entry, TextStyle, InArgs._TransformPolicy)
				];
			break;
		}

		ChildSlot
		[
			ContentWidget.IsValid() ? ContentWidget.ToSharedRef() : SNullWidget::NullWidget
		];
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		IconBrush.AddReferencedObjects(Collector);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("SCommonInlineIcon");
	}

private:
	TSharedRef<SWidget> CreateIconWidget(const FInlineIconEntry& Entry, const FTextBlockStyle& TextStyle, float IconScale, bool bTintIcon)
	{
#if UE_BUILD_SHIPPING
		if (Entry.ResourceObject.IsNull())
		{
			// Just be blank in shipping if there's no texture
			return SNullWidget::NullWidget;
		}
#endif
		IconBrush.ImageSize = Entry.ImageSize;
		TSoftObjectPtr<UObject> LazyAsset = Entry.ResourceObject;
		if (!LazyAsset.IsValid())
		{
			TWeakPtr<SCommonInlineIcon> LocalWeakThis = SharedThis(this);
			UAssetManager::GetStreamableManager().RequestAsyncLoad(LazyAsset.ToSoftObjectPath(),
				[this, LocalWeakThis, LazyAsset]()
				{
					if (LocalWeakThis.IsValid() && LazyAsset.IsValid())
					{
						IconBrush.SetResourceObject(LazyAsset.Get());
					}
				}, FStreamableManager::AsyncLoadHighPriority);
		}
		else
		{
			IconBrush.SetResourceObject(LazyAsset.Get());
		}

		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const float MaxTextHeight = (float)FontMeasure->GetMaxCharacterHeight(TextStyle.Font);
		const float IconSize = FMath::Min(MaxTextHeight * IconScale, IconBrush.ImageSize.Y);
		IconBrush.ImageSize.X = IconSize;
		IconBrush.ImageSize.Y = IconSize;

		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.HeightOverride(MaxTextHeight)
			.WidthOverride(MaxTextHeight)
			[
				SNew(SImage)
				.Image(&IconBrush)
				.ColorAndOpacity(bTintIcon ? TextStyle.ColorAndOpacity : FLinearColor::White)
			];
	}

	TSharedRef<SWidget> CreateTextWidget(const FInlineIconEntry& Entry, const FTextBlockStyle& TextStyle, ETextTransformPolicy TransformPolicy)
	{
		return SNew(STextBlock)
			.Text(Entry.Name)
			.TransformPolicy(TransformPolicy)
			.TextStyle(&TextStyle);
	}

	FSlateBrush IconBrush;
};

class FCommonDecorator_InlineIcon : public FCommonRichTextDecorator
{
public:
	static TSharedRef<FCommonDecorator_InlineIcon> CreateInstance(UCommonRichTextBlock& InOwner)
	{
		return MakeShareable(new FCommonDecorator_InlineIcon(InOwner));
	}

	virtual bool Supports(const FTextRunParseResults& RunParseResult, const FString& Text) const override
	{
		return InlineIconsByTag.Contains(RunParseResult.Name.ToLower());
	}

protected:
	virtual TSharedPtr<SWidget> CreateDecoratorWidget(const FTextRunInfo& RunInfo, const FTextBlockStyle& TextStyle) const override
	{
		FTextBlockStyle ModifiedTextStyle = TextStyle;
		if (const FString* HexColor = RunInfo.MetaData.Find(SupportedMarkupKeys::Color))
		{
			ModifiedTextStyle.SetColorAndOpacity(FLinearColor::FromSRGBColor(FColor::FromHex(*HexColor)));
		}

		float IconScale = 0.75f; // default value
		if (const FString* ScaleString = RunInfo.MetaData.Find(TEXT("scale")))
		{
			FDefaultValueHelper::ParseFloat(*ScaleString, IconScale);
		}
		
		FString InlineIconEntryName = RunInfo.Name.ToLower();
		const FInlineIconEntry* InlineIconEntry = InlineIconsByTag.Find(InlineIconEntryName);
		if (ensure(InlineIconEntry))
		{
			// Real money symbols are within fonts and will never have an icon, so we force text only mode now 
			// to make sure we don't have an errant icon potentially appear (or a pink error box in non-shipping)
			// Ideally, this could be more data-driven, but that would be overkill for what is currently a singleton use case
			ERichTextInlineIconDisplayMode EntryDisplayMode = InlineIconEntryName == TEXT("cash") ? ERichTextInlineIconDisplayMode::TextOnly : Owner->InlineIconDisplayMode;

			// Note: Icon may not be tinted same color as text, for now must be addressed in game
			return SNew(SCommonInlineIcon, *InlineIconEntry, ModifiedTextStyle)
				.DisplayMode(EntryDisplayMode)
				.IconScale(IconScale)
				.bTintIcon(Owner->bTintInlineIcon)
				.TransformPolicy(Owner->GetTextTransformPolicy());
		}
		return TSharedPtr<SWidget>();
	}

private:
	static TMap<FString, FInlineIconEntry> InlineIconsByTag;

	FCommonDecorator_InlineIcon(UCommonRichTextBlock& InOwner)
		: FCommonRichTextDecorator(InOwner)
	{
		if (InlineIconsByTag.Num() == 0)
		{
			if (UCommonUIRichTextData* RichTextData = UCommonUIRichTextData::Get())
			{
				for (const TPair<FName, uint8*>& IconPair : RichTextData->GetIconMap())
				{
					if (FRichTextIconData* IconData = (FRichTextIconData*)IconPair.Value)
					{
						InlineIconsByTag.Add(IconPair.Key.ToString(), FInlineIconEntry(IconData->DisplayName, IconData->ImageSize, IconData->ResourceObject));
					}
				}
			}
		}
	}
};
TMap<FString, FInlineIconEntry> FCommonDecorator_InlineIcon::InlineIconsByTag;

void UCommonRichTextBlock::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyTextScroller.Reset();
}

void UCommonRichTextBlock::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && bDisplayAllCaps_DEPRECATED)
	{
		SetTextTransformPolicy(ETextTransformPolicy::ToUpper);
		bDisplayAllCaps_DEPRECATED = false;
	}
}

void UCommonRichTextBlock::SetText(const FText& InText)
{
	Super::SetText(InText);
	if (MyRichTextBlock.IsValid())
	{
		if (CommonUIUtils::ShouldScaleText())
		{
			ApplyTextBlockScale();
		}
		MyRichTextBlock->SetText(InText);
	}

	if (bAutoCollapseWithEmptyText)
	{
		SetVisibility(InText.IsEmpty() ? ESlateVisibility::Collapsed : ESlateVisibility::SelfHitTestInvisible);
	}
}

void UCommonRichTextBlock::SetStyle(const TSubclassOf<UCommonTextStyle>& InStyle)
{
	DefaultTextStyleOverrideClass = InStyle;
	SynchronizeProperties();
}

void UCommonRichTextBlock::SetScrollingEnabled(bool bInIsScrollingEnabled)
{
	bIsScrollingEnabled = bInIsScrollingEnabled;
	SynchronizeProperties();
}

#if WITH_EDITOR

void UCommonRichTextBlock::OnCreationFromPalette()
{
	DefaultTextStyleOverrideClass = ICommonUIModule::GetEditorSettings().GetTemplateTextStyle();
	RefreshOverrideStyle();
}

const FText UCommonRichTextBlock::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

bool UCommonRichTextBlock::CanEditChange(const FProperty* InProperty) const
{
	const FName PropertyName = InProperty->GetFName();

	if (DefaultTextStyleOverrideClass)
	{
		if (PropertyName.IsEqual(TEXT("DefaultTextStyleOverride")) ||
			PropertyName.IsEqual(TEXT("MinDesiredWidth")))
		{
			return false;
		}
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bAutoCollapseWithEmptyText && PropertyName.IsEqual(GET_MEMBER_NAME_CHECKED(UCommonRichTextBlock, Visibility)))
	{
		return false;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return Super::CanEditChange(InProperty);
}

#endif

/*static*/ FString UCommonRichTextBlock::EscapeStringForRichText(FString InString)
{
	TSharedRef<FDefaultRichTextMarkupWriter> Writer = FDefaultRichTextMarkupWriter::GetStaticInstance();
	FString Result;
	TArray<IRichTextMarkupWriter::FRichTextLine> Line;
	Line.Add(
		IRichTextMarkupWriter::FRichTextLine
		{
			{
				IRichTextMarkupWriter::FRichTextRun(FRunInfo(), MoveTemp(InString))
			}
		}
	);
	Writer->Write(Line, Result);

	return Result;
}

TSharedRef<SWidget> UCommonRichTextBlock::RebuildWidget()
{
	const UCommonTextScrollStyle* TextScrollStyle = ScrollStyle.GetDefaultObject();
	if (!TextScrollStyle)
	{
		return Super::RebuildWidget();
	}

	// If the clipping mode is the default, but we're using a scrolling style,
	// we need to switch over to a clip to bounds style.
	if (GetClipping() == EWidgetClipping::Inherit)
	{
		SetClipping(EWidgetClipping::OnDemand);
	}

	MyTextScroller =
		SNew(STextScroller)
		.ScrollOptions(TextScrollStyle->ToScrollOptions())
		.ScrollOrientation(ScrollOrientation)
		[
			Super::RebuildWidget()
		];
	
	// Set the inner text block to self hit test invisible
	MyRichTextBlock->SetVisibility(EVisibility::SelfHitTestInvisible);

#if WIDGET_INCLUDE_RELFECTION_METADATA
	MyRichTextBlock->AddMetadata<FReflectionMetaData>(MakeShared<FReflectionMetaData>(GetFName(), GetClass(), this, GetSourceAssetOrClass()));
#endif

	return MyTextScroller.ToSharedRef();
}

void UCommonRichTextBlock::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (bAutoCollapseWithEmptyText)
	{
		SetVisibility(GetText().IsEmpty() ? ESlateVisibility::Collapsed : ESlateVisibility::SelfHitTestInvisible);
	}
	
	RefreshOverrideStyle();

	if (MyRichTextBlock.IsValid())
	{
		if (CommonUIUtils::ShouldScaleText() && DefaultTextStyleOverrideClass.GetDefaultObject() == nullptr)
		{
			ApplyTextBlockScale();
		}
	}

	if (MyTextScroller.IsValid())
	{
		if (bIsScrollingEnabled)
		{
			MyTextScroller->StartScrolling();

			if (MyRichTextBlock.IsValid())
			{
				MyRichTextBlock->SetOverflowPolicy(ETextOverflowPolicy::Clip);
			}
		}
		else
		{
			MyTextScroller->SuspendScrolling();

			if (MyRichTextBlock.IsValid())
			{
				MyRichTextBlock->SetOverflowPolicy(TOptional<ETextOverflowPolicy>());
			}
		}
	}
}

void UCommonRichTextBlock::ApplyUpdatedDefaultTextStyle()
{
	Super::ApplyUpdatedDefaultTextStyle();

	if (CommonUIUtils::ShouldScaleText())
	{
		ApplyTextBlockScale();
	}
}

void UCommonRichTextBlock::CreateDecorators(TArray<TSharedRef<ITextDecorator>>& OutDecorators)
{
	Super::CreateDecorators(OutDecorators);
	
	OutDecorators.Add(FCommonRichTextDecorator::CreateInstance(*this));
	OutDecorators.Add(FCommonDecorator_InlineIcon::CreateInstance(*this));
}

void UCommonRichTextBlock::RefreshOverrideStyle()
{
	if (const UCommonTextStyle* DefaultStyleClass = DefaultTextStyleOverrideClass.GetDefaultObject())
	{
		FTextBlockStyle DefaultStyle;
		DefaultStyleClass->ToTextBlockStyle(DefaultStyle);

		SetDefaultTextStyle(DefaultStyle);
	}
}

void UCommonRichTextBlock::ApplyTextBlockScale() const
{
	if (MyRichTextBlock.IsValid())
	{
		float Scale = 1.0f;
		if (CommonUIUtils::ShouldDisplayMobileUISizes())
		{
			Scale = GetMobileTextBlockScale();
		}
		else if (CommonUIUtils::UISmallTextScaling > 0.0f)
		{
			if (GetCurrentDefaultTextStyle().Font.Size <= CommonUIUtils::UISmallTextScalingThresholdMax)
			{
				Scale = CommonUIUtils::UISmallTextScaling;
			}
		}
		else
		{
			checkNoEntry();
		}
		MyRichTextBlock->SetTextBlockScale(Scale);
	}
}

