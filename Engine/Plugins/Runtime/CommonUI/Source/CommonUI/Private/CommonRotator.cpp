// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonRotator.h"
#include "CommonTextBlock.h"
#include "CommonUIPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonRotator)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UCommonRotator
UCommonRotator::UCommonRotator(const FObjectInitializer& ObjectInitializer)
{
	SetIsFocusable(true);
}

bool UCommonRotator::Initialize()
{
	if (Super::Initialize())
	{
		OnNavigation.BindUObject(this, &UCommonRotator::HandleNavigation);

		return true;
	}

	return false;
}

FNavigationReply UCommonRotator::NativeOnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent, const FNavigationReply& InDefaultReply)
{
	switch (InNavigationEvent.GetNavigationType())
	{
	case EUINavigation::Left:
	case EUINavigation::Right:
		return FNavigationReply::Custom(OnNavigation);
	default:
		return InDefaultReply;
	}
}

TSharedPtr<SWidget> UCommonRotator::HandleNavigation(EUINavigation UINavigation)
{
	if (UINavigation == EUINavigation::Left)
	{
		ShiftTextLeftInternal(true);
	}
	else if (UINavigation == EUINavigation::Right)
	{
		ShiftTextRightInternal(true);
	}

	return nullptr;
}

void UCommonRotator::PopulateTextLabels(TArray<FText> Labels)
{
	TextLabels = Labels;
	SelectedIndex = 0;

	BP_OnOptionsPopulated(TextLabels.Num());
}

FText UCommonRotator::GetSelectedText() const
{
	return MyText->GetText();
}

void UCommonRotator::SetSelectedItem(int32 InIndex)
{
	if (TextLabels.IsValidIndex(InIndex))
	{
		SelectedIndex = InIndex;
		MyText->SetText(TextLabels[SelectedIndex]);

		BP_OnOptionSelected(SelectedIndex);
	}
	else
	{
		UE_LOGF(LogCommonUI, Warning, "Trying to set CommonRotator to an out of bounds index: %i", InIndex);
	}
}

void UCommonRotator::ShiftTextLeft()
{
	ShiftTextLeftInternal(false);
}

void UCommonRotator::ShiftTextLeftInternal(bool bFromNavigation)
{
	if (IsInteractionEnabled())
	{
		if (SelectedIndex <= 0)
		{
			SelectedIndex = TextLabels.Num() - 1;
		}
		else
		{
			SelectedIndex--;
		}

		if (TextLabels.IsValidIndex(SelectedIndex))
		{
			SetSelectedItem(SelectedIndex);

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (OnRotated.IsBound())
			{
				OnRotated.Broadcast(SelectedIndex);
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (OnRotatedWithDirection.IsBound())
			{
				OnRotatedWithDirection.Broadcast(SelectedIndex, ERotatorDirection::Left);
			}

			OnRotatedEvent.Broadcast(SelectedIndex, bFromNavigation);
		}
	}
}

void UCommonRotator::ShiftTextRight()
{
	ShiftTextRightInternal(false);
}

void UCommonRotator::ShiftTextRightInternal(bool bFromNavigation)
{
	if (IsInteractionEnabled())
	{
		if (SelectedIndex >= TextLabels.Num() - 1)
		{
			SelectedIndex = 0;
		}
		else
		{
			SelectedIndex++;
		}

		if (TextLabels.IsValidIndex(SelectedIndex))
		{
			SetSelectedItem(SelectedIndex);

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (OnRotated.IsBound())
			{
				OnRotated.Broadcast(SelectedIndex);
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (OnRotatedWithDirection.IsBound())
			{
				OnRotatedWithDirection.Broadcast(SelectedIndex, ERotatorDirection::Right);
			}

			OnRotatedEvent.Broadcast(SelectedIndex, bFromNavigation);
		}
	}
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

