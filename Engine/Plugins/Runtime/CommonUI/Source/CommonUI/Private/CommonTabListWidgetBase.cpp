// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonTabListWidgetBase.h"

#include "CommonAnimatedSwitcher.h"
#include "CommonUIPrivate.h"
#include "Groups/CommonButtonGroupBase.h"
#include "Input/CommonUIInputTypes.h"
#include "CommonUITypes.h"
#include "InputAction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonTabListWidgetBase)

UCommonTabListWidgetBase::UCommonTabListWidgetBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutoListenForInput(false)
	, bDeferRebuildingTabList(false)
	, TabButtonGroup(nullptr)
	, bIsListeningForInput(false)
	, RegisteredTabsByID()
	, TabButtonWidgetPool(*this)
	, ActiveTabID(NAME_None)
	, bIsRebuildingList(false)
	, bPendingRebuild(false)
{
}
 
void UCommonTabListWidgetBase::SetLinkedSwitcher(UCommonAnimatedSwitcher* CommonSwitcher)
{
	if (LinkedSwitcher.Get() != CommonSwitcher)
	{
		HandlePreLinkedSwitcherChanged();
		LinkedSwitcher = CommonSwitcher;
		HandlePostLinkedSwitcherChanged();
	}
}

UCommonAnimatedSwitcher* UCommonTabListWidgetBase::GetLinkedSwitcher() const
{
	return LinkedSwitcher.Get();
}

bool UCommonTabListWidgetBase::RegisterTab(FName TabNameID, TSubclassOf<UCommonButtonBase> ButtonWidgetType, UWidget* ContentWidget, const int32 TabIndex /*= INDEX_NONE*/)
{
	bool bAreParametersValid = true;

	// Early out on redundant tab registration.
	if (RegisteredTabsByID.Contains(TabNameID))
	{
		bAreParametersValid = false;
		UE_LOGF(LogCommonUI, Warning, "RegisteredTabsByID already contains a tab called [%ls]", *TabNameID.ToString());
	}

	// Early out on invalid tab button type.
	if (ButtonWidgetType == nullptr)
	{
		bAreParametersValid = false;
		UE_LOGF(LogCommonUI, Warning, "RegisteredTabsByID missing ButtonWidgetType [%ls]", *TabNameID.ToString());
	}
	
	// NOTE: Adding the button to the group may change it's selection, which raises an event we listen to,
	// which can only properly be handled if we already know that this button is associated with a registered tab.
	if (TabButtonGroup == nullptr)
	{
		bAreParametersValid = false;
		UE_LOGF(LogCommonUI, Warning, "RegisteredTabsByID missing TabButtonGroup [%ls]", *TabNameID.ToString());
	}

	if (!bAreParametersValid)
	{
		return false;
	}

	// There is no PlayerController in Designer
	UCommonButtonBase* const NewTabButton = TabButtonWidgetPool.GetOrCreateInstance<UCommonButtonBase>(ButtonWidgetType);
	if (!ensureMsgf(NewTabButton, TEXT("Failed to create tab button. Aborting tab registration.")))
	{
		return false;
	}

	const int32 NumRegisteredTabs = RegisteredTabsByID.Num();
	const int32 NewTabIndex = (TabIndex == INDEX_NONE) ? NumRegisteredTabs : FMath::Clamp(TabIndex, 0, NumRegisteredTabs);

	// If the new tab is being inserted before the end of the list, we need to rebuild the tab list.
	const bool bRequiresRebuild = !IsRebuildingList() && (NewTabIndex < NumRegisteredTabs);

	if (bRequiresRebuild)
	{
		for (TPair<FName, FCommonRegisteredTabInfo>& Pair : RegisteredTabsByID)
		{
			if (NewTabIndex <= Pair.Value.TabIndex)
			{
				// Increment this tab's index as we are inserting the new tab before it.
				Pair.Value.TabIndex++;
			}
		}
	}

	// Tab book-keeping.
	FCommonRegisteredTabInfo NewTabInfo;
	NewTabInfo.TabIndex = NewTabIndex;
	NewTabInfo.TabButtonClass = ButtonWidgetType;
	NewTabInfo.TabButton = NewTabButton;
	NewTabInfo.ContentInstance = ContentWidget;
	RegisteredTabsByID.Add(TabNameID, NewTabInfo);

	// Enforce the "contract" that tab buttons require - single-selectability, but not toggleability.
	NewTabButton->SetIsSelectable(true);
	NewTabButton->SetIsToggleable(false);

	TabButtonGroup->AddWidget(NewTabButton);
	HandleTabCreation(TabNameID, NewTabInfo.TabButton);
	
	OnTabButtonCreation.Broadcast(TabNameID, NewTabInfo.TabButton);
	
	if (bRequiresRebuild)
	{
		if (bDeferRebuildingTabList)
		{
			if (!bPendingRebuild)
			{
				bPendingRebuild = true;

				FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCommonTabListWidgetBase::DeferredRebuildTabList));
			}
		}
		else
		{
			RebuildTabList();
		}
	}

	return true;
}

bool UCommonTabListWidgetBase::RemoveTab(FName TabNameID)
{
	const FCommonRegisteredTabInfo* TabInfo = RegisteredTabsByID.Find(TabNameID);

	if (!TabInfo)
	{
		return false;
	}

	if (TabInfo->TabIndex >= 0)
	{
		for (TPair<FName, FCommonRegisteredTabInfo>& RegisteredTabByID : RegisteredTabsByID)
		{
			if (RegisteredTabByID.Value.TabIndex > TabInfo->TabIndex)
			{
				// Decrement this tab's index as we have removed a tab before it.
				RegisteredTabByID.Value.TabIndex--;
			}
		}
	}

	RemoveTab_Internal(TabNameID, *TabInfo);

	return true;
}

void UCommonTabListWidgetBase::RemoveAllTabs()
{
	TArray<FName> RegisteredTabIDs;
	RegisteredTabsByID.GetKeys(RegisteredTabIDs);
	for (const FName& RegisteredTabID : RegisteredTabIDs)
	{
		// Go through the regular remove tab flow, so that we properly free up pooled widgets
		RemoveTab(RegisteredTabID);
	}
}

int32 UCommonTabListWidgetBase::GetTabCount() const
{
	return RegisteredTabsByID.Num();
}

void UCommonTabListWidgetBase::SetListeningForInput(bool bShouldListen)
{
	if (bShouldListen && !TabButtonGroup)
	{
		// If there's no tab button group, it means we haven't been constructed and we shouldn't listen to anything
		return;
	}

	if (GetUISubsystem() == nullptr)
	{
		// Shutting down
		return;
	}

	if (bShouldListen != bIsListeningForInput)
	{
		bIsListeningForInput = bShouldListen;
		UpdateBindings();
	}
}

void UCommonTabListWidgetBase::UpdateBindings()
{
	// New input system binding flow
	if (bIsListeningForInput)
	{
		bool bIsEnhancedInputSupportEnabled = CommonUI::IsEnhancedInputSupportEnabled();
		if (bIsEnhancedInputSupportEnabled && NextTabEnhancedInputAction)
		{
			NextTabActionHandle = RegisterUIActionBinding(FBindUIActionArgs(NextTabEnhancedInputAction, false, FSimpleDelegate::CreateUObject(this, &UCommonTabListWidgetBase::HandleNextTabAction)));
		}
		else
		{
			NextTabActionHandle = RegisterUIActionBinding(FBindUIActionArgs(NextTabInputActionData, false, FSimpleDelegate::CreateUObject(this, &UCommonTabListWidgetBase::HandleNextTabAction)));
		}

		if (bIsEnhancedInputSupportEnabled && PreviousTabEnhancedInputAction)
		{
			PrevTabActionHandle = RegisterUIActionBinding(FBindUIActionArgs(PreviousTabEnhancedInputAction, false, FSimpleDelegate::CreateUObject(this, &UCommonTabListWidgetBase::HandlePreviousTabAction)));
		}
		else
		{
			PrevTabActionHandle = RegisterUIActionBinding(FBindUIActionArgs(PreviousTabInputActionData, false, FSimpleDelegate::CreateUObject(this, &UCommonTabListWidgetBase::HandlePreviousTabAction)));
		}
	}
	else
	{
		NextTabActionHandle.Unregister();
		PrevTabActionHandle.Unregister();
	}
}

bool UCommonTabListWidgetBase::IsRebuildingList() const
{
	return bIsRebuildingList;
}

bool UCommonTabListWidgetBase::SelectTabByID(FName TabNameID, bool bSuppressClickFeedback)
{
	for (auto& TabPair : RegisteredTabsByID)
	{
		if (TabPair.Key == TabNameID && ensure(TabPair.Value.TabButton))
		{
			TabPair.Value.TabButton->SetIsSelected(true, !bSuppressClickFeedback);
			return true;
		}
	}

	return false;
}

FName UCommonTabListWidgetBase::GetSelectedTabId() const
{
	FName FoundId = NAME_None;

	for (auto& TabPair : RegisteredTabsByID)
	{
		if (TabPair.Value.TabButton != nullptr && TabPair.Value.TabButton->GetSelected())
		{
			FoundId = TabPair.Key;
			break;
		}
	}

	return FoundId;
}

FName UCommonTabListWidgetBase::GetTabIdAtIndex(int32 Index) const
{
	FName FoundId = NAME_None;

	if (ensure(Index < RegisteredTabsByID.Num()))
	{
		for (auto& TabPair : RegisteredTabsByID)
		{
			if (TabPair.Value.TabIndex == Index)
			{
				FoundId = TabPair.Key;
				break;
			}
		}
	}

	return FoundId;
}

void UCommonTabListWidgetBase::SetTabVisibility(FName TabNameID, ESlateVisibility NewVisibility)
{
	for (auto& TabPair : RegisteredTabsByID)
	{
		if (TabPair.Key == TabNameID && ensure(TabPair.Value.TabButton))
		{
			TabPair.Value.TabButton->SetVisibility(NewVisibility);
			
			if (NewVisibility == ESlateVisibility::Collapsed || NewVisibility == ESlateVisibility::Hidden)
			{
				TabPair.Value.TabButton->SetIsInteractionEnabled(false);
			}
			else
			{
				TabPair.Value.TabButton->SetIsInteractionEnabled(true);
			}
			
			break;
		}
	}
}

void UCommonTabListWidgetBase::SetTabEnabled(FName TabNameID, bool bEnable)
{
	for (auto& TabPair : RegisteredTabsByID)
	{
		if (TabPair.Key == TabNameID && ensure(TabPair.Value.TabButton))
		{
			if (bEnable)
			{
				TabPair.Value.TabButton->SetIsEnabled(true);
			}
			else
			{
				TabPair.Value.TabButton->SetIsEnabled(false);
			}

			break;
		}
	}
}

void UCommonTabListWidgetBase::SetTabInteractionEnabled(FName TabNameID, bool bEnable)
{
	for (auto& TabPair : RegisteredTabsByID)
	{
		if (TabPair.Key == TabNameID && ensure(TabPair.Value.TabButton))
		{
			if (bEnable)
			{
				TabPair.Value.TabButton->SetIsInteractionEnabled(true);
			}
			else
			{
				TabPair.Value.TabButton->SetIsInteractionEnabled(false);
			}

			break;
		}
	}
}

void UCommonTabListWidgetBase::DisableTabWithReason(FName TabNameID, const FText& Reason)
{
	for (auto& TabPair : RegisteredTabsByID)
	{
		if (TabPair.Key == TabNameID && ensure(TabPair.Value.TabButton))
		{
			TabPair.Value.TabButton->DisableButtonWithReason(Reason);
			break;
		}
	}
}

UCommonButtonBase* UCommonTabListWidgetBase::GetTabButtonBaseByID(FName TabNameID) const
{
	if (const FCommonRegisteredTabInfo* TabInfo = RegisteredTabsByID.Find(TabNameID))
	{
		return TabInfo->TabButton;
	}

	return nullptr;
}

bool UCommonTabListWidgetBase::HasTabContentWidget(const FName TabNameId) const
{
	const FCommonRegisteredTabInfo* FoundTabInfo = RegisteredTabsByID.Find(TabNameId);
	return FoundTabInfo && FoundTabInfo->ContentInstance != nullptr;
}

bool UCommonTabListWidgetBase::RegisterTabContentWidget(const FName TabNameId, UWidget* ContentWidget)
{
	if (!ensure(ContentWidget))
	{
		return false;
	}

	FCommonRegisteredTabInfo* FoundTabInfo = RegisteredTabsByID.Find(TabNameId);
	if (!ensure(FoundTabInfo))
	{
		return false;
	}

	UWidget* OldContentWidget = FoundTabInfo->ContentInstance;
	FoundTabInfo->ContentInstance = ContentWidget;

	if (UCommonAnimatedSwitcher* Switcher = LinkedSwitcher.Get())
	{
		// Remove Old Widget if it exists
		if (OldContentWidget != nullptr)
		{
			Switcher->RemoveChild(OldContentWidget);
		}

		// Add the new widget
		Switcher->AddChild(ContentWidget);

		// If this tab is selected we need to set it as the active widget
		if (TabNameId == GetSelectedTabId())
		{
			Switcher->SetActiveWidget(ContentWidget);
		}
	}

	return true;
}

void UCommonTabListWidgetBase::SetSelectionRequired(bool bSelectionRequired)
{
	if (TabButtonGroup)
	{
		TabButtonGroup->SetSelectionRequired(bSelectionRequired);
	}
}

void UCommonTabListWidgetBase::NativeOnInitialized()
{
	Super::NativeOnInitialized();

	// Create the button group once up-front
	TabButtonGroup = NewObject<UCommonButtonGroupBase>(this);
	SetSelectionRequired(true);
	TabButtonGroup->OnSelectedButtonBaseChanged.AddDynamic(this, &UCommonTabListWidgetBase::HandleTabButtonSelected);
}

void UCommonTabListWidgetBase::NativeConstruct()
{
	Super::NativeConstruct();
	
	if (bAutoListenForInput)
	{
		SetListeningForInput(true);
	}
}

void UCommonTabListWidgetBase::NativeDestruct()
{
	Super::NativeDestruct();

	SetListeningForInput(false);

	ActiveTabID = NAME_None;

	// Suppress selection when tearing down tabs
	SetSelectionRequired(false);
	RemoveAllTabs();
	SetSelectionRequired(true);

	if (TabButtonGroup)
	{
		TabButtonGroup->RemoveAll();
	}
}

void UCommonTabListWidgetBase::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	TabButtonWidgetPool.ResetPool();
}

void UCommonTabListWidgetBase::HandlePreLinkedSwitcherChanged()
{
	HandlePreLinkedSwitcherChanged_BP();
}

void UCommonTabListWidgetBase::HandlePostLinkedSwitcherChanged()
{
	HandlePostLinkedSwitcherChanged_BP();
}

void UCommonTabListWidgetBase::HandleTabCreation_Implementation(FName TabNameID, UCommonButtonBase* TabButton)
{
}

void UCommonTabListWidgetBase::HandleTabRemoval_Implementation(FName TabNameID, UCommonButtonBase* TabButton)
{
}

const TMap<FName, FCommonRegisteredTabInfo>& UCommonTabListWidgetBase::GetRegisteredTabsByID() const
{
	return RegisteredTabsByID;
}

void UCommonTabListWidgetBase::HandleTabButtonSelected(UCommonButtonBase* SelectedTabButton, int32 ButtonIndex)
{
	for (auto& TabPair : RegisteredTabsByID)
	{
		FCommonRegisteredTabInfo& TabInfo = TabPair.Value;
			
		if (TabInfo.TabButton == SelectedTabButton)
		{
			ActiveTabID = TabPair.Key;

			if (TabInfo.ContentInstance && LinkedSwitcher.IsValid())
			{
				// There's already an instance of the widget to display, so go for it
				LinkedSwitcher->SetActiveWidget(TabInfo.ContentInstance);
			}

			OnTabSelected.Broadcast(TabPair.Key);
		}
	}
}

void UCommonTabListWidgetBase::HandleNextTabInputAction(bool& bPassThrough)
{
	HandleNextTabAction();
}

void UCommonTabListWidgetBase::HandleNextTabAction()
{
	if (ensure(TabButtonGroup))
	{
		TabButtonGroup->SelectNextButton(bShouldWrapNavigation);
	}
}

void UCommonTabListWidgetBase::HandlePreviousTabInputAction(bool& bPassThrough)
{
	HandlePreviousTabAction();
}

void UCommonTabListWidgetBase::HandlePreviousTabAction()
{
	if (ensure(TabButtonGroup))
	{
		TabButtonGroup->SelectPreviousButton(bShouldWrapNavigation);
	}
}

bool UCommonTabListWidgetBase::DeferredRebuildTabList(float DeltaTime)
{
	bPendingRebuild = false;
	RebuildTabList();
	return false;
}

void UCommonTabListWidgetBase::RebuildTabList()
{
	// Mark that we're currently rebuilding the tab list
	bIsRebuildingList = true;

	// Cache the registered tabs, as we are about to clear them with RemoveAllTabs()
	TMap<FName, FCommonRegisteredTabInfo> CachedRegisteredTabsByID = RegisteredTabsByID;

	// Keep track of the current ActiveTabID so we can restore it after the list is rebuilt.
	const FName CachedActiveTabID = ActiveTabID;

	// Disable selection required temporarily so we can deselect everything, rebuild the list, then select the tab we want.
	SetSelectionRequired(false);
	TabButtonGroup->DeselectAll();

	// Clear all tabs, releasing their widgets back to the widget pool
	RemoveAllTabs();

	// Re-Register tabs using CachedRegisteredTabsByID
	for (const TPair<FName, FCommonRegisteredTabInfo>& Pair : CachedRegisteredTabsByID)
	{
		const FName& TabID = Pair.Key;
		const FCommonRegisteredTabInfo& TabInfo = Pair.Value;
		RegisterTab(
			TabID, 
			TabInfo.TabButtonClass, 
			TabInfo.ContentInstance, 
			TabInfo.TabIndex
		);
	}

	// Done rebuilding our tab list
	bIsRebuildingList = false;

	// re-select the previously active tab
	constexpr bool bSuppressClickFeedback = true;
	SelectTabByID(CachedActiveTabID, bSuppressClickFeedback);

	// Turn back on selection requirement
	SetSelectionRequired(true);
	
	// Broadcast our rebuilt delegate
	OnTabListRebuilt.Broadcast();
}

void UCommonTabListWidgetBase::RemoveTab_Internal(const FName TabNameID, const FCommonRegisteredTabInfo& TabInfo)
{
	UCommonButtonBase* const TabButton = TabInfo.TabButton;

	if (TabButton)
	{
		TabButtonGroup->RemoveWidget(TabButton);
		TabButton->RemoveFromParent();
		TabButtonWidgetPool.Release(TabButton);
	}

	RegisteredTabsByID.Remove(TabNameID);

	// Callbacks
	HandleTabRemoval(TabNameID, TabButton);
	OnTabButtonRemoval.Broadcast(TabNameID, TabButton);
}
