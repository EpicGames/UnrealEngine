// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonTextBlock.h"
#include "CommonUIEditorSettings.h"
#include "CommonUIUtils.h"
#include "CommonWidgetPaletteCategories.h"
#include "ICommonUIModule.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/STextScroller.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonTextBlock)

FTextScrollerOptions UCommonTextScrollStyle::ToScrollOptions() const
{
	FTextScrollerOptions ScrollOptions;
	ScrollOptions.Speed = Speed;
	ScrollOptions.StartDelay = StartDelay;
	ScrollOptions.EndDelay = EndDelay;
	ScrollOptions.FadeInDelay = FadeInDelay;
	ScrollOptions.FadeOutDelay = FadeOutDelay;
	return ScrollOptions;
}

// UCommonTextStyle
////////////////////////////////////////////////////////////////////////////////////

UCommonTextStyle::UCommonTextStyle()
	: Color(FLinearColor::Black)
	, LineHeightPercentage(1.0f)
	, ApplyLineHeightToBottomLine(true)
{
}

void UCommonTextStyle::GetFont(FSlateFontInfo& OutFont) const
{
	OutFont = Font;
}

void UCommonTextStyle::GetColor(FLinearColor& OutColor) const
{
	OutColor = Color;
}

void UCommonTextStyle::GetMargin(FMargin& OutMargin) const
{
	OutMargin = Margin;
}

float UCommonTextStyle::GetLineHeightPercentage() const
{
	return LineHeightPercentage;
}

bool UCommonTextStyle::GetApplyLineHeightToBottomLine() const
{
	return ApplyLineHeightToBottomLine;
}

void UCommonTextStyle::GetShadowOffset(FVector2D& OutShadowOffset) const
{
	OutShadowOffset = ShadowOffset;
}

void UCommonTextStyle::GetShadowColor(FLinearColor& OutColor) const
{
	OutColor = ShadowColor;
}

void UCommonTextStyle::GetStrikeBrush(FSlateBrush& OutStrikeBrush) const
{
	OutStrikeBrush = StrikeBrush;
}

void UCommonTextStyle::ToTextBlockStyle(FTextBlockStyle& OutTextBlockStyle) const
{
	OutTextBlockStyle.SetFont(Font).SetColorAndOpacity(Color).SetStrikeBrush(StrikeBrush);
	if (bUsesDropShadow)
	{
		OutTextBlockStyle.SetShadowOffset(ShadowOffset).SetShadowColorAndOpacity(ShadowColor);
	}
}

void UCommonTextStyle::ApplyToTextBlock(const TSharedRef<STextBlock>& TextBlock) const
{
	TextBlock->SetFont(Font);
	TextBlock->SetStrikeBrush(&StrikeBrush);
	TextBlock->SetMargin(Margin);
	TextBlock->SetLineHeightPercentage(LineHeightPercentage);
	TextBlock->SetApplyLineHeightToBottomLine(ApplyLineHeightToBottomLine);
	TextBlock->SetColorAndOpacity(Color);
	if (bUsesDropShadow)
	{
		TextBlock->SetShadowOffset(ShadowOffset);
		TextBlock->SetShadowColorAndOpacity(ShadowColor);
	}
	else
	{
		TextBlock->SetShadowOffset(FVector2D::ZeroVector);
		TextBlock->SetShadowColorAndOpacity(FLinearColor::Transparent);
	}
}

// UCommonTextBlock
////////////////////////////////////////////////////////////////////////////////////

UCommonTextBlock::UCommonTextBlock(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
	SetClipping(EWidgetClipping::Inherit);
}

void UCommonTextBlock::PostInitProperties()
{
	Super::PostInitProperties();

	UpdateFromStyle();
}

void UCommonTextBlock::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// We will remove this once existing content is fixed up. Since previously the native CDO was actually the default style, this code will attempt to set the style on assets that were once using this default
	if (!Style && !bStyleNoLongerNeedsConversion && !IsRunningDedicatedServer())
	{
		UCommonUIEditorSettings& Settings = ICommonUIModule::GetEditorSettings();
		Settings.ConditionalPostLoad();
		TSubclassOf<UCommonTextStyle> DefaultStyle = Settings.GetTemplateTextStyle();
		UCommonTextStyle* DefaultStyleCDO = DefaultStyle ? Cast<UCommonTextStyle>(DefaultStyle->GetDefaultObject(false)) : nullptr;
		if (DefaultStyleCDO)
		{
			DefaultStyleCDO->ConditionalPostLoad();
			UCommonTextBlock* CDO = CastChecked<UCommonTextBlock>(GetClass()->GetDefaultObject());
			bool bAllDefaults = true;

			// Font
			{
				bool bFontHasChanged = false;
				FSlateFontInfo MyFont = GetFont();
				const FSlateFontInfo& CDOFont = CDO->GetFont();
				if (MyFont.FontObject == CDOFont.FontObject)
				{
					MyFont.FontObject = DefaultStyleCDO->Font.FontObject;
					bFontHasChanged = true;
				}
				if (MyFont.FontMaterial == CDOFont.FontMaterial)
				{
					MyFont.FontMaterial = DefaultStyleCDO->Font.FontMaterial;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.OutlineSize == CDOFont.OutlineSettings.OutlineSize)
				{
					MyFont.OutlineSettings.OutlineSize = DefaultStyleCDO->Font.OutlineSettings.OutlineSize;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.bSeparateFillAlpha == CDOFont.OutlineSettings.bSeparateFillAlpha)
				{
					MyFont.OutlineSettings.bSeparateFillAlpha = DefaultStyleCDO->Font.OutlineSettings.bSeparateFillAlpha;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.bApplyOutlineToDropShadows == CDOFont.OutlineSettings.bApplyOutlineToDropShadows)
				{
					MyFont.OutlineSettings.bApplyOutlineToDropShadows = DefaultStyleCDO->Font.OutlineSettings.bApplyOutlineToDropShadows;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.OutlineMaterial == CDOFont.OutlineSettings.OutlineMaterial)
				{
					MyFont.OutlineSettings.OutlineMaterial = DefaultStyleCDO->Font.OutlineSettings.OutlineMaterial;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.OutlineColor.R == CDOFont.OutlineSettings.OutlineColor.R)
				{
					MyFont.OutlineSettings.OutlineColor.R = DefaultStyleCDO->Font.OutlineSettings.OutlineColor.R;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.OutlineColor.G == CDOFont.OutlineSettings.OutlineColor.G)
				{
					MyFont.OutlineSettings.OutlineColor.G = DefaultStyleCDO->Font.OutlineSettings.OutlineColor.G;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.OutlineColor.B == CDOFont.OutlineSettings.OutlineColor.B)
				{
					MyFont.OutlineSettings.OutlineColor.B = DefaultStyleCDO->Font.OutlineSettings.OutlineColor.B;
					bFontHasChanged = true;
				}
				if (MyFont.OutlineSettings.OutlineColor.A == CDOFont.OutlineSettings.OutlineColor.A)
				{
					MyFont.OutlineSettings.OutlineColor.A = DefaultStyleCDO->Font.OutlineSettings.OutlineColor.A;
					bFontHasChanged = true;
				}
				if (MyFont.TypefaceFontName == CDOFont.TypefaceFontName)
				{
					MyFont.TypefaceFontName = DefaultStyleCDO->Font.TypefaceFontName;
					bFontHasChanged = true;
				}
				if (MyFont.Size == CDOFont.Size)
				{
					MyFont.Size = DefaultStyleCDO->Font.Size;
					bFontHasChanged = true;
				}

				if (bFontHasChanged)
				{
					SetFont(MyFont);
				}
				bAllDefaults = bAllDefaults && !bFontHasChanged;
			}

			// Margin
			if (Margin.Left == CDO->Margin.Left) { Margin.Left = DefaultStyleCDO->Margin.Left; } else { bAllDefaults = false; }
			if (Margin.Top == CDO->Margin.Top) { Margin.Top = DefaultStyleCDO->Margin.Top; } else { bAllDefaults = false; }
			if (Margin.Right == CDO->Margin.Right) { Margin.Right = DefaultStyleCDO->Margin.Right; } else { bAllDefaults = false; }
			if (Margin.Bottom == CDO->Margin.Bottom) { Margin.Bottom = DefaultStyleCDO->Margin.Bottom; } else { bAllDefaults = false; }

			// LineHeightPercentage
			if (LineHeightPercentage == CDO->LineHeightPercentage) { LineHeightPercentage = DefaultStyleCDO->LineHeightPercentage; } else { bAllDefaults = false; }

			// ApplyLineHeightToBottomLine
			if (ApplyLineHeightToBottomLine == CDO->ApplyLineHeightToBottomLine) { ApplyLineHeightToBottomLine = DefaultStyleCDO->ApplyLineHeightToBottomLine; } else { bAllDefaults = false; }

			// ColorAndOpacity
			if (GetColorAndOpacity() == CDO->GetColorAndOpacity()) { SetColorAndOpacity(DefaultStyleCDO->Color); } else { bAllDefaults = false; }

			// ShadowOffset
			{
				bool bShadowOffsetHasChanged = false;
				FVector2D MyShadowOffset = GetShadowOffset();
				const FVector2D CDOShadowOffset = CDO->GetShadowOffset();
				if (MyShadowOffset.X == CDOShadowOffset.X)
				{
					DefaultStyleCDO->bUsesDropShadow ? MyShadowOffset.X = DefaultStyleCDO->ShadowOffset.X : 0.f;
					bShadowOffsetHasChanged = true;
				}
				if (MyShadowOffset.Y == CDOShadowOffset.Y)
				{
					DefaultStyleCDO->bUsesDropShadow ? MyShadowOffset.Y = DefaultStyleCDO->ShadowOffset.Y : 0.f;
					bShadowOffsetHasChanged = true;
				}

				if (bShadowOffsetHasChanged)
				{
					SetShadowOffset(MyShadowOffset);
				}
				bAllDefaults = bAllDefaults && !bShadowOffsetHasChanged;
			}

			// ShadowColorAndOpacity
			{
				bool bShadowColorAndOpacityHasChanged = false;
				FLinearColor MyShadowOffset = GetShadowColorAndOpacity();
				const FLinearColor CDOShadowOffset = CDO->GetShadowColorAndOpacity();
				if (MyShadowOffset.R == CDOShadowOffset.R)
				{
					DefaultStyleCDO->bUsesDropShadow ? MyShadowOffset.R = DefaultStyleCDO->ShadowColor.R : 0.f;
					bShadowColorAndOpacityHasChanged = true;
				}
				if (MyShadowOffset.G == CDOShadowOffset.G)
				{
					DefaultStyleCDO->bUsesDropShadow ? MyShadowOffset.G = DefaultStyleCDO->ShadowColor.G : 0.f;
					bShadowColorAndOpacityHasChanged = true;
				}
				if (MyShadowOffset.B == CDOShadowOffset.B)
				{
					DefaultStyleCDO->bUsesDropShadow ? MyShadowOffset.B = DefaultStyleCDO->ShadowColor.B : 0.f;
					bShadowColorAndOpacityHasChanged = true;
				}
				if (MyShadowOffset.A == CDOShadowOffset.A)
				{
					DefaultStyleCDO->bUsesDropShadow ? MyShadowOffset.A = DefaultStyleCDO->ShadowColor.A : 0.f;
					bShadowColorAndOpacityHasChanged = true;
				}

				if (bShadowColorAndOpacityHasChanged)
				{
					SetShadowColorAndOpacity(MyShadowOffset);
				}
				bAllDefaults = bAllDefaults && !bShadowColorAndOpacityHasChanged;
			}

			if (bAllDefaults)
			{
				Style = Settings.GetTemplateTextStyle();
			}
		}
	}

	bStyleNoLongerNeedsConversion = true;
#endif

	UpdateFromStyle();
}

void UCommonTextBlock::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	// @FIXME: If a text style is set, clear style related properties on PreSave when cooking to avoid 
	// non-deterministic cooking issues. The root cause of this is still unclear, but we're sometimes 
	// getting either the wrong style CDO (a parent BP instead of the child BP) or the child BP CDO properties
	// are being initialized with the parent's values.
	if (Style && Ar.IsSaving())
	{
		FSlateFontInfo TempFont;
		FSlateBrush TempStrikeBrush;
		FMargin TempMargin;
		float TempLineHeightPercentage = 1.f;
		bool TempApplyLineHeightToBottomLine = true;
		FSlateColor TempColorAndOpacity;
		FVector2D TempShadowOffset = FVector2D::ZeroVector;
		FLinearColor TempShadowColorAndOpacity = FLinearColor::Transparent;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Swap(Font, TempFont);
		Swap(StrikeBrush, TempStrikeBrush);
		Swap(Margin, TempMargin);
		Swap(LineHeightPercentage, TempLineHeightPercentage);
		Swap(ApplyLineHeightToBottomLine, TempApplyLineHeightToBottomLine);
		Swap(ColorAndOpacity, TempColorAndOpacity);
		Swap(ShadowOffset, TempShadowOffset);
		Swap(ShadowColorAndOpacity, TempShadowColorAndOpacity);

		Super::Serialize(Ar);

		Swap(TempFont, Font);
		Swap(TempStrikeBrush, StrikeBrush);
		Swap(TempMargin, Margin);
		Swap(TempLineHeightPercentage, LineHeightPercentage);
		Swap(TempApplyLineHeightToBottomLine, ApplyLineHeightToBottomLine);
		Swap(TempColorAndOpacity, ColorAndOpacity);
		Swap(TempShadowOffset, ShadowOffset);
		Swap(TempShadowColorAndOpacity, ShadowColorAndOpacity);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
#endif //if WITH_EDITORONLY_DATA
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bDisplayAllCaps_DEPRECATED)
		{
			SetTextTransformPolicy(ETextTransformPolicy::ToUpper);
			bDisplayAllCaps_DEPRECATED = false;
		}
	}
}

void UCommonTextBlock::SetWrapTextWidth(int32 InWrapTextAt)
{
	SetWrapTextAt(InWrapTextAt);
}

void UCommonTextBlock::SetTextCase(bool bUseAllCaps)
{
	SetTextTransformPolicy(bUseAllCaps ? ETextTransformPolicy::ToUpper : ETextTransformPolicy::None);
	SynchronizeProperties();
}

void UCommonTextBlock::SetLineHeightPercentage(float InLineHeightPercentage)
{
	UTextLayoutWidget::SetLineHeightPercentage(InLineHeightPercentage);
}

void UCommonTextBlock::SetApplyLineHeightToBottomLine(bool InApplyLineHeightToBottomLine)
{
	UTextLayoutWidget::SetApplyLineHeightToBottomLine(InApplyLineHeightToBottomLine);
}

void UCommonTextBlock::SetStyle(TSubclassOf<UCommonTextStyle> InStyle)
{
	Style = InStyle;
	SynchronizeProperties();
}

void UCommonTextBlock::SetScrollOrientation(TEnumAsByte<EOrientation> InScrollOrientation)
{
	ScrollOrientation = InScrollOrientation;
	SynchronizeProperties();
}

const FMargin& UCommonTextBlock::GetMargin()
{
	return Margin;
}

void UCommonTextBlock::SetMargin(const FMargin& InMargin)
{
	UTextLayoutWidget::SetMargin(InMargin);
}

void UCommonTextBlock::ResetScrollState()
{
	if (TextScroller.IsValid())
	{
		TextScroller->ResetScrollState();
	}
}

void UCommonTextBlock::SetScrollingEnabled(bool bInIsScrollingEnabled)
{
	bIsScrollingEnabled = bInIsScrollingEnabled;
	SynchronizeProperties();
}

float UCommonTextBlock::GetMobileFontSizeMultiplier() const
{
	return MobileFontSizeMultiplier;
}

void UCommonTextBlock::SetMobileFontSizeMultiplier(float InMobileFontSizeMultiplier)
{
	if (MobileFontSizeMultiplier != InMobileFontSizeMultiplier)
	{
		MobileFontSizeMultiplier = InMobileFontSizeMultiplier;

		if (CommonUIUtils::ShouldScaleText())
		{
			ApplyFontSizeMultiplier();
		}
	}
}

void UCommonTextBlock::SynchronizeProperties()
{
	UpdateFromStyle();

	Super::SynchronizeProperties();

	if (bAutoCollapseWithEmptyText)
	{
		if (IsDesignTime())
		{
			SetVisibility(GetText().IsEmpty() ? ESlateVisibility::Collapsed : ESlateVisibility::Visible);
		}
		else
		{
			SetVisibility(GetText().IsEmpty() ? ESlateVisibility::Collapsed : ESlateVisibility::SelfHitTestInvisible);
		}
	}

	if (CommonUIUtils::ShouldScaleText())
	{
		ApplyFontSizeMultiplier();
	}

	if (TextScroller.IsValid())
	{
		if (bIsScrollingEnabled)
		{
			TextScroller->StartScrolling();

			// Hide ellipsis when scrolling
			if (MyTextBlock.IsValid())
			{
				MyTextBlock->SetOverflowPolicy(ETextOverflowPolicy::Clip);
			}
		}
		else
		{
			TextScroller->SuspendScrolling();
		}
	}
}

void UCommonTextBlock::OnTextChanged()
{
	Super::OnTextChanged();
	if (bAutoCollapseWithEmptyText)
	{
		if (IsDesignTime())
		{
			SetVisibility(GetText().IsEmpty() ? ESlateVisibility::Collapsed : ESlateVisibility::Visible);
		}
		else
		{
			SetVisibility(GetText().IsEmpty() ? ESlateVisibility::Collapsed : ESlateVisibility::SelfHitTestInvisible);
		}
	}
}

void UCommonTextBlock::OnFontChanged()
{
	Super::OnFontChanged();
	if (CommonUIUtils::ShouldScaleText())
	{
		ApplyFontSizeMultiplier();
	}
}

void UCommonTextBlock::UpdateFromStyle()
{
	if (const UCommonTextStyle* TextStyle = GetStyleCDO())
	{
		SetFont(TextStyle->Font);
		SetStrikeBrush(TextStyle->StrikeBrush);
		Margin = TextStyle->Margin;
		LineHeightPercentage = TextStyle->LineHeightPercentage;
		ApplyLineHeightToBottomLine = TextStyle->ApplyLineHeightToBottomLine;
		SetColorAndOpacity(TextStyle->Color);

		if (TextStyle->bUsesDropShadow)
		{
			SetShadowOffset(TextStyle->ShadowOffset);
			SetShadowColorAndOpacity(TextStyle->ShadowColor);
		}
		else
		{
			SetShadowOffset(FVector2D::ZeroVector);
			SetShadowColorAndOpacity(FLinearColor::Transparent);
		}
	}
}

void UCommonTextBlock::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	TextScroller.Reset();
}

#if WITH_EDITOR
void UCommonTextBlock::OnCreationFromPalette()
{
	bStyleNoLongerNeedsConversion = true;
	if (!Style)
	{
		Style = ICommonUIModule::GetEditorSettings().GetTemplateTextStyle();
		UpdateFromStyle();
	}
	Super::OnCreationFromPalette();
}

const FText UCommonTextBlock::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

bool UCommonTextBlock::CanEditChange(const FProperty* InProperty) const
{
	if (Super::CanEditChange(InProperty))
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bool bIsInvalidProperty = false;
		
		if (const UCommonTextStyle* TextStyle = GetStyleCDO())
		{
			static TArray<FName> InvalidPropertiesWithStyle =
			{
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, Font),
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, StrikeBrush),
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, Margin),
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, LineHeightPercentage),
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, ApplyLineHeightToBottomLine),
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, ColorAndOpacity),
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, ShadowOffset),
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, ShadowColorAndOpacity)
			};

			bIsInvalidProperty |= InvalidPropertiesWithStyle.Contains(InProperty->GetFName());
		}
		
		if (const UCommonTextScrollStyle* TextScrollStyle = UCommonTextBlock::GetScrollStyleCDO())
		{
			static TArray<FName> InvalidPropertiesWithScrollStyle =
			{
				GET_MEMBER_NAME_CHECKED(UCommonTextBlock, Clipping)
			};
			
			bIsInvalidProperty |= InvalidPropertiesWithScrollStyle.Contains(InProperty->GetFName());
		}

		if (bIsInvalidProperty)
		{
			return false;
		}

		if (bAutoCollapseWithEmptyText && InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCommonTextBlock, Visibility))
		{
			return false;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return true;
	}

	return false;
}

#endif // WITH_EDITOR

TSharedRef<SWidget> UCommonTextBlock::RebuildWidget()
{
	const UCommonTextScrollStyle* TextScrollStyle = UCommonTextBlock::GetScrollStyleCDO();
	if (!TextScrollStyle)
	{
		return Super::RebuildWidget();
	}

	SetClipping(TextScrollStyle->Clipping);

	// clang-format off
	TextScroller = 
		SNew(STextScroller)
		.ScrollOptions(TextScrollStyle->ToScrollOptions())
		.ScrollOrientation(ScrollOrientation)
		[ 
			Super::RebuildWidget() 
		];
	// clang-format on

	// Set the inner text block to self hit test invisible
	MyTextBlock->SetVisibility(EVisibility::SelfHitTestInvisible);

#if WIDGET_INCLUDE_RELFECTION_METADATA
	MyTextBlock->AddMetadata<FReflectionMetaData>(MakeShared<FReflectionMetaData>(GetFName(), GetClass(), this, GetSourceAssetOrClass()));
#endif

	return TextScroller.ToSharedRef();
}

const UCommonTextStyle* UCommonTextBlock::GetStyleCDO() const
{
	return Style ? Cast<UCommonTextStyle>(Style->GetDefaultObject(false)) : nullptr;
}

const UCommonTextScrollStyle* UCommonTextBlock::GetScrollStyleCDO() const
{
	return ScrollStyle ? Cast<UCommonTextScrollStyle>(ScrollStyle->GetDefaultObject(false)) : nullptr;
}

void UCommonTextBlock::ApplyFontSizeMultiplier() const
{
	if (MyTextBlock.IsValid())
	{
		FSlateFontInfo FontInfo = GetFont();
		if (CommonUIUtils::ShouldDisplayMobileUISizes())
		{
			FontInfo.Size *= GetMobileFontSizeMultiplier();
		}
		else if (CommonUIUtils::UISmallTextScaling > 0.0f)
		{
			if (FontInfo.Size <= CommonUIUtils::UISmallTextScalingThresholdMax)
			{
				FontInfo.Size *= CommonUIUtils::UISmallTextScaling;
			}
		}
		else
		{
			checkNoEntry();
		}
		MyTextBlock->SetFont(FontInfo);
	}
}

