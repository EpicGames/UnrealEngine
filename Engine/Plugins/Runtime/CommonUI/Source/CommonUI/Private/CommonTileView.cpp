// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonTileView.h"
#include "Components/ListView.h"
#include "SCommonButtonTableRow.h"
#include "SCommonTileView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonTileView)

UCommonTileView::UCommonTileView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEnableScrollAnimation = true;
}

TSharedRef<STableViewBase> UCommonTileView::RebuildListWidget()
{
	return ConstructTileView<SCommonTileView>();
}

UUserWidget& UCommonTileView::OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (DesiredEntryClass->IsChildOf<UCommonButtonBase>())
	{
		return GenerateTypedEntry<UUserWidget, SCommonButtonTableRow<UObject*>>(DesiredEntryClass, OwnerTable);
	}
	return GenerateTypedEntry(DesiredEntryClass, OwnerTable);
}
