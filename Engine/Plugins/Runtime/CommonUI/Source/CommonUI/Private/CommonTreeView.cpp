// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonTreeView.h"
#include "Components/ListView.h"
#include "SCommonButtonTableRow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonTreeView)

UCommonTreeView::UCommonTreeView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)	
{
	bEnableScrollAnimation = true;
}

TSharedRef<STableViewBase> UCommonTreeView::RebuildListWidget()
{
	return ConstructTreeView<SCommonTreeView>();
}

UUserWidget& UCommonTreeView::OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (DesiredEntryClass->IsChildOf<UCommonButtonBase>())
	{
		return GenerateTypedEntry<UUserWidget, SCommonButtonTableRow<UObject*>>(DesiredEntryClass, OwnerTable);
	}
	return GenerateTypedEntry(DesiredEntryClass, OwnerTable);
}
