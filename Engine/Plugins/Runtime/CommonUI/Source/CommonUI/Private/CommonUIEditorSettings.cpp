// Copyright Epic Games, Inc. All Rights Reserved.


#include "CommonUIEditorSettings.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUIEditorSettings)

UCommonUIEditorSettings::UCommonUIEditorSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, bDefaultDataLoaded(false)
{}

#if WITH_EDITOR
void UCommonUIEditorSettings::LoadData()
{
	LoadEditorData();
}

void UCommonUIEditorSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bDefaultDataLoaded = false;
	LoadData();
}

void UCommonUIEditorSettings::LoadEditorData()
{
	if (!bDefaultDataLoaded)
	{

		TemplateTextStyleClass = TemplateTextStyle.LoadSynchronous();
		TemplateButtonStyleClass = TemplateButtonStyle.LoadSynchronous();
		TemplateBorderStyleClass = TemplateBorderStyle.LoadSynchronous();

		if (GUObjectArray.IsDisregardForGC(this))
		{
			if (TemplateTextStyleClass)
			{
				TemplateTextStyleClass->AddToRoot();
			}
			if (TemplateButtonStyleClass)
			{
				TemplateButtonStyleClass->AddToRoot();
			}
			if (TemplateBorderStyleClass)
			{
				TemplateBorderStyleClass->AddToRoot();
			}
		}

		bDefaultDataLoaded = true;
	}
}

const TSubclassOf<UCommonTextStyle>& UCommonUIEditorSettings::GetTemplateTextStyle() const
{
	ensure(bDefaultDataLoaded);

	return TemplateTextStyleClass;
}

const TSubclassOf<UCommonButtonStyle>& UCommonUIEditorSettings::GetTemplateButtonStyle() const
{
	ensure(bDefaultDataLoaded);

	return TemplateButtonStyleClass;
}

const TSubclassOf<UCommonBorderStyle>& UCommonUIEditorSettings::GetTemplateBorderStyle() const
{
	ensure(bDefaultDataLoaded);

	return TemplateBorderStyleClass;
}
#endif
