// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUILibrary.h"

#include "Blueprint/WidgetTree.h"
#include "CommonActivatableWidget.h"
#include "CommonUIPrivate.h"
#include "Components/PanelWidget.h"
#include "Input/CommonUIActionRouterBase.h"
#include "Input/UIActionRouterTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUILibrary)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UCommonUILibrary

UCommonUILibrary::UCommonUILibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UCommonUILibrary::RefreshFocusIfLeafmostDescendant(UWidget* ContextWidget)
{
	if (!ContextWidget)
	{
		UE_LOGF(LogCommonUI, Warning, "RefreshFocusIfLeafmostDescendant was called on/from a null widget.");
		return false;
	}
	
	if(UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*ContextWidget))
	{
		// Ensure that the widget is actually hosted by the current leaf-most activatable node
		if (UCommonActivatableWidget* ActivatableWidget = ActionRouter->GetLeafmostActivatableWidget())
		{
			// Ensure that the widget is actually hosted and that it is a descendant of the leaf-most activatable widget
			if (ActionRouter->IsWidgetInLeafmostNodeHierarchy(*ContextWidget))
			{
				ActivatableWidget->RequestRefreshFocus();
				return true;
			}
			else
			{
				UE_LOGF(LogCommonUI, Warning, "RefreshFocusIfLeafmostDescendant was called on/from a widget(%ls) that was not a descendant of the leaf-most widget.", *ContextWidget->GetName());
			}
		}
	}
	
	return false;
}

UWidget* UCommonUILibrary::FindParentWidgetOfType(UWidget* StartingWidget, TSubclassOf<UWidget> Type)
{
	while ( StartingWidget )
	{
		UWidget* LocalRoot = StartingWidget;
		UWidget* LocalParent = LocalRoot->GetParent();
		while (LocalParent)
		{
			if (LocalParent->IsA(Type))
			{
				return LocalParent;
			}
			LocalRoot = LocalParent;
			LocalParent = LocalParent->GetParent();
		}

		UWidgetTree* WidgetTree = Cast<UWidgetTree>(LocalRoot->GetOuter());
		if ( WidgetTree == nullptr )
		{
			break;
		}

		StartingWidget = Cast<UUserWidget>(WidgetTree->GetOuter());
		if ( StartingWidget && StartingWidget->IsA(Type) )
		{
			return StartingWidget;
		}
	}

	return nullptr;
}

UWidget* UCommonUILibrary::FindParentWidgetImplementingInterface(UWidget* StartingWidget, TSubclassOf<UInterface> Interface)
{
	while (StartingWidget)
	{
		UWidget* LocalRoot = StartingWidget;
		UWidget* LocalParent = LocalRoot->GetParent();
		while (LocalParent)
		{
			if (UClass* LocalParentClass = LocalParent->GetClass())
			{
				if (LocalParentClass->ImplementsInterface(Interface))
				{
					return LocalParent;
				}
			}
			LocalRoot = LocalParent;
			LocalParent = LocalParent->GetParent();
		}

		UWidgetTree* WidgetTree = Cast<UWidgetTree>(LocalRoot->GetOuter());
		if (WidgetTree == nullptr)
		{
			break;
		}

		StartingWidget = Cast<UUserWidget>(WidgetTree->GetOuter());
		if (StartingWidget)
		{
			if (UClass* StartingWidgetClass = StartingWidget->GetClass())
			{
				if (StartingWidgetClass->ImplementsInterface(Interface))
				{
					return StartingWidget;
				}
			}
		}
	}

	return nullptr;
}

FGameplayTag UCommonUILibrary::Conv_UITagToGameplayTag(FUITag InValue)
{
	return InValue;
}

FGameplayTag UCommonUILibrary::Conv_UIActionTagToGameplayTag(FUIActionTag InValue)
{
	return InValue;
}

#undef LOCTEXT_NAMESPACE

