// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUIPrivate.h"
#include "ICommonUIModule.h"
#include "CommonUISettings.h"
#include "CommonUIEditorSettings.h"
#include "Engine/AssetManager.h"

#include "Input/CommonUIInputSettings.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#endif // WITH_EDITOR

/**
 * Implements the FCommonUIModule module.
 */
class FCommonUIModule
	: public ICommonUIModule
{
public:
	FCommonUIModule();
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual FStreamableManager& GetStreamableManager() const override;

	virtual TAsyncLoadPriority GetLazyLoadPriority() const override;

protected:
	virtual UCommonUISettings* GetSettingsInstance() const override;

#if WITH_EDITOR
	virtual UCommonUIEditorSettings* GetEditorSettingsInstance() const override;
#endif

private:
	void CreateUISettings();

	UCommonUISettings* CommonUISettings;

#if WITH_EDITORONLY_DATA
	UCommonUIEditorSettings* CommonUIEditorSettings;
#endif
};

FCommonUIModule::FCommonUIModule()
{
	CommonUISettings = nullptr;
#if WITH_EDITOR
	CommonUIEditorSettings = nullptr;
#endif
}

void FCommonUIModule::StartupModule()
{
	CreateUISettings();
}

void FCommonUIModule::CreateUISettings()
{
	CommonUISettings = NewObject<UCommonUISettings>(GetTransientPackage(), UCommonUISettings::StaticClass());
	check(CommonUISettings);
	CommonUISettings->AddToRoot();

#if WITH_EDITOR

	CommonUIEditorSettings = NewObject<UCommonUIEditorSettings>(GetTransientPackage(), UCommonUIEditorSettings::StaticClass());
	check(CommonUIEditorSettings);
	CommonUIEditorSettings->AddToRoot();

	// Register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "CommonUI",
			NSLOCTEXT("CommonUIPlugin", "CommonUISettingsName", "Common UI Framework"),
			NSLOCTEXT("CommonUIPlugin", "CommonUISettingsDescription", "Configure Common UI Framework defaults."),
			CommonUISettings);

		ISettingsSectionPtr EditorSettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "CommonUIEditor",
			NSLOCTEXT("CommonUIPlugin", "CommonUIEditorSettingsName", "Common UI Editor"),
			NSLOCTEXT("CommonUIPlugin", "CommonUIEditorSettingsDescription", "Configure Common UI Editor defaults."),
			CommonUIEditorSettings);

		SettingsModule->RegisterSettings(TEXT("Project"), TEXT("Plugins"), TEXT("CommonUIInputSettings"),
			NSLOCTEXT("CommonUIPlugin", "CommonUIInputSettingsName", "Common UI Input Settings"),
			NSLOCTEXT("CommonUIPlugin", "CommonUIInputSettingsDescription", "Establish project-wide UI input settings and action mappings."),
			GetMutableDefault<UCommonUIInputSettings>());
	}

	// Always load editor data when in the editor
	CommonUIEditorSettings->LoadData();
#endif //WITH_EDITOR

	CommonUISettings->AutoLoadData();
}

void FCommonUIModule::ShutdownModule()
{
	
#if WITH_EDITOR
	// Unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "CommonUI");
		SettingsModule->UnregisterSettings("Project", "Plugins", "CommonUIEditor");
	}

	if (!GExitPurge) // If GExitPurge Object is already gone
	{
		CommonUIEditorSettings->RemoveFromRoot();
	}
	CommonUIEditorSettings = nullptr;
#endif //WITH_EDITOR

	if (!GExitPurge) // If GExitPurge Object is already gone
	{
		CommonUISettings->RemoveFromRoot();
	}
	CommonUISettings = nullptr;
}

FStreamableManager& FCommonUIModule::GetStreamableManager() const
{
	// CommonUI depends on there being an AssetManagerClassName defined in defaultengine.ini
	return UAssetManager::Get().GetStreamableManager();
}

TAsyncLoadPriority FCommonUIModule::GetLazyLoadPriority() const
{
	return 0;
}

UCommonUISettings* FCommonUIModule::GetSettingsInstance() const
{
	return CommonUISettings;
}

#if WITH_EDITOR
UCommonUIEditorSettings* FCommonUIModule::GetEditorSettingsInstance() const
{
	return CommonUIEditorSettings;
}
#endif

IMPLEMENT_MODULE(FCommonUIModule, CommonUI);

DEFINE_LOG_CATEGORY(LogCommonUI);
