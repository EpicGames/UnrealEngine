// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUIObjectVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid ECommonUIObjectVersion::Guid = FGuid(0x4288211B, 0x454816C6, 0x1A7667B2, 0x507A2A00);
FCustomVersionRegistration GCommonUIModuleVersion(ECommonUIObjectVersion::Guid, ECommonUIObjectVersion::LatestVersion, TEXT("CommonUI"));