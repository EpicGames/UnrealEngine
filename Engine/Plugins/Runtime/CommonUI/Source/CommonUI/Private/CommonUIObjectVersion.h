// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

// Custom serialization version for changes made to the CommonUI module
namespace ECommonUIObjectVersion
{
	enum Type
	{
		Initial = 0,

		// Updated the UCommonUILoadGuard to expose its content in a ULoadGuardSlot
		CreatedLoadGuardSlot,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const extern FGuid Guid;
};
