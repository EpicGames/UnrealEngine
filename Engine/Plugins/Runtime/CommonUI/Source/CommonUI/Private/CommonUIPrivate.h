// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Dependencies
 *****************************************************************************/

#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCommonUI, Log, All);
