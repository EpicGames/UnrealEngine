// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUIRichTextData.h"

#include "CommonUISettings.h"
#include "ICommonUIModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUIRichTextData)

UCommonUIRichTextData* UCommonUIRichTextData::Get()
{
	return ICommonUIModule::GetSettings().GetRichTextData();
}

const FRichTextIconData* UCommonUIRichTextData::FindIcon(const FName& InKey)
{
	FString ContextString = TEXT("UCommonUIRichTextData::FindIcon");
	return InlineIconSet->FindRow<FRichTextIconData>(InKey, ContextString);
}

