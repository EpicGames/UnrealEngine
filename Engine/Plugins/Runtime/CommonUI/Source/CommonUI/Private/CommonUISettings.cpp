// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUISettings.h"

#include "Engine/PlatformSettingsManager.h"
#include "Materials/MaterialInterface.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUISettings)

UE_DEFINE_GAMEPLAY_TAG(TAG_PlatformTrait_PlayInEditor, "Platform.Trait.PlayInEditor");

UCommonUISettings::UCommonUISettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, bAutoLoadData(true)
	, CommonButtonAcceptKeyHandling(ECommonButtonAcceptKeyHandling::Ignore)
	, bDefaultDataLoaded(false)
{}

UCommonUISettings::UCommonUISettings(FVTableHelper& Helper) : Super(Helper)
{
}

UCommonUISettings::~UCommonUISettings() = default;

void UCommonUISettings::LoadData()
{
	LoadEditorData();
}

#if WITH_EDITOR
void UCommonUISettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bDefaultDataLoaded = false;
	AutoLoadData();
	RebuildTraitContainer();
}
#endif

void UCommonUISettings::PostReloadConfig(FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	RebuildTraitContainer();
	AutoLoadData();
}

void UCommonUISettings::PostInitProperties()
{
	Super::PostInitProperties();

	RebuildTraitContainer();
	AutoLoadData();
}

void UCommonUISettings::LoadEditorData()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (!bDefaultDataLoaded)
	{
		DefaultImageResourceObjectInstance = DefaultImageResourceObject.LoadSynchronous();
		DefaultThrobberMaterialInstance = DefaultThrobberMaterial.LoadSynchronous();

		TSubclassOf<UCommonUIRichTextData> RichTextDataClass = DefaultRichTextDataClass.LoadSynchronous();
		RichTextDataInstance = RichTextDataClass.GetDefaultObject();

		if (GUObjectArray.IsDisregardForGC(this))
		{
			if (DefaultImageResourceObjectInstance)
			{
				DefaultImageResourceObjectInstance->AddToRoot();
			}
			if (DefaultThrobberMaterialInstance)
			{
				DefaultThrobberMaterialInstance->AddToRoot();
			}
			if (RichTextDataInstance)
			{
				RichTextDataInstance->AddToRoot();
			}
		}

		DefaultThrobberBrush.SetResourceObject(DefaultThrobberMaterialInstance);
		DefaultThrobberBrush.ImageSize = FVector2D(64.f, 64.f);
		DefaultThrobberBrush.ImageType = ESlateBrushImageType::FullColor;
		DefaultThrobberBrush.DrawAs = DefaultThrobberMaterialInstance ? ESlateBrushDrawType::Image : ESlateBrushDrawType::NoDrawType;

		bDefaultDataLoaded = true;
	}
}

void UCommonUISettings::RebuildTraitContainer()
{
	PlatformTraitContainer.Reset();
	for (FGameplayTag Trait : PlatformTraits)
	{
		PlatformTraitContainer.AddTag(Trait);
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		PlatformTraitContainer.AddTag(TAG_PlatformTrait_PlayInEditor);
	}
#endif
}

void UCommonUISettings::AutoLoadData()
{
	if (bAutoLoadData)
	{
		LoadData();
	}
}

UCommonUIRichTextData* UCommonUISettings::GetRichTextData() const
{
	ensure(bDefaultDataLoaded);

	return RichTextDataInstance;
}

const FSlateBrush& UCommonUISettings::GetDefaultThrobberBrush() const
{
	ensure(bDefaultDataLoaded);

	return DefaultThrobberBrush;
}

UObject* UCommonUISettings::GetDefaultImageResourceObject() const
{
	ensure(bDefaultDataLoaded);

	return DefaultImageResourceObjectInstance;
}

const FGameplayTagContainer& UCommonUISettings::GetPlatformTraits() const
{
#if WITH_EDITOR
	const FName SimulatedPlatform = UPlatformSettingsManager::GetEditorSimulatedPlatform();
	if (SimulatedPlatform != NAME_None)
	{
		static FName LastSimulatedPlatform = NAME_None;
		static FGameplayTagContainer SimulatedPlatformTraitContainer;

		if (SimulatedPlatform != LastSimulatedPlatform)
		{
			LastSimulatedPlatform = SimulatedPlatform;
			SimulatedPlatformTraitContainer.Reset();

			if (FConfigCacheIni* PlatformIni = FConfigCacheIni::ForPlatform(SimulatedPlatform))
			{
				TArray<FString> Entries;
				PlatformIni->GetArray(*GetClass()->GetPathName(), GET_MEMBER_NAME_STRING_CHECKED(UCommonUISettings, PlatformTraits), Entries, TEXT("Game"));

				UScriptStruct* GameplayTagStruct = StaticStruct<FGameplayTag>();
				TArray<FGameplayTag> OtherPlatformTraits;
				for(FString Entry : Entries)
				{
					FGameplayTag Element;
					GameplayTagStruct->ImportText(*Entry, &Element, nullptr, EPropertyPortFlags::PPF_None, nullptr, GameplayTagStruct->GetName(), true);
					OtherPlatformTraits.Add(Element);
				}

				for (FGameplayTag Trait : OtherPlatformTraits)
				{
					SimulatedPlatformTraitContainer.AddTag(Trait);
				}
			}
		}

		return SimulatedPlatformTraitContainer;
	}
#endif

	return PlatformTraitContainer;
}

ECommonButtonAcceptKeyHandling UCommonUISettings::GetCommonButtonAcceptKeyHandling() const
{
	return CommonButtonAcceptKeyHandling;
}
