// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUISubsystemBase.h"


#include "CommonInputBaseTypes.h"
#include "CommonInputSubsystem.h"
#include "CommonUITypes.h"
#include "IAnalyticsProviderET.h"
#include "InputAction.h"
#include "Components/Widget.h"
#include "Engine/GameInstance.h"
#include "EnhancedInputSubsystems.h"
#include "Styling/StyleDefaults.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUISubsystemBase)


UCommonUISubsystemBase* UCommonUISubsystemBase::Get(const UWidget& Widget)
{
	return UGameInstance::GetSubsystem<UCommonUISubsystemBase>(Widget.GetGameInstance());
}

bool UCommonUISubsystemBase::ShouldCreateSubsystem(UObject* Outer) const
{
	return !CastChecked<UGameInstance>(Outer)->IsDedicatedServerInstance();
}

void UCommonUISubsystemBase::Initialize(FSubsystemCollectionBase& Collection)
{
	CommonUI::SetupStyles();
}

void UCommonUISubsystemBase::SetAnalyticProvider(const TSharedPtr<IAnalyticsProviderET>& AnalyticProvider)
{
	AnalyticProviderWeakPtr = AnalyticProvider;
}

FSlateBrush UCommonUISubsystemBase::GetInputActionButtonIcon(const FDataTableRowHandle& InputActionRowHandle, ECommonInputType InputType, const FName& GamepadName) const
{
	if (ensure(InputType != ECommonInputType::Count) && !InputActionRowHandle.IsNull())
	{
		const FCommonInputActionDataBase* InputActionData = CommonUI::GetInputActionData(InputActionRowHandle);
		if (ensure(InputActionData))
		{
			const FCommonInputTypeInfo& InputTypeInfo = InputActionData->GetInputTypeInfo(InputType, GamepadName);
			if (InputTypeInfo.OverrideBrush.DrawAs != ESlateBrushDrawType::NoDrawType)
			{
				return InputTypeInfo.OverrideBrush;
			}

			FSlateBrush SlateBrush;
			if (UCommonInputPlatformSettings::Get()->TryGetInputBrush(SlateBrush, InputTypeInfo.GetKey(), InputType, GamepadName))
			{
				return SlateBrush;
			}
		}
	}

	return *FStyleDefaults::GetNoBrush();
}

FSlateBrush UCommonUISubsystemBase::GetEnhancedInputActionButtonIcon(const UInputAction* InputAction, const ULocalPlayer* LocalPlayer) const
{
	if (LocalPlayer)
	{
		return CommonUI::GetIconForEnhancedInputAction(UCommonInputSubsystem::Get(LocalPlayer), InputAction);
	}

	return *FStyleDefaults::GetNoBrush();
}

/**
 * @EventName CommonUI.ButtonClicked
 * @Trigger (DISABLED) Button presses that are marked for analytic events
 * @Type Client
 * @EventParam ButtonName string The name of the button
 * @EventParam ABTestName string The AB Test group the button is a part of if applicable
 * @EventParam ExtraData string Extra data about the button that may be useful
 * @owner chris.gagnon
 */
void UCommonUISubsystemBase::FireEvent_ButtonClicked(const FString& ButtonName, const FString& ABTestName, const FString& ExtraData) const
{
	// WRH 2018-03-07 Disable this analytics event until someone is using it.
#if 0
	const FString EventName = TEXT("CommonUI.ButtonClicked");
	const FString Attrib_ButtonName = TEXT("ButtonName");
	const FString Attrib_ABTestName = TEXT("ABTestName");
	const FString Attrib_ExtraData = TEXT("ExtraData");

	TSharedPtr<IAnalyticsProviderET> AnalyticsProvider = AnalyticProviderWeakPtr.Pin();
	if (AnalyticsProvider.IsValid())
	{
		TArray<FAnalyticsEventAttribute> ParamArray;

		ParamArray.Add(FAnalyticsEventAttribute(Attrib_ButtonName, ButtonName));
		ParamArray.Add(FAnalyticsEventAttribute(Attrib_ABTestName, ABTestName));
		ParamArray.Add(FAnalyticsEventAttribute(Attrib_ExtraData, ExtraData));

		AnalyticsProvider->RecordEvent(EventName, ParamArray);
	}
#endif
}

/**
 * @EventName CommonUI.PanelPushed
 * @Trigger (DISABLED) Button presses that are marked for analytic events
 * @Type Client
 * @EventParam PanelName string The name of the panel that was pushed
 * @owner chris.gagnon
 */
void UCommonUISubsystemBase::FireEvent_PanelPushed(const FString& PanelName) const
{
	// WRH 2018-03-07 Disable this analytics event until someone is using it.
#if 0
	const FString EventName = TEXT("CommonUI.PanelPushed");
	const FString Attrib_PanelName = TEXT("PanelName");

	TSharedPtr<IAnalyticsProviderET> AnalyticsProvider = AnalyticProviderWeakPtr.Pin();
	if (AnalyticsProvider.IsValid())
	{
		TArray<FAnalyticsEventAttribute> ParamArray;

		ParamArray.Add(FAnalyticsEventAttribute(Attrib_PanelName, PanelName));

		AnalyticsProvider->RecordEvent(EventName, ParamArray);
	}
#endif
}

void UCommonUISubsystemBase::SetInputAllowed(bool bEnabled, const FName& Reason, const ULocalPlayer& LocalPlayer)
{
	if (UCommonInputSubsystem* InputSubsystem = UCommonInputSubsystem::Get(&LocalPlayer))
	{
		InputSubsystem->SetInputTypeFilter(ECommonInputType::MouseAndKeyboard, Reason, !bEnabled);
		InputSubsystem->SetInputTypeFilter(ECommonInputType::Gamepad, Reason, !bEnabled);
		InputSubsystem->SetInputTypeFilter(ECommonInputType::Touch, Reason, !bEnabled);
	}
}

bool UCommonUISubsystemBase::IsInputAllowed(const ULocalPlayer* LocalPlayer) const
{
	if (UCommonInputSubsystem* InputSubSystem = UCommonInputSubsystem::Get(LocalPlayer))
	{
		const bool bKeyboardAndMouseFiltered = InputSubSystem->GetInputTypeFilter(ECommonInputType::MouseAndKeyboard);
		const bool bGamepadFiltered = InputSubSystem->GetInputTypeFilter(ECommonInputType::Gamepad);
		const bool bTouchFiltered = InputSubSystem->GetInputTypeFilter(ECommonInputType::Touch);
		return !bKeyboardAndMouseFiltered || !bGamepadFiltered || !bTouchFiltered;
	}

	return true;
}

