// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "CommonUITypes.h"

#include "CommonInputBaseTypes.h"
#include "CommonInputSubsystem.h"
#include "CommonUIPrivate.h"
#include "CommonInputSettings.h"
#include "Engine/Blueprint.h"
#include "EnhancedActionKeyMapping.h"
#include "EnhancedInputSubsystems.h"
#include "Engine/LocalPlayer.h"
#include "HAL/PlatformInput.h"
#include "ICommonInputModule.h"
#include "InputAction.h"
#include "InputActionValue.h"
#include "Misc/ConfigCacheIni.h"
#include "PlayerMappableKeySettings.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleDefaults.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUITypes)

FScrollBoxStyle CommonUI::EmptyScrollBoxStyle = FScrollBoxStyle();

FCommonInputTypeInfo::FCommonInputTypeInfo()
{
	OverrrideState = EInputActionState::Enabled;
	OverrideBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	bActionRequiresHold = false;
	HoldTime = 0.5f;
	HoldRollbackTime = 0.0f;
}

FKey FCommonInputTypeInfo::GetKey() const
{
	return FPlatformInput::RemapKey(Key);
}

TArray<FKey> FCommonInputTypeInfo::GetAllKeys() const
{
	TArray<FKey> AllKeys;
	FKey PrimaryKey = GetKey();
	if (PrimaryKey.IsValid())
	{
		AllKeys.Add(PrimaryKey);
	}
	for (const FKey& AdditionalKey : AdditionalKeys)
	{
		FKey RemappedKey = FPlatformInput::RemapKey(AdditionalKey);
		if (RemappedKey.IsValid())
		{
			AllKeys.Add(RemappedKey);
		}
	}
	return AllKeys;
}

bool FCommonInputTypeInfo::IsKeyBound(const FKey& InKey) const
{
	if (InKey == GetKey())
	{
		return true;
	}
	for (const FKey& AdditionalKey : AdditionalKeys)
	{
		if (InKey == FPlatformInput::RemapKey(AdditionalKey))
		{
			return true;
		}
	}
	return false;
}

FCommonInputActionDataBase::FCommonInputActionDataBase()
{
}

bool FCommonInputActionDataBase::CanDisplayInReflector(ECommonInputType InputType, const FName& GamepadName) const
{
	if (InputType == ECommonInputType::MouseAndKeyboard)
	{
		return true;
	}
	else if (InputType == ECommonInputType::Gamepad)
	{
		const FCommonInputTypeInfo& TypeInfo = GetInputTypeInfo(InputType, GamepadName);
		return TypeInfo.GetKey().IsValid();
	}
	else if (InputType == ECommonInputType::Touch)
	{
		return true;
	}

	return false;
}

const FCommonInputTypeInfo& FCommonInputActionDataBase::GetCurrentInputTypeInfo(const UCommonInputSubsystem* CommonInputSubsystem) const
{
	ECommonInputType CurrentInputType = ECommonInputType::MouseAndKeyboard;
	FName CurrentGamepadName = FCommonInputDefaults::GamepadGeneric;

	if (CommonInputSubsystem) // may not always have a valid input Subsystem, e.g. for the demo driver's SpectatorPlayerController when recording a replay
	{
		CurrentInputType = CommonInputSubsystem->GetCurrentInputType();
		CurrentGamepadName = CommonInputSubsystem->GetCurrentGamepadName();
	}

	return GetInputTypeInfo(CurrentInputType, CurrentGamepadName);
}

const FCommonInputTypeInfo& FCommonInputActionDataBase::GetInputTypeInfo(ECommonInputType InputType, const FName& GamepadName) const
{
	if (InputType == ECommonInputType::MouseAndKeyboard)
	{
		return KeyboardInputTypeInfo;
	}
	else if (InputType == ECommonInputType::Gamepad)
	{
		const FCommonInputTypeInfo* GamepadTypeInfo = &DefaultGamepadInputTypeInfo;
		if (const FCommonInputTypeInfo* OverrideTypeInfo = GamepadInputOverrides.Find(GamepadName))
		{
			GamepadTypeInfo = OverrideTypeInfo;
		}

		//ensureAlways(NewGamepadTypeInfo->GetKey().IsValid());
		UE_CLOGF(!GamepadTypeInfo->GetKey().IsValid(), LogCommonUI, Verbose, "Invalid default common action key for action \"%ls\"", *DisplayName.ToString());
		return *GamepadTypeInfo;
	}
	else if (InputType == ECommonInputType::Touch)
	{
		return TouchInputTypeInfo;
	}

	return KeyboardInputTypeInfo;
}

bool FCommonInputActionDataBase::IsKeyBoundToInputActionData(const FKey& Key) const
{
	if (KeyboardInputTypeInfo.IsKeyBound(Key) || TouchInputTypeInfo.IsKeyBound(Key))
	{
		return true;
	}

	for (const FName& GamepadName : UCommonInputBaseControllerData::GetRegisteredGamepads())
	{
		const FCommonInputTypeInfo& TypeInfo = GetInputTypeInfo(ECommonInputType::Gamepad, GamepadName);
		if (TypeInfo.IsKeyBound(Key))
		{
			return true;
		}
	}

	return false;
}

bool FCommonInputActionDataBase::IsKeyBoundToInputActionData(const FKey& Key, const UCommonInputSubsystem* CommonInputSubsystem) const
{
	const FCommonInputTypeInfo&	InputTypeInfo = GetCurrentInputTypeInfo(CommonInputSubsystem);

	if (InputTypeInfo.IsKeyBound(Key))
	{
		return true;
	}

	return false;
}

FSlateBrush FCommonInputActionDataBase::GetCurrentInputActionIcon(const UCommonInputSubsystem* CommonInputSubsystem) const
{
	const FCommonInputTypeInfo& CurrentInputTypeInfo = GetCurrentInputTypeInfo(CommonInputSubsystem);

	//if (CurrentInputTypeInfo.OverrideBrush.DrawAs != ESlateBrushDrawType::NoDrawType)
	//{
	//	return CurrentInputTypeInfo.OverrideBrush;
	//}

	FSlateBrush SlateBrush;
	if (UCommonInputPlatformSettings::Get()->TryGetInputBrush(SlateBrush, CurrentInputTypeInfo.GetKey(), CommonInputSubsystem->GetCurrentInputType(), CommonInputSubsystem->GetCurrentGamepadName()))
	{
		return SlateBrush;
	}

	return *FStyleDefaults::GetNoBrush();
}

bool FCommonInputActionDataBase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

	// doesn't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FCommonInputActionDataBase::PostSerialize(const FArchive& Ar)
{
	// During load, if the asset version is lower then when we added our platform upgrade path
	// then we should run that upgrade path data
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::CommonUIPlatformNamingUpgradeOption)
		{
			// Upgrade the gamepad override path stuff!
			const TMap<FName, FName>& PlatformNameRedirects = GetDefault<UCommonInputSettings>()->GetPlatformNameUpgradeMap();

			for (const TPair<FName, FName>& Redirect : PlatformNameRedirects)
			{
				// If there is a platform override defined for one of the platforms we want to copy from...
				const FCommonInputTypeInfo* FromPadData = GamepadInputOverrides.Find(Redirect.Key);
				if (!FromPadData)
				{
					continue;
				}

				// Make sure that the override is not already defined for this platform
				if (GamepadInputOverrides.Contains(Redirect.Value))
				{
					continue;
				}

				// Add the new gamepad override data and initialize it to the data which we had for the older platform name
				GamepadInputOverrides.Add(Redirect.Value, FCommonInputTypeInfo (*FromPadData));
			}
		}
	}
}

void FCommonInputActionDataBase::OnPostDataImport(const UDataTable* InDataTable, const FName InRowName, TArray<FString>& OutCollectedImportProblems)
{
#if WITH_EDITOR
#endif // WITH_EDITOR
}

bool FCommonInputActionDataBase::HasHoldBindings() const
{
	if (DefaultGamepadInputTypeInfo.bActionRequiresHold)
	{
		return true;
	}

	for (const TPair<FName, FCommonInputTypeInfo>& GamepadInfo : GamepadInputOverrides)
	{
		if (GamepadInfo.Value.bActionRequiresHold)
		{
			return true;
		}
	}

	if (KeyboardInputTypeInfo.bActionRequiresHold)
	{
		return true;
	}

	if (TouchInputTypeInfo.bActionRequiresHold)
	{
		return true;
	}

	return false;
}

const FCommonInputTypeInfo& FCommonInputActionDataBase::GetDefaultGamepadInputTypeInfo() const
{
	return DefaultGamepadInputTypeInfo;
}

bool FCommonInputActionDataBase::HasGamepadInputOverride(const FName& GamepadName) const
{
	return GamepadInputOverrides.Contains(GamepadName);
}

void FCommonInputActionDataBase::AddGamepadInputOverride(const FName& GamepadName, const FCommonInputTypeInfo& InputInfo)
{
	GamepadInputOverrides.Add(GamepadName, InputInfo);
}

const UCommonInputMetadata* UCommonMappingContextMetadata::GetCommonInputMetadata(const UInputAction* InInputAction) const
{
	if (const TObjectPtr<const UCommonInputMetadata>* PerActionMetdata = PerActionEnhancedInputMetadata.Find(InInputAction))
	{
		return *PerActionMetdata;
	}

	return EnhancedInputMetadata;
}

void CommonUI::SetupStyles()
{
	EmptyScrollBoxStyle.BottomShadowBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	EmptyScrollBoxStyle.TopShadowBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	EmptyScrollBoxStyle.LeftShadowBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
	EmptyScrollBoxStyle.RightShadowBrush.DrawAs = ESlateBrushDrawType::NoDrawType;
}

const FCommonInputActionDataBase* CommonUI::GetInputActionData(const FDataTableRowHandle& InputActionRowHandle)
{
	return InputActionRowHandle.GetRow<FCommonInputActionDataBase>(TEXT("CommonUIUtils::GetInputActionData couldn't find the row passed in, check data table if its missing."));
}

FSlateBrush CommonUI::GetIconForInputActions(const UCommonInputSubsystem* CommonInputSubsystem, const TArray<FDataTableRowHandle>& InputActions)
{
	TArray<FKey> Keys;
	for (const FDataTableRowHandle& InputAction : InputActions)
	{
		if (const FCommonInputActionDataBase* InputActionData = GetInputActionData(InputAction))
		{
			const FCommonInputTypeInfo& TypeInfo = InputActionData->GetCurrentInputTypeInfo(CommonInputSubsystem);
			Keys.Add(TypeInfo.GetKey());
		}
		else
		{
			return *FStyleDefaults::GetNoBrush();
		}
	}

	FSlateBrush SlateBrush;
	if (UCommonInputPlatformSettings::Get()->TryGetInputBrush(SlateBrush, Keys, CommonInputSubsystem->GetCurrentInputType(), CommonInputSubsystem->GetCurrentGamepadName()))
	{
		return SlateBrush;
	}

	return *FStyleDefaults::GetNoBrush();
}

bool CommonUI::IsEnhancedInputSupportEnabled()
{
	if (IsEngineExitRequested())
	{
		return false;
	}

	static bool bEnabled = ICommonInputModule::Get().GetSettings().GetEnableEnhancedInputSupport();
	return bEnabled;
}

TObjectPtr<const UCommonInputMetadata> CommonUI::GetEnhancedInputActionMetadata(const UInputAction* InputAction)
{
	if (!InputAction->GetPlayerMappableKeySettings())
	{
		return nullptr;
	}

	if (ICommonMappingContextMetadataInterface* MappingContextMetadata = Cast<ICommonMappingContextMetadataInterface>(InputAction->GetPlayerMappableKeySettings()->Metadata))
	{
		return MappingContextMetadata->GetCommonInputMetadata(InputAction);
	}

	return nullptr;
}

void CommonUI::GetEnhancedInputActionKeys(const ULocalPlayer* LocalPlayer, const UInputAction* InputAction, TArray<FKey>& OutKeys)
{
	if (LocalPlayer)
	{
		if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
		{
			OutKeys = EnhancedInputLocalPlayerSubsystem->QueryKeysMappedToAction(InputAction);
		}
	}
}

void CommonUI::InjectEnhancedInputForAction(const ULocalPlayer* LocalPlayer, const UInputAction* InputAction, FInputActionValue RawValue)
{
	if (LocalPlayer)
	{
		if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
		{
			EnhancedInputLocalPlayerSubsystem->InjectInputForAction(InputAction, RawValue, {}, {});
		}
	}
}

FSlateBrush CommonUI::GetIconForEnhancedInputAction(const UCommonInputSubsystem* CommonInputSubsystem, const UInputAction* InputAction)
{
	FKey FirstKeyForCurrentInput = GetFirstKeyForInputType(CommonInputSubsystem->GetLocalPlayer(), CommonInputSubsystem->GetCurrentInputType(), InputAction);

	FSlateBrush SlateBrush;
	if (FirstKeyForCurrentInput.IsValid() && UCommonInputPlatformSettings::Get()->TryGetInputBrush(SlateBrush, FirstKeyForCurrentInput, CommonInputSubsystem->GetCurrentInputType(), CommonInputSubsystem->GetCurrentGamepadName()))
	{
		return SlateBrush;
	}

	return *FStyleDefaults::GetNoBrush();
}

bool CommonUI::ActionValidForInputType(const ULocalPlayer* LocalPlayer, ECommonInputType InputType, const UInputAction* InputAction)
{
	if (!LocalPlayer || !InputAction)
	{
		return false;
	}

	TArray<FKey> Keys;
	if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
	{
		Keys = EnhancedInputLocalPlayerSubsystem->QueryKeysMappedToAction(InputAction);
	}

	for (const FKey& Key : Keys)
	{
		if (IsKeyValidForInputType(Key, InputType))
		{
			return true;
		}
	}

	return false;
}

FKey CommonUI::GetFirstKeyForInputType(const ULocalPlayer* LocalPlayer, ECommonInputType InputType, const UInputAction* InputAction)
{
	if (!LocalPlayer || !InputAction)
	{
		return FKey();
	}

	TArray<FKey> Keys;
	if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = LocalPlayer->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
	{
		Keys = EnhancedInputLocalPlayerSubsystem->QueryKeysMappedToAction(InputAction);
	}

	for (const FKey& Key : Keys)
	{
		if (IsKeyValidForInputType(Key, InputType))
		{
			return Key;
		}
	}

	return FKey();
}

bool CommonUI::ActionValidForInputType(const ULocalPlayer* LocalPlayer, ECommonInputType InputType, const FCommonInputActionDataBase* InputAction)
{
	if (!LocalPlayer || !InputAction)
	{
		return false;
	}
	
	UCommonInputSubsystem* InputSubsystem = LocalPlayer->GetSubsystem<UCommonInputSubsystem>();
	const FCommonInputTypeInfo& TypeInfo = InputAction->GetInputTypeInfo(InputType, InputSubsystem->GetCurrentGamepadName());
	return IsKeyValidForInputType(TypeInfo.GetKey(), InputType);
}

bool CommonUI::IsKeyValidForInputType(const FKey& Key, ECommonInputType InputType)
{
	if (!Key.IsValid())
	{
		return false;
	}

	switch (InputType)
	{
	case ECommonInputType::MouseAndKeyboard:
		return !Key.IsTouch() && !Key.IsGamepadKey();
	case ECommonInputType::Gamepad:
		return Key.IsGamepadKey();
	case ECommonInputType::Touch:
		return Key.IsTouch();
	case ECommonInputType::Count:
	default:
		checkNoEntry(); // Unhandled type
		return false;
	}
}
