// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "CommonUIUtils.h"
#include "Components/PanelWidget.h"
#include "Editor/WidgetCompilerLog.h"

#define LOCTEXT_NAMESPACE "CommonUIUtils"

namespace CommonUIUtils
{
	static TAutoConsoleVariable<int32> EnableMobileUITextScaling(
		TEXT("Mobile.EnableUITextScaling"),
		0,
		TEXT("Enables Mobile UI Text Scaling"),
		ECVF_Default);

	float UISmallTextScaling = -1.0f;
	FAutoConsoleVariableRef CVarUITextSmallTextScaling(
		TEXT("CommonUI.SmallTextScaling"),
		UISmallTextScaling,
		TEXT("Scaling to apply to common ui texts smaller than the configured threshold (unless mobile UI text scaling is on). Ignored if 0 or lesser"));

	float UISmallTextScalingThresholdMax = 15.0f;
	FAutoConsoleVariableRef CVarUITextSmallTextScalingThresholdMax(
		TEXT("CommonUI.SmallTextScalingThresholdMax"),
		UISmallTextScalingThresholdMax,
		TEXT("Maximum size on which small text Scaling will be applied."));

	bool ShouldDisplayMobileUISizes()
	{
#if PLATFORM_ANDROID || PLATFORM_IOS
		return true;
#else
		return EnableMobileUITextScaling->GetInt() == 1;
#endif
	}

	bool ShouldScaleText()
	{
		return ShouldDisplayMobileUISizes() || UISmallTextScaling > 0.0f;
	}

	FString PrintAllOwningUserWidgets(const UWidget* Widget)
	{
		FString OutputString;
		const UUserWidget* ParentWidget = nullptr;
		const UWidget* CurrentWidget = Widget;

		while (CurrentWidget)
		{
			ParentWidget = CommonUIUtils::GetOwningUserWidget<UUserWidget>(CurrentWidget);
			if (ParentWidget)
			{
				if (OutputString.Len() > 0)
				{
					OutputString.Append(TEXT(", "));
				}

				OutputString.Append(ParentWidget->GetName());
			}

			CurrentWidget = ParentWidget;
		}

		return MoveTemp(OutputString);
	}

#if WITH_EDITOR
	void ValidateBoundWidgetHierarchy(const UWidgetTree& WidgetTree, IWidgetCompilerLog& CompileLog, ECollisionPolicy CollisionPolicy, FName ParentWidgetName, TArray<FName>&& ChildNames)
	{
		if (UPanelWidget* ParentWidget = WidgetTree.FindWidget<UPanelWidget>(ParentWidgetName))
		{
			TArray<int32> ChildIndices;
			ChildIndices.Reserve(ChildNames.Num());

			for (FName ChildName : ChildNames)
			{
				int32 ChildIdx = INDEX_NONE;
				UWidgetTree::FindWidgetChild(ParentWidget, ChildName, ChildIdx);

				if (ChildIdx == INDEX_NONE)
				{
					const FText MissingChildError = LOCTEXT("Error_MissingBoundChild", "Bound widget \"{0}\" is expected to be a child of widget \"{1}\"");
					CompileLog.Error(FText::Format(MissingChildError, FText::FromName(ChildName), FText::FromName(ParentWidgetName)));
				}
				else if (ChildIndices.Num() && (CollisionPolicy == ECollisionPolicy::Require) && !ChildIndices.Contains(ChildIdx))
				{
					const FText ChildCollisionRequiredError = LOCTEXT("Error_ChildSlotCollisionRequired", "Bound widget \"{0}\" is expected to be in the same slot of \"{1}\" as \"{2}\".");
					CompileLog.Error(FText::Format(ChildCollisionRequiredError, FText::FromName(ChildName), FText::FromName(ParentWidgetName), FText::FromName(ChildNames[0])));
				}
				else if ((CollisionPolicy == ECollisionPolicy::Forbid) && ChildIndices.Contains(ChildIdx))
				{
					const FText ChildCollisionForbiddenError = LOCTEXT("Error_ChildSlotCollisionForbidden", "Bound widget \"{0}\" is in slot {1} of \"{2}\", but widget \"{3}\" is already there. These widgets are expected to be in different slots.");
					CompileLog.Error(FText::Format(ChildCollisionForbiddenError, FText::FromName(ChildName), FText::AsNumber(ChildIdx), FText::FromName(ParentWidgetName), FText::FromName(ChildNames[ChildIndices.FindLast(ChildIdx)])));
				}
				ChildIndices.Add(ChildIdx);
			}
		}
	}
#endif
}

#undef LOCTEXT_NAMESPACE 
