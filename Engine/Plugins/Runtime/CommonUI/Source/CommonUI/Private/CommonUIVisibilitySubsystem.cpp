// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUIVisibilitySubsystem.h"

#include "CommonInputTypeEnum.h"
#include "Engine/LocalPlayer.h"
#include "Engine/GameInstance.h"
#include "CommonInputSubsystem.h"
#include "UObject/UObjectIterator.h"
#include "ICommonUIModule.h"
#include "CommonUISettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUIVisibilitySubsystem)

UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_INPUT_MOUSEANDKEYBOARD, "Input.MouseAndKeyboard");
UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_INPUT_GAMEPAD, "Input.Gamepad");
UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_INPUT_TOUCH, "Input.Touch");

#if WITH_EDITOR
FGameplayTagContainer UCommonUIVisibilitySubsystem::DebugTagsToEnable;
FGameplayTagContainer UCommonUIVisibilitySubsystem::DebugTagsToSuppress;
#endif

UCommonUIVisibilitySubsystem* UCommonUIVisibilitySubsystem::Get(const ULocalPlayer* LocalPlayer)
{
	return LocalPlayer ? LocalPlayer->GetSubsystem<UCommonUIVisibilitySubsystem>() : nullptr;
}

UCommonUIVisibilitySubsystem* UCommonUIVisibilitySubsystem::GetChecked(const ULocalPlayer* LocalPlayer)
{
	check(LocalPlayer);
	UCommonUIVisibilitySubsystem* Result = Get(LocalPlayer);
	check(Result);
	return Result;
}

UCommonUIVisibilitySubsystem::UCommonUIVisibilitySubsystem()
{
}

bool UCommonUIVisibilitySubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	const UGameInstance* GameInstance = CastChecked<ULocalPlayer>(Outer)->GetGameInstance();
	if (GameInstance && !GameInstance->IsDedicatedServerInstance())
	{
		TArray<UClass*> ChildClasses;
		GetDerivedClasses(GetClass(), ChildClasses, false);

		// Only create an instance if there is no override implementation defined elsewhere
		return ChildClasses.Num() == 0;
	}
	
	return false;
}

void UCommonUIVisibilitySubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	Collection.InitializeDependency<UCommonInputSubsystem>();
	
	if (UCommonInputSubsystem* Input = GetLocalPlayer()->GetSubsystem<UCommonInputSubsystem>())
	{
		Input->OnInputMethodChangedNative.AddUObject(this, &ThisClass::OnInputMethodChanged);
	}

	ComputedVisibilityTags = ComputeVisibilityTags();
}

void UCommonUIVisibilitySubsystem::Deinitialize()
{
	Super::Deinitialize();
}

void UCommonUIVisibilitySubsystem::AddUserVisibilityCondition(const FGameplayTag UserTag)
{
	UserVisibilityTags.AddTag(UserTag);
	RefreshVisibilityTags();
}

void UCommonUIVisibilitySubsystem::RemoveUserVisibilityCondition(const FGameplayTag UserTag)
{
	UserVisibilityTags.RemoveTag(UserTag);
	RefreshVisibilityTags();
}

#if WITH_EDITOR
void UCommonUIVisibilitySubsystem::SetDebugVisibilityConditions(const FGameplayTagContainer& TagsToEnable, const FGameplayTagContainer& TagsToSuppress)
{
	DebugTagsToEnable = TagsToEnable;
	DebugTagsToSuppress = TagsToSuppress;

	for (TObjectIterator<UCommonUIVisibilitySubsystem> SubsystemIt; SubsystemIt; ++SubsystemIt)
	{
		SubsystemIt->RefreshVisibilityTags();
	}
}
#endif

void UCommonUIVisibilitySubsystem::RefreshVisibilityTags()
{
	ComputedVisibilityTags = ComputeVisibilityTags();

	OnVisibilityTagsChanged.Broadcast(this);
}

FGameplayTagContainer UCommonUIVisibilitySubsystem::ComputeVisibilityTags() const
{
	FGameplayTagContainer ComputedTags = UserVisibilityTags;

	if (const UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(GetLocalPlayer()))
	{
		switch(CommonInputSubsystem->GetCurrentInputType())
		{
			case ECommonInputType::MouseAndKeyboard:
				ComputedTags.AddTag(TAG_INPUT_MOUSEANDKEYBOARD);
				break;
			case ECommonInputType::Gamepad:
				ComputedTags.AddTag(TAG_INPUT_GAMEPAD);
				break;
			case ECommonInputType::Touch:
				ComputedTags.AddTag(TAG_INPUT_TOUCH);
				break;
		}
	}

	ComputedTags.AppendTags(ICommonUIModule::GetSettings().GetPlatformTraits());

#if WITH_EDITOR
	ComputedTags.AppendTags(DebugTagsToEnable);
	ComputedTags.RemoveTags(DebugTagsToSuppress);
#endif

	// Debug printing
#if 0
	UE_LOGF(LogInit, Log, "UCommonUIVisibilitySubsystem::ComputeVisibilityTags() -> %ls", *ComputedTags.ToString());
#endif

	return ComputedTags;
}

void UCommonUIVisibilitySubsystem::OnInputMethodChanged(ECommonInputType CurrentInputType)
{
	RefreshVisibilityTags();
}

