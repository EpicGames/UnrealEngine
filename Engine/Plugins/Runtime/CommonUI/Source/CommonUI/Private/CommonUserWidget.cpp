// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUserWidget.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "Engine/GameInstance.h"
#include "CommonInputSubsystem.h"
#include "CommonUISubsystemBase.h"
#include "CommonWidgetPaletteCategories.h"
#include "Input/CommonUIActionRouterBase.h"
#include "Input/CommonUIInputTypes.h"
#include "InputMappingContext.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUserWidget)

UCommonUserWidget::UCommonUserWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{	
#if WITH_EDITORONLY_DATA
	PaletteCategory = FText::FromString(TEXT("Common UI"));
#endif
}

void UCommonUserWidget::SetConsumePointerInput(bool bInConsumePointerInput)
{
	bConsumePointerInput = bInConsumePointerInput;
}

UCommonInputSubsystem* UCommonUserWidget::GetInputSubsystem() const
{
	return UCommonInputSubsystem::Get(GetOwningLocalPlayer());
}

UCommonUISubsystemBase* UCommonUserWidget::GetUISubsystem() const
{
	return UGameInstance::GetSubsystem<UCommonUISubsystemBase>(GetGameInstance());
}

TSharedPtr<FSlateUser> UCommonUserWidget::GetOwnerSlateUser() const
{
	ULocalPlayer* LocalPlayer = GetOwningLocalPlayer();
	return LocalPlayer ? LocalPlayer->GetSlateUser() : nullptr;
}

FReply UCommonUserWidget::NativeOnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnMouseButtonDown(InGeometry, InMouseEvent);
}

FReply UCommonUserWidget::NativeOnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnMouseButtonUp(InGeometry, InMouseEvent);
}

FReply UCommonUserWidget::NativeOnMouseWheel(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnMouseWheel(InGeometry, InMouseEvent);
}

FReply UCommonUserWidget::NativeOnMouseButtonDoubleClick(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnMouseButtonDoubleClick(InGeometry, InMouseEvent);
}

FReply UCommonUserWidget::NativeOnTouchGesture(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnTouchGesture(InGeometry, InGestureEvent);
}

FReply UCommonUserWidget::NativeOnTouchStarted(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnTouchStarted(InGeometry, InGestureEvent);
}

FReply UCommonUserWidget::NativeOnTouchMoved(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnTouchMoved(InGeometry, InGestureEvent);
}

FReply UCommonUserWidget::NativeOnTouchEnded(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent)
{
	return bConsumePointerInput ? FReply::Handled() : Super::NativeOnTouchEnded(InGeometry, InGestureEvent);
}

FUIActionBindingHandle UCommonUserWidget::RegisterUIActionBinding(const FBindUIActionArgs& BindActionArgs)
{
	if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
	{
		FBindUIActionArgs FinalBindActionArgs = BindActionArgs;
		if (bDisplayInActionBar && !BindActionArgs.bDisplayInActionBar)
		{
			FinalBindActionArgs.bDisplayInActionBar = true;
		}
		FUIActionBindingHandle BindingHandle = ActionRouter->RegisterUIActionBinding(*this, FinalBindActionArgs);
		ActionBindings.Add(BindingHandle);
		return BindingHandle;
	}

	return FUIActionBindingHandle();
}

FUIActionBindingHandle UCommonUserWidget::RegisterUIAction(const UInputAction* EnhancedInputAction, bool ShouldDisplayInActionBar /* = true */)
{
	// Empty Callback are those are meant to be used as EnhancedInputAction. If we need we could add a callback in BP with the InputAction later.
	return RegisterUIActionBinding(
		FBindUIActionArgs(EnhancedInputAction, ShouldDisplayInActionBar, FSimpleDelegate::CreateLambda([]{})));
}

TArray<FUIActionBindingHandle> UCommonUserWidget::RegisterUIActionsFromMappingContext(const UInputMappingContext* MappingContext
	, bool ShouldDisplayInActionBar)
{
	if (MappingContext == nullptr)
	{
		return {};
	}

	TArray<FUIActionBindingHandle> ReturnActionBindings;
	for (const FEnhancedActionKeyMapping& Mapping : MappingContext->GetMappings())
	{
		ReturnActionBindings.Add(RegisterUIAction(Mapping.Action, ShouldDisplayInActionBar));
	}
	
	return ReturnActionBindings;
}

void UCommonUserWidget::RemoveUIAction(FUIActionBindingHandle ActionBinding)
{
	ActionBinding.Unregister();
	RemoveActionBinding(ActionBinding);
}

void UCommonUserWidget::RemoveAllUIActionBinding()
{
	if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
	{
		for (FUIActionBindingHandle& ActionBinding : ActionBindings)
		{
			ActionBinding.Unregister();
			ActionRouter->RemoveBinding(ActionBinding);
		}
	}
	ActionBindings.Reset();
}

void UCommonUserWidget::RemoveActionBinding(FUIActionBindingHandle ActionBinding)
{
	ActionBindings.Remove(ActionBinding);
	if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
	{
		ActionRouter->RemoveBinding(ActionBinding);
	}
}

void UCommonUserWidget::AddActionBinding(FUIActionBindingHandle ActionBinding)
{
	ActionBindings.Add(ActionBinding);
	if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
	{
		ActionRouter->AddBinding(ActionBinding);
	}
}

void UCommonUserWidget::RegisterScrollRecipient(const UWidget& AnalogScrollRecipient, ECommonUIScrollRecipientOwningNodeSource OwningNodeSource)
{
	if (!ScrollRecipients.Contains(&AnalogScrollRecipient))
	{
		ScrollRecipients.AddUnique(&AnalogScrollRecipient);
		if (GetCachedWidget())
		{
			if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
			{
				ActionRouter->RegisterScrollRecipient(AnalogScrollRecipient, OwningNodeSource == ECommonUIScrollRecipientOwningNodeSource::RegisteringUserWidget ? this : nullptr);
			}
		}
	}
}

void UCommonUserWidget::UnregisterScrollRecipient(const UWidget& AnalogScrollRecipient, ECommonUIScrollRecipientOwningNodeSource OwningNodeSource)
{
	if (ScrollRecipients.Remove(&AnalogScrollRecipient) && GetCachedWidget())
	{
		if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
		{
			ActionRouter->UnregisterScrollRecipient(AnalogScrollRecipient, OwningNodeSource == ECommonUIScrollRecipientOwningNodeSource::RegisteringUserWidget ? this : nullptr);
		}
	}
}

void UCommonUserWidget::RegisterScrollRecipientExternal(const UWidget* AnalogScrollRecipient)
{
	if (AnalogScrollRecipient != nullptr)
	{
		RegisterScrollRecipient(*AnalogScrollRecipient);
	}
}

void UCommonUserWidget::UnregisterScrollRecipientExternal(const UWidget* AnalogScrollRecipient)
{
	if (AnalogScrollRecipient != nullptr)
	{
		UnregisterScrollRecipient(*AnalogScrollRecipient);
	}
}

void UCommonUserWidget::OnWidgetRebuilt()
{
	// Using OnWidgetRebuilt instead of NativeConstruct to ensure we register ourselves with the ActionRouter before the child receives NativeConstruct
	if (!IsDesignTime())
	{
		// Clear out any invalid bindings before we bother trying to register them
		for (int32 BindingIdx = ActionBindings.Num() - 1; BindingIdx >= 0; --BindingIdx)
		{
			if (!ActionBindings[BindingIdx].IsValid())
			{
				ActionBindings.RemoveAt(BindingIdx);
			}
		}

		if (ActionBindings.Num() > 0)
		{
			if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
			{
				ActionRouter->NotifyUserWidgetConstructed(*this);
			}
		}
	}

	Super::OnWidgetRebuilt();

	if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
	{
		for (const TWeakObjectPtr<const UWidget>& WidgetPtr : GetScrollRecipients())
		{
			if (const UWidget* Widget = WidgetPtr.Get())
			{
				ActionRouter->RegisterScrollRecipient(*Widget);
			}
		}
	}
}

void UCommonUserWidget::NativeDestruct()
{
	if (ActionBindings.Num() > 0)
	{
		if (UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*this))
		{
			ActionRouter->NotifyUserWidgetDestructed(*this);
		}
	}

	Super::NativeDestruct();
}

#if WITH_EDITOR
const FText UCommonUserWidget::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

UWorld* UCommonUserWidget::GetWorld() const
{
	UWorld* World = Super::GetWorld();

	if (!World && (GEditor && !GEditor->PlayWorld) && !GIsPlayInEditorWorld)
	{
		return GWorld;
	}

	return World;
}
#endif // WITH_EDITOR
