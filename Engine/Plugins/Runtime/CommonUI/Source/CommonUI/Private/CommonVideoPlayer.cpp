// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonVideoPlayer.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "MediaPlaylist.h"
#include "MediaSource.h"
#include "UObject/ConstructorHelpers.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MediaSoundComponent.h"
#include "IMediaEventSink.h"
#include "Widgets/Images/SImage.h"
#include "CommonWidgetPaletteCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonVideoPlayer)

UCommonVideoPlayer::UCommonVideoPlayer(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	static ConstructorHelpers::FObjectFinder<UMaterial> VideoPlayerMaterialFinder(TEXT("/CommonUI/VideoPlayerMaterial"));
	VideoMaterial = VideoPlayerMaterialFinder.Object;
}

void UCommonVideoPlayer::PostInitProperties()
{
	Super::PostInitProperties();
	
	if (!IsTemplate())
	{
		MediaPlayer = NewObject<UMediaPlayer>(this);
		MediaPlayer->PlayOnOpen = false;
		MediaPlayer->OnMediaEvent().AddUObject(this, &UCommonVideoPlayer::HandleMediaPlayerEvent);

		MediaTexture = NewObject<UMediaTexture>(this);
		MediaTexture->AutoClear = true;
		MediaTexture->SetMediaPlayer(MediaPlayer);
		MediaTexture->UpdateResource();

		if (ensure(VideoMaterial))
		{
			UMaterialInstanceDynamic* VideoMID = UMaterialInstanceDynamic::Create(VideoMaterial, this);
			VideoMID->SetTextureParameterValue(TEXT("MediaTexture"), MediaTexture);
			VideoBrush.SetResourceObject(VideoMID);
		}

		SoundComponent = NewObject<UMediaSoundComponent>(this);
		SoundComponent->Channels = EMediaSoundChannels::Stereo;
		SoundComponent->bIsUISound = true;
		
		SoundComponent->SetMediaPlayer(MediaPlayer);
		SoundComponent->Initialize();
		SoundComponent->UpdatePlayer();
	}
}

void UCommonVideoPlayer::SetVideo(UMediaSource* NewVideo)
{
	if (MediaPlayer->GetPlaylist()->Get(0) != NewVideo)
	{
		MediaPlayer->Close();
		MediaPlayer->OpenSource(NewVideo);
	}
}

void UCommonVideoPlayer::Seek(float PlaybackTime)
{
	MediaPlayer->Seek(FTimespan::FromSeconds(PlaybackTime));
}

void UCommonVideoPlayer::Close()
{
	MediaPlayer->Close();
	SoundComponent->Stop();

	OnPlaybackComplete().Broadcast();
}

void UCommonVideoPlayer::SetPlaybackRate(float PlaybackRate)
{
	MediaPlayer->SetRate(PlaybackRate);
}

void UCommonVideoPlayer::SetLooping(bool bShouldLoopPlayback)
{
	MediaPlayer->SetLooping(bShouldLoopPlayback);
}

void UCommonVideoPlayer::SetIsMuted(bool bInIsMuted)
{
	bIsMuted = bInIsMuted;
	if (bIsMuted)
	{
		SoundComponent->Stop();
	}
	else if (IsPlaying())
	{
		SoundComponent->Start();
	}
}

void UCommonVideoPlayer::SetShouldMatchSize(bool bInMatchSize)
{
	if (bMatchSize != bInMatchSize)
	{
		bMatchSize = bInMatchSize;
		if (bMatchSize && MediaPlayer->IsReady())
		{
			VideoBrush.ImageSize = MediaPlayer->GetVideoTrackDimensions(INDEX_NONE, INDEX_NONE);
			if (MyImage)
			{
				MyImage->InvalidateImage();
			}
		}
	}
}

void UCommonVideoPlayer::Play()
{
	SetPlaybackRate(1.f);
}

void UCommonVideoPlayer::Reverse()
{
	SetPlaybackRate(-1.f);
}

void UCommonVideoPlayer::Pause()
{
	MediaPlayer->Pause();
}

void UCommonVideoPlayer::PlayFromStart()
{
	MediaPlayer->Rewind();
	Play();
}

float UCommonVideoPlayer::GetVideoDuration() const
{
	return MediaPlayer->GetDuration().GetTotalSeconds();
}

float UCommonVideoPlayer::GetPlaybackTime() const
{
	return MediaPlayer->GetTime().GetTotalSeconds();
}

float UCommonVideoPlayer::GetPlaybackRate() const
{
	return MediaPlayer->GetRate();
}

bool UCommonVideoPlayer::IsLooping() const
{
	return MediaPlayer->IsLooping();
}

bool UCommonVideoPlayer::IsPaused() const
{
	return MediaPlayer->IsPaused();
}

bool UCommonVideoPlayer::IsPlaying() const
{
	return MediaPlayer->IsPlaying();
}

TSharedRef<SWidget> UCommonVideoPlayer::RebuildWidget()
{
	return SAssignNew(MyImage, SImage)
		.Image(&VideoBrush);
}

void UCommonVideoPlayer::SynchronizeProperties()
{
	Super::SynchronizeProperties();
	
	if (MediaPlayer == nullptr)
	{
		return;
	}
	SetVideo(Video);
}

void UCommonVideoPlayer::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	MyImage.Reset();
}

void UCommonVideoPlayer::PlayInternal() const
{
	if (MediaPlayer->IsReady() && !MediaPlayer->IsPlaying())
	{
		MediaPlayer->Play();
	}
}

void UCommonVideoPlayer::HandleMediaPlayerEvent(EMediaEvent EventType)
{
	switch (EventType)
	{
	case EMediaEvent::TracksChanged:
	case EMediaEvent::MediaOpened:
		if (bMatchSize)
		{
			VideoBrush.ImageSize = MediaPlayer->GetVideoTrackDimensions(INDEX_NONE, INDEX_NONE);
			if (MyImage)
			{
				MyImage->InvalidateImage();
			}
		}
		break;
	case EMediaEvent::MediaClosed:
		Close();
		break;
	case EMediaEvent::PlaybackEndReached:
		if (!IsLooping())
		{
			SoundComponent->Stop();
			OnPlaybackComplete().Broadcast();
		}
		break;
	case EMediaEvent::PlaybackResumed:
		if (!bIsMuted)
		{
			SoundComponent->Start();
		}
		if (MyImage)
		{
			MyImage->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateUObject(this, &UCommonVideoPlayer::HandlePlaybackTick));
		}
		break;
	case EMediaEvent::PlaybackSuspended:
		SoundComponent->Stop();
		break;
	}
}

void UCommonVideoPlayer::PlaybackTick(double InCurrentTime, float InDeltaTime)
{
	if (!bIsMuted)
	{
		SoundComponent->UpdatePlayer();
	}
}

#if WITH_EDITOR
const FText UCommonVideoPlayer::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif // WITH_EDITOR

EActiveTimerReturnType UCommonVideoPlayer::HandlePlaybackTick(double InCurrentTime, float InDeltaTime)
{
	PlaybackTick(InCurrentTime, InDeltaTime);
	return MediaPlayer->IsPlaying() ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}
