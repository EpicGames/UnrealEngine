// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonVisibilitySwitcher.h"

#include "CommonActivatableWidget.h"
#include "CommonUIPrivate.h"
#include "CommonVisibilitySwitcherSlot.h"
#include "CommonWidgetPaletteCategories.h"
#include "Containers/StringConv.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonVisibilitySwitcher)

#if WITH_EDITOR
#include "Editor/WidgetCompilerLog.h"
#endif

void UCommonVisibilitySwitcher::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();

	ResetSlotVisibilities();
}

void UCommonVisibilitySwitcher::SetActiveWidgetIndex(int32 Index)
{
	if (ActiveWidgetIndex == Index)
	{
		UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - [%d] is already the active index, doing nothing", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), ActiveWidgetIndex);
		return;
	}

	SetActiveWidgetIndex_Internal(Index);
}

UWidget* UCommonVisibilitySwitcher::GetActiveWidget() const
{
	return GetChildAt(ActiveWidgetIndex);
}

void UCommonVisibilitySwitcher::SetActiveWidget(const UWidget* Widget)
{
	const int32 ChildIndex = GetChildIndex(Widget);

	if (ChildIndex == INDEX_NONE && Widget)
	{
		UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - [%ls] is not a child of the switcher, doing nothing", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), *Widget->GetName());
		return;
	}

	SetActiveWidgetIndex(ChildIndex);
}

void UCommonVisibilitySwitcher::IncrementActiveWidgetIndex(bool bAllowWrapping)
{
	int32 NewIndex = ActiveWidgetIndex + 1;

	if (NewIndex == Slots.Num())
	{
		if (bAllowWrapping)
		{
			NewIndex = 0;
		}
		else
		{
			return;
		}
	}

	SetActiveWidgetIndex(NewIndex);
}

void UCommonVisibilitySwitcher::DecrementActiveWidgetIndex(bool bAllowWrapping)
{
	int32 NewIndex = ActiveWidgetIndex - 1;

	if (NewIndex == -1)
	{
		if (bAllowWrapping)
		{
			NewIndex = Slots.Num() - 1;
		}
		else
		{
			return;
		}
	}

	SetActiveWidgetIndex(NewIndex);
}

void UCommonVisibilitySwitcher::ActivateVisibleSlot()
{
	if (UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(GetActiveWidget()))
	{
		UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - Activating [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), *ActivatableWidget->GetName());
		ActivatableWidget->ActivateWidget();
	}
}

void UCommonVisibilitySwitcher::DeactivateVisibleSlot()
{
	if (UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(GetActiveWidget()))
	{
		UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - Deactivating [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), *ActivatableWidget->GetName());
		ActivatableWidget->DeactivateWidget();
	}
}

UWidget* UCommonVisibilitySwitcher::GetWidgetAtIndex(int32 Index) const
{
	return Slots.IsValidIndex(Index) ? Slots[Index]->Content : nullptr;
}

void UCommonVisibilitySwitcher::SetActiveWidgetIndex_Internal(int32 Index, bool bBroadcastChange /*= true*/)
{
	TGuardValue<bool> bCurrentlySwitchingGuard(bCurrentlySwitching, true);

	if (Slots.IsValidIndex(ActiveWidgetIndex))
	{
		if (UCommonVisibilitySwitcherSlot* OldActiveSlot = Cast<UCommonVisibilitySwitcherSlot>(Slots[ActiveWidgetIndex]))
		{
			if (UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(OldActiveSlot->Content))
			{
				UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - Deactivating [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), *ActivatableWidget->GetName());
				ActivatableWidget->DeactivateWidget();
			}

			UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - Setting visibility of slot containing [%ls] to Collapsed", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), *OldActiveSlot->GetName());
			OldActiveSlot->SetSlotVisibility(ESlateVisibility::Collapsed);
		}
	}

	ActiveWidgetIndex = Index;

	if (Slots.IsValidIndex(ActiveWidgetIndex))
	{
		if (UCommonVisibilitySwitcherSlot* NewActiveSlot = Cast<UCommonVisibilitySwitcherSlot>(Slots[ActiveWidgetIndex]))
		{
			if (bAutoActivateSlot)
			{
				if (UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(NewActiveSlot->Content))
				{
					UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - Activating [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), *ActivatableWidget->GetName());
					ActivatableWidget->ActivateWidget();
				}
			}

			UE_LOGF(LogCommonUI, Verbose, "%ls [%ls] - Setting visibility of slot containing [%ls] to [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetName(), *NewActiveSlot->GetName(), *StaticEnum<ESlateVisibility>()->GetDisplayValueAsText(ShownVisibility).ToString());
			NewActiveSlot->SetSlotVisibility(ShownVisibility);
		}
	}

	if (bBroadcastChange)
	{
		OnActiveWidgetIndexChanged().Broadcast(ActiveWidgetIndex);
	}
}

void UCommonVisibilitySwitcher::ResetSlotVisibilities()
{
	for (UPanelSlot* PanelSlot : Slots)
	{
		if (UCommonVisibilitySwitcherSlot* SwitcherSlot = Cast<UCommonVisibilitySwitcherSlot>(PanelSlot))
		{
			SwitcherSlot->SetSlotVisibility(ESlateVisibility::Collapsed);
		}
	}

	const bool bBroadcastChange = false;
	SetActiveWidgetIndex_Internal(ActiveWidgetIndex, bBroadcastChange);
}

void UCommonVisibilitySwitcher::SynchronizeProperties()
{
	Super::SynchronizeProperties();

#if WITH_EDITOR
	if (IsDesignTime())
	{
		ResetSlotVisibilities();
	}
#endif
}

void UCommonVisibilitySwitcher::MoveChild(int32 CurrentIdx, int32 NewIdx)
{
	if (Slots.IsValidIndex(CurrentIdx) && Slots.IsValidIndex(NewIdx))
	{
		UPanelSlot* PanelSlot = Slots[CurrentIdx];
		Slots.RemoveAt(CurrentIdx);
		Slots.Insert(PanelSlot, NewIdx);
	}
}

#if WITH_EDITOR
const FText UCommonVisibilitySwitcher::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

void UCommonVisibilitySwitcher::OnDescendantSelectedByDesigner(UWidget* DescendantWidget)
{
	if (UWidget* SelectedChild = UWidget::FindChildContainingDescendant(this, DescendantWidget))
	{
		const int32 OverrideIndex = GetChildIndex(SelectedChild);
		// Store active index so we can restore it when descendant is deselected
		DesignTime_ActiveIndex = ActiveWidgetIndex;
		SetActiveWidgetIndex_Internal(OverrideIndex);
	}
}

void UCommonVisibilitySwitcher::OnDescendantDeselectedByDesigner(UWidget* DescendantWidget)
{
	SetActiveWidgetIndex_Internal(DesignTime_ActiveIndex);
}

void UCommonVisibilitySwitcher::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	ActiveWidgetIndex = FMath::Clamp(ActiveWidgetIndex, -1, FMath::Max(0, Slots.Num() - 1));

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UCommonVisibilitySwitcher::ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const
{
	Super::ValidateCompiledDefaults(CompileLog);

	const TArray<UWidget*>& AllChildren = GetAllChildren();
	for (UWidget* Child : AllChildren)
	{
		if (UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(Child))
		{
			if (ActivatableWidget->SetsVisibilityOnActivated())
			{
				CompileLog.Error(FText::Format(FText::FromString("{0} has bSetVisibilityOnActivated enabled but is in a switcher. Its visibility will be set by the switcher."), FText::FromString(ActivatableWidget->GetName())));
			}

			if (ActivatableWidget->SetsVisibilityOnDeactivated())
			{
				CompileLog.Error(FText::Format(FText::FromString("{0} has bSetVisibilityOnDeactivated enabled but is in a switcher. Its visibility will be set by the switcher."), FText::FromString(ActivatableWidget->GetName())));
			}
		}
	}
}
#endif

UClass* UCommonVisibilitySwitcher::GetSlotClass() const
{
	return UCommonVisibilitySwitcherSlot::StaticClass();
}

void UCommonVisibilitySwitcher::OnSlotAdded(UPanelSlot* InSlot)
{
	if (MyOverlay.IsValid())
	{
		if (UCommonVisibilitySwitcherSlot* SwitcherSlot = CastChecked<UCommonVisibilitySwitcherSlot>(InSlot))
		{
			SwitcherSlot->BuildSlot(MyOverlay.ToSharedRef());
			SwitcherSlot->SetSlotVisibility(ESlateVisibility::Collapsed);

			if (Slots.Num() == 1 && bActivateFirstSlotOnAdding)
			{
				SetActiveWidgetIndex(0);
			}
		}
	}
}

void UCommonVisibilitySwitcher::OnSlotRemoved(UPanelSlot* InSlot)
{
	if (MyOverlay.IsValid())
	{
		UCommonVisibilitySwitcherSlot* SwitcherSlot = CastChecked<UCommonVisibilitySwitcherSlot>(InSlot);
		const TSharedPtr<SBox>& VisibilityBox = SwitcherSlot ? SwitcherSlot->GetVisibilityBox() : nullptr;
		if (VisibilityBox.IsValid())
		{
			MyOverlay->RemoveSlot(StaticCastSharedRef<SWidget>(VisibilityBox.ToSharedRef()));
		}
	}
}

