// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonVisibilitySwitcherSlot.h"

#include "Components/Widget.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonVisibilitySwitcherSlot)

UCommonVisibilitySwitcherSlot::UCommonVisibilitySwitcherSlot(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	SetHorizontalAlignment(EHorizontalAlignment::HAlign_Fill);
	SetVerticalAlignment(EVerticalAlignment::VAlign_Fill);
}

void UCommonVisibilitySwitcherSlot::BuildSlot(TSharedRef<SOverlay> Overlay)
{
	Overlay->AddSlot()
		.Expose(Slot)
		.Padding(GetPadding())
		.HAlign(GetHorizontalAlignment())
		.VAlign(GetVerticalAlignment())
		[
			SAssignNew(VisibilityBox, SBox)
			[
				Content ? Content->TakeWidget() : SNullWidget::NullWidget
			]
		];
}

void UCommonVisibilitySwitcherSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	VisibilityBox.Reset();
}

void UCommonVisibilitySwitcherSlot::SetSlotVisibility(ESlateVisibility Visibility)
{
	if (SBox* Box = VisibilityBox.Get())
	{
		Box->SetVisibility(UWidget::ConvertSerializedVisibilityToRuntime(Visibility));
	}
}

