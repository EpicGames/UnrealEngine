// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonVisibilityWidgetBase.h"
#include "CommonInputSubsystem.h"
#include "CommonUIPrivate.h"

#include "CommonInputBaseTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonVisibilityWidgetBase)

UDEPRECATED_UCommonVisibilityWidgetBase::UDEPRECATED_UCommonVisibilityWidgetBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShowForGamepad(true)
	, bShowForMouseAndKeyboard(true)
	, bShowForTouch(true)
	, VisibleType(ESlateVisibility::SelfHitTestInvisible)
	, HiddenType(ESlateVisibility::Collapsed)
{
	for (const FName& RegisteredPlatform : FCommonInputPlatformBaseData::GetRegisteredPlatforms())
	{
		VisibilityControls.Add(RegisteredPlatform, false);
	}
}

void UDEPRECATED_UCommonVisibilityWidgetBase::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();
	UpdateVisibility();
	ListenToInputMethodChanged();
}

void UDEPRECATED_UCommonVisibilityWidgetBase::UpdateVisibility()
{
	if (!IsDesignTime())
	{
		const UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(GetOwningLocalPlayer());
		if (ensure(CommonInputSubsystem))
		{
			bool bVisibleForInput = bShowForMouseAndKeyboard;

			if (CommonInputSubsystem->IsInputMethodActive(ECommonInputType::Gamepad))
			{
				bVisibleForInput = bShowForGamepad;
			}
			else if (CommonInputSubsystem->IsInputMethodActive(ECommonInputType::Touch))
			{
				bVisibleForInput = bShowForTouch;
			}

			if (bool* bVisibleForPlatform = VisibilityControls.Find(FCommonInputBase::GetCurrentPlatformName()))
			{
				SetVisibility(*bVisibleForPlatform && bVisibleForInput ? VisibleType : HiddenType);
			}
			else
			{
				// Current setup assumes all platforms should have a value, so log if we didn't get a hit.
				UE_LOGF(LogCommonUI, Warning, "Invalid platform: '%ls' used to control visibility.", *FCommonInputBase::GetCurrentPlatformName().ToString())
				SetVisibility(bVisibleForInput ? VisibleType : HiddenType);
			}
		}
	}
}

void UDEPRECATED_UCommonVisibilityWidgetBase::ListenToInputMethodChanged(bool bListen)
{
	if (IsDesignTime())
	{
		return;
	}

	if (UCommonInputSubsystem* CommonInputSubsystem = UCommonInputSubsystem::Get(GetOwningLocalPlayer()))
	{
		CommonInputSubsystem->OnInputMethodChangedNative.RemoveAll(this);
		if (bListen)
		{
			CommonInputSubsystem->OnInputMethodChangedNative.AddUObject(this, &ThisClass::HandleInputMethodChanged);
		}
	}
}

void UDEPRECATED_UCommonVisibilityWidgetBase::HandleInputMethodChanged(ECommonInputType input)
{
	UpdateVisibility();
}

const TArray<FName>& UDEPRECATED_UCommonVisibilityWidgetBase::GetRegisteredPlatforms()
{
	return FCommonInputPlatformBaseData::GetRegisteredPlatforms();
}

