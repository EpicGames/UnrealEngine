// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonVisualAttachment.h"
#include "SVisualAttachmentBox.h"
#include "Components/SizeBoxSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonVisualAttachment)

UCommonVisualAttachment::UCommonVisualAttachment(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


FVector2D UCommonVisualAttachment::GetContentAnchor() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ContentAnchor;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UCommonVisualAttachment::SetContentAnchor(FVector2D InContentAnchor)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ContentAnchor = InContentAnchor;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (MyAttachmentBox)
	{
		MyAttachmentBox->SetContentAnchor(GetContentAnchor());
	}
}

void UCommonVisualAttachment::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyAttachmentBox.Reset();
}

TSharedRef<SWidget> UCommonVisualAttachment::RebuildWidget()
{
	MyAttachmentBox = SNew(SVisualAttachmentBox);
	MySizeBox = MyAttachmentBox;

	if (GetChildrenCount() > 0)
	{
		Cast<USizeBoxSlot>(GetContentSlot())->BuildSlot(MySizeBox.ToSharedRef());
	}

	return MySizeBox.ToSharedRef();
}

void UCommonVisualAttachment::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyAttachmentBox)
	{
		MyAttachmentBox->SetContentAnchor(GetContentAnchor());
	}
}