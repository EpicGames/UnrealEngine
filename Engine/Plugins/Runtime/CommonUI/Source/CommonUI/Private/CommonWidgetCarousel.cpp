// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonWidgetCarousel.h"
#include "CommonWidgetPaletteCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonWidgetCarousel)

UCommonWidgetCarousel::UCommonWidgetCarousel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = true;
	SetClipping(EWidgetClipping::ClipToBounds);

	MoveSpeed = 5.f;
	bCacheChildren = true;
}

void UCommonWidgetCarousel::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyCommonWidgetCarousel.Reset();
	CachedSlotWidgets.Empty();

	EndAutoScrolling();
}

void UCommonWidgetCarousel::BeginAutoScrolling(float ScrollInterval)
{
	EndAutoScrolling();
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCommonWidgetCarousel::AutoScrollCallback), ScrollInterval);
}

void UCommonWidgetCarousel::EndAutoScrolling()
{
	if ( TickerHandle.IsValid() )
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}
}

bool UCommonWidgetCarousel::AutoScrollCallback(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_UCommonWidgetCarousel_AutoScrollCallback);

	if ( MyCommonWidgetCarousel.IsValid() )
	{
		MyCommonWidgetCarousel->SetPreviousWidget();
	}

	return true;
}

void UCommonWidgetCarousel::NextPage()
{
	if ( MyCommonWidgetCarousel.IsValid() && Slots.Num() > 1 )
	{
		MyCommonWidgetCarousel->SetNextWidget();
	}
}

void UCommonWidgetCarousel::PreviousPage()
{
	if ( MyCommonWidgetCarousel.IsValid() && Slots.Num() > 1 )
	{
		MyCommonWidgetCarousel->SetPreviousWidget();
	}
}

void UCommonWidgetCarousel::SetMoveSpeed(float InMoveSpeed)
{
	MoveSpeed = InMoveSpeed;
	if (MoveSpeed != InMoveSpeed)
	{
		if (MyCommonWidgetCarousel.IsValid())
		{
			MyCommonWidgetCarousel->SetMoveSpeed(MoveSpeed);
		}
	}
}

float UCommonWidgetCarousel::GetMoveSpeed() const
{
	if (MyCommonWidgetCarousel.IsValid())
	{
		return MyCommonWidgetCarousel->GetMoveSpeed();
	}

	return MoveSpeed;
}

void UCommonWidgetCarousel::SetCacheChildren(bool InCacheChildren)
{
	if (bCacheChildren != InCacheChildren)
	{
		bCacheChildren = InCacheChildren;
		if (bCacheChildren)
		{
			for (UPanelSlot* PanelSlot : Slots)
			{
				if (PanelSlot->Content)
				{
					CachedSlotWidgets.AddUnique(PanelSlot->Content->TakeWidget());
				}
			}
		}
		else
		{
			CachedSlotWidgets.Empty();
		}
	}
}

bool UCommonWidgetCarousel::GetCacheChildren() const
{
	return bCacheChildren;
}

int32 UCommonWidgetCarousel::GetActiveWidgetIndex() const
{
	if ( MyCommonWidgetCarousel.IsValid() )
	{
		return MyCommonWidgetCarousel->GetWidgetIndex();
	}

	return ActiveWidgetIndex;
}

void UCommonWidgetCarousel::SetActiveWidgetIndex(int32 Index)
{
	ActiveWidgetIndex = Index;
	if ( MyCommonWidgetCarousel.IsValid() )
	{
		// Ensure the index is clamped to a valid range.
		int32 SafeIndex = FMath::Clamp(ActiveWidgetIndex, 0, FMath::Max(0, Slots.Num() - 1));
		MyCommonWidgetCarousel->SetActiveWidgetIndex(SafeIndex);
	}
}

void UCommonWidgetCarousel::SetActiveWidget(UWidget* Widget)
{
	ActiveWidgetIndex = GetChildIndex(Widget);
	if ( MyCommonWidgetCarousel.IsValid() )
	{
		// Ensure the index is clamped to a valid range.
		int32 SafeIndex = FMath::Clamp(ActiveWidgetIndex, 0, FMath::Max(0, Slots.Num() - 1));
		MyCommonWidgetCarousel->SetActiveWidgetIndex(SafeIndex);
	}
}

UWidget* UCommonWidgetCarousel::GetWidgetAtIndex( int32 Index ) const
{
	if ( Slots.IsValidIndex( Index ) )
	{
		return Slots[ Index ]->Content;
	}

	return nullptr;
}

UClass* UCommonWidgetCarousel::GetSlotClass() const
{
	return UPanelSlot::StaticClass();
}

void UCommonWidgetCarousel::OnSlotAdded(UPanelSlot* InSlot)
{
	if (bCacheChildren && InSlot && InSlot->Content)
	{
		const TSharedPtr<SWidget> SafeWidget = InSlot->Content->TakeWidget();
		if (SafeWidget.IsValid())
		{
			CachedSlotWidgets.AddUnique(SafeWidget.ToSharedRef());
		}
	}

	if (MyCommonWidgetCarousel)
	{
		MyCommonWidgetCarousel->GenerateCurrentWidgets();
	}
}

void UCommonWidgetCarousel::OnSlotRemoved(UPanelSlot* InSlot)
{
	if (bCacheChildren && InSlot && InSlot->Content)
	{
		const TSharedPtr<SWidget> SafeWidget = InSlot->Content->GetCachedWidget();
		if (SafeWidget.IsValid())
		{
			CachedSlotWidgets.Remove(SafeWidget.ToSharedRef());
		}
	}
}

TSharedRef<SWidget> UCommonWidgetCarousel::RebuildWidget()
{
	MyCommonWidgetCarousel = SNew(SWidgetCarousel<UPanelSlot*>)
		.WidgetItemsSource(&ToRawPtrTArrayUnsafe(Slots))
		.FadeRate(0)
		.SlideValueLeftLimit(-1)
		.SlideValueRightLimit(1)
		.MoveSpeed(MoveSpeed)
		.OnGenerateWidget_UObject(this, &UCommonWidgetCarousel::OnGenerateWidgetForCarousel)
		.OnPageChanged_UObject(this, &UCommonWidgetCarousel::HandlePageChanged);

	CachedSlotWidgets.Empty();

	for (UPanelSlot* PanelSlot : Slots)
	{
		PanelSlot->Parent = this;
		if (bCacheChildren && PanelSlot->Content)
		{
			CachedSlotWidgets.AddUnique(PanelSlot->Content->TakeWidget());
		}
	}

	return MyCommonWidgetCarousel.ToSharedRef();
}

TSharedRef<SWidget> UCommonWidgetCarousel::OnGenerateWidgetForCarousel(UPanelSlot* PanelSlot)
{
	if ( UWidget* Content = PanelSlot->Content )
	{
		return Content->TakeWidget();
	}

	return SNullWidget::NullWidget;
}

void UCommonWidgetCarousel::HandlePageChanged(int32 PageIndex)
{
	if (!IsDesignTime())
	{
		OnCurrentPageIndexChanged.Broadcast(this, PageIndex);
	}
}

void UCommonWidgetCarousel::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	SetActiveWidgetIndex(ActiveWidgetIndex);
}

#if WITH_EDITOR

const FText UCommonWidgetCarousel::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

void UCommonWidgetCarousel::OnDescendantSelectedByDesigner(UWidget* DescendantWidget)
{
	// Temporarily sets the active child to the selected child to make
	// dragging and dropping easier in the editor.
	UWidget* SelectedChild = UWidget::FindChildContainingDescendant(this, DescendantWidget);
	if ( SelectedChild )
	{
		int32 OverrideIndex = GetChildIndex(SelectedChild);
		if ( OverrideIndex != INDEX_NONE && MyCommonWidgetCarousel.IsValid() )
		{
			// Need to cancel transition if it's active to ensure the selected widget actually displays if there's a transition in progress
			MyCommonWidgetCarousel->ResetTransition();
			MyCommonWidgetCarousel->SetActiveWidgetIndex(OverrideIndex);
		}
	}
}

void UCommonWidgetCarousel::OnDescendantDeselectedByDesigner(UWidget* DescendantWidget)
{
	if (MyCommonWidgetCarousel.IsValid())
	{
		// Need to cancel transition if it's active to ensure the active widget actually displays if there's a transition in progress
		MyCommonWidgetCarousel->ResetTransition();
	}

	SetActiveWidgetIndex(ActiveWidgetIndex);
}

void UCommonWidgetCarousel::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	ActiveWidgetIndex = FMath::Clamp(ActiveWidgetIndex, 0, FMath::Max(0, Slots.Num() - 1));

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif
