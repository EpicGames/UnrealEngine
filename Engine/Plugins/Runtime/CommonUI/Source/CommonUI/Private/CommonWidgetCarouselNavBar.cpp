// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonWidgetCarouselNavBar.h"
#include "CommonWidgetPaletteCategories.h"
#include "CommonWidgetCarousel.h"
#include "Groups/CommonButtonGroupBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonWidgetCarouselNavBar)

UCommonWidgetCarouselNavBar::UCommonWidgetCarouselNavBar(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ButtonPadding(0)
{
}

void UCommonWidgetCarouselNavBar::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyContainer.Reset();
}

void UCommonWidgetCarouselNavBar::SetLinkedCarousel(UCommonWidgetCarousel* CommonCarousel)
{
	if (LinkedCarousel)
	{
		LinkedCarousel->OnCurrentPageIndexChanged.RemoveAll(this);
	}

	LinkedCarousel = CommonCarousel;

	if (LinkedCarousel)
	{
		LinkedCarousel->OnCurrentPageIndexChanged.AddDynamic(this, &UCommonWidgetCarouselNavBar::HandlePageChanged);
	}

	RebuildButtons();
}

void UCommonWidgetCarouselNavBar::RebuildButtons()
{
	ButtonGroup->RemoveAll();
	ButtonGroup->OnSelectedButtonBaseChanged.RemoveAll(this);

	if (ensure(MyContainer) && LinkedCarousel)
	{
		MyContainer->ClearChildren();

		const int32 NumPages = LinkedCarousel->GetChildrenCount();
		for (int32 CurPage = 0; CurPage < NumPages; CurPage++)
		{
			UCommonButtonBase* ButtonUserWidget = Cast<UCommonButtonBase>(CreateWidget(GetOwningPlayer(), ButtonWidgetType));
			if (ensure(ButtonUserWidget))
			{
				Buttons.Add(ButtonUserWidget);
				ButtonGroup->AddWidget(ButtonUserWidget);

				TSharedRef<SWidget> ButtonSWidget = ButtonUserWidget->TakeWidget();
				MyContainer->AddSlot()
					.Padding(ButtonPadding)
					[
						ButtonSWidget
					];
			}
		}

		if (NumPages > 0)
		{
			ButtonGroup->SelectButtonAtIndex(LinkedCarousel->GetActiveWidgetIndex());
			
			ButtonGroup->OnButtonBaseClicked.AddDynamic(this, &UCommonWidgetCarouselNavBar::HandleButtonClicked);
		}
	}
}

TSharedRef<SWidget> UCommonWidgetCarouselNavBar::RebuildWidget()
{
	ButtonGroup = NewObject<UCommonButtonGroupBase>();
	ButtonGroup->SetSelectionRequired(true);
	
	MyContainer = SNew(SHorizontalBox);
	
	return MyContainer.ToSharedRef();
}

void UCommonWidgetCarouselNavBar::HandlePageChanged(UCommonWidgetCarousel* CommonCarousel, int32 PageIndex)
{
	if (ButtonGroup && ButtonGroup->GetSelectedButtonIndex() != PageIndex)
	{
		ButtonGroup->SelectButtonAtIndex(PageIndex);
	}
}

void UCommonWidgetCarouselNavBar::HandleButtonClicked(UCommonButtonBase* AssociatedButton, int32 ButtonIndex)
{
	if (LinkedCarousel && LinkedCarousel->GetActiveWidgetIndex() != ButtonIndex)
	{
		LinkedCarousel->EndAutoScrolling();
		LinkedCarousel->SetActiveWidgetIndex(ButtonIndex);
	}
}

#if WITH_EDITOR

const FText UCommonWidgetCarouselNavBar::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}

#endif
