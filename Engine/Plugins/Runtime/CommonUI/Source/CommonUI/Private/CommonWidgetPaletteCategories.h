// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"

namespace CommonWidgetPaletteCategories
{
	const FText Default = NSLOCTEXT("CommonUI", "WidgetPaletteCategory", "Common UI Plugin");
}