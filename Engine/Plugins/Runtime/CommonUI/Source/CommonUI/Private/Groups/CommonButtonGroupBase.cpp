// Copyright Epic Games, Inc. All Rights Reserved.

#include "Groups/CommonButtonGroupBase.h"
#include <memory>
#include <tuple>

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonButtonGroupBase)

#include <functional>

UCommonButtonGroupBase::UCommonButtonGroupBase()
	: bSelectionRequired(false)
	, SelectedButtonIndex(INDEX_NONE)
{
}

void UCommonButtonGroupBase::SetSelectionRequired(bool bRequireSelection)
{
	if (bSelectionRequired != bRequireSelection)
	{
		bSelectionRequired = bRequireSelection;
		
		if (bSelectionRequired && 
			Buttons.Num() > 0 && 
			SelectedButtonIndex < 0)
		{
			// Selection is now required and nothing is selected, so select the first button
			SelectButtonAtIndex(0);
		}
	}
}

bool UCommonButtonGroupBase::GetSelectionRequired() const
{
	return bSelectionRequired;
}

void UCommonButtonGroupBase::DeselectAll()
{
	ensureMsgf(!bSelectionRequired, TEXT("BaseButton Groups that require selection should not deselect all."));

	for (auto& Button : Buttons)
	{
		if (Button.IsValid() && Button->GetSelected())
		{
			Button->SetSelectedInternal(false);
		}
	}

	if (SelectedButtonIndex != INDEX_NONE)
	{
		SelectedButtonIndex = INDEX_NONE;
		OnSelectionCleared.Broadcast();
	}
}

void UCommonButtonGroupBase::SelectNextButton(bool bAllowWrap /*= true*/, const bool bAllowSound /*= true*/)
{
	const int32 NumButtons = Buttons.Num();
	int32 NumRecursiveSelections = 0;

	std::function<void(int32, bool)> SelectNextButtonRecursive = [&](int32 SelectionIndex, bool bAllowWrap)
	{
		// We have recursively gone through every button in the group. Stop.
		if (NumRecursiveSelections > NumButtons)
		{
			return;
		}

		if (SelectionIndex < (NumButtons - 1))
		{
			++SelectionIndex;
		}
		else if (bAllowWrap)
		{
			SelectionIndex = 0;
		}
		else
		{
			// We are at the last button and aren't wrapping, so there isn't a next button. Stop.
			return;
		}

		if (Buttons.IsValidIndex(SelectionIndex))
		{
			UCommonButtonBase* NextButton = GetButtonBaseAtIndex(SelectionIndex);
			if (NextButton && NextButton->IsInteractionEnabled())
			{
				SelectButtonAtIndex(SelectionIndex, bAllowSound);
			}
			else
			{
				NumRecursiveSelections++;
				SelectNextButtonRecursive(SelectionIndex, bAllowWrap);
			}
		}
	};

	SelectNextButtonRecursive(SelectedButtonIndex, bAllowWrap);
}

void UCommonButtonGroupBase::SelectPreviousButton(bool bAllowWrap /*= true*/, const bool bAllowSound /*= true*/)
{
	const int32 NumButtons = Buttons.Num();
	int32 NumRecursiveSelections = 0;

	std::function<void(int32, bool)> SelectPrevButtonRecursive = [&](int32 SelectionIndex, bool bAllowWrap)
	{
		// We have recursively gone through every button in the group. Stop.
		if (NumRecursiveSelections > NumButtons)
		{
			return;
		}

		if (SelectionIndex > 0)
		{
			--SelectionIndex;
		}
		else if (bAllowWrap)
		{
			SelectionIndex = NumButtons - 1;
		}
		else
		{
			// We are at the last button and aren't wrapping, so there isn't a next button. Stop.
			return;
		}

		if (Buttons.IsValidIndex(SelectionIndex))
		{
			UCommonButtonBase* PrevButton = GetButtonBaseAtIndex(SelectionIndex);
			if (PrevButton && PrevButton->IsInteractionEnabled())
			{
				SelectButtonAtIndex(SelectionIndex, bAllowSound);
			}
			else
			{
				NumRecursiveSelections++;
				SelectPrevButtonRecursive(SelectionIndex, bAllowWrap);
			}
		}
	};

	SelectPrevButtonRecursive(SelectedButtonIndex, bAllowWrap);
}

void UCommonButtonGroupBase::SelectButtonAtIndex(int32 ButtonIndex, const bool bAllowSound)
{
	if (ButtonIndex < 0 || ButtonIndex >= Buttons.Num())
	{
		DeselectAll();
	}
	else if (ButtonIndex != SelectedButtonIndex)
	{
		UCommonButtonBase* ButtonToSelect = GetButtonBaseAtIndex(ButtonIndex);
		if (ButtonToSelect && !ButtonToSelect->GetSelected() && ButtonToSelect->GetIsEnabled())
		{
			SelectedButtonIndex = ButtonIndex;
			ButtonToSelect->SetSelectedInternal(true, bAllowSound);
		}
	}
}

int32 UCommonButtonGroupBase::GetSelectedButtonIndex() const
{
	return SelectedButtonIndex;
}

int32 UCommonButtonGroupBase::GetHoveredButtonIndex() const
{
	return HoveredButtonIndex;
}

int32 UCommonButtonGroupBase::FindButtonIndex(const UCommonButtonBase* ButtonToFind) const
{
	return Buttons.IndexOfByPredicate([ButtonToFind](const TWeakObjectPtr<UCommonButtonBase>& Element)
	{
		return (Element.Get() == ButtonToFind);
	});
}

void UCommonButtonGroupBase::ForEach(TFunctionRef<void(UCommonButtonBase&, int32)> Functor)
{
	for (int32 ButtonIndex = 0; ButtonIndex < Buttons.Num(); ButtonIndex++)
	{
		UCommonButtonBase* Button = GetButtonBaseAtIndex(ButtonIndex);
		if (Button)
		{
			Functor(*Button, ButtonIndex);
		}
	}
}

bool UCommonButtonGroupBase::HasAnyButtons() const
{
	return Buttons.Num() > 0;
}

int32 UCommonButtonGroupBase::GetButtonCount() const
{
	return Buttons.Num();
}

void UCommonButtonGroupBase::OnWidgetAdded(UWidget* NewWidget)
{
	if (UCommonButtonBase* Button = Cast<UCommonButtonBase>(NewWidget))
	{
		Button->OnSelectedChangedBase.AddUniqueDynamic(this, &UCommonButtonGroupBase::OnSelectionStateChangedBase);
		Button->OnButtonBaseClicked.AddUniqueDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseClicked);
		Button->OnButtonBaseDoubleClicked.AddUniqueDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseDoubleClicked);
		Button->OnButtonBaseHovered.AddUniqueDynamic(this, &UCommonButtonGroupBase::OnButtonBaseHovered);
		Button->OnButtonBaseUnhovered.AddUniqueDynamic(this, &UCommonButtonGroupBase::OnButtonBaseUnhovered);
		Button->OnButtonBaseLockClicked.AddUniqueDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseLockClicked);
		Button->OnButtonBaseLockDoubleClicked.AddUniqueDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseLockDoubleClicked);

		Buttons.Emplace(Button);

		// If selection in the group is required and this is the first button added, make sure it's selected (quietly)
		if (bSelectionRequired && Buttons.Num() == 1 && !Button->GetSelected())
		{
			Button->SetSelectedInternal(true, false);
		}
	}
}

void UCommonButtonGroupBase::OnWidgetRemoved( UWidget* OldWidget )
{
	if (UCommonButtonBase* Button = Cast<UCommonButtonBase>(OldWidget))
	{
		const int32 ButtonIndex = FindButtonIndex(Button);

		Button->OnSelectedChangedBase.RemoveDynamic(this, &UCommonButtonGroupBase::OnSelectionStateChangedBase);
		Button->OnButtonBaseClicked.RemoveDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseClicked);
		Button->OnButtonBaseDoubleClicked.RemoveDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseDoubleClicked);
		Button->OnButtonBaseHovered.RemoveDynamic(this, &UCommonButtonGroupBase::OnButtonBaseHovered);
		Button->OnButtonBaseUnhovered.RemoveDynamic(this, &UCommonButtonGroupBase::OnButtonBaseUnhovered);
		Button->OnButtonBaseLockClicked.RemoveDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseLockClicked);
		Button->OnButtonBaseLockDoubleClicked.RemoveDynamic(this, &UCommonButtonGroupBase::OnHandleButtonBaseLockDoubleClicked);

		Buttons.RemoveAll( [Button]( TWeakObjectPtr<UCommonButtonBase> Entry ) { return Entry == Button || !Entry.IsValid(); } );

		if (Button->GetSelected())
		{
			constexpr bool bAllowSound = false;
			Button->SetSelectedInternal(false, bAllowSound);
		}

		if (ButtonIndex == SelectedButtonIndex)
		{
			SelectedButtonIndex = INDEX_NONE;
			
			if (bSelectionRequired && Buttons.Num() > 0)
			{
				for (int32 NewButtonIndex = 0; NewButtonIndex < Buttons.Num(); NewButtonIndex++)
				{
					UCommonButtonBase* NewButtonToSelect = GetButtonBaseAtIndex(NewButtonIndex);
					if (NewButtonToSelect && NewButtonToSelect->bSelectable)
					{
						SelectButtonAtIndex(NewButtonIndex);
						return; // Early out to only select one button
					}
				}
				ensureMsgf(false, TEXT("Button group requires selection, but no button is selectable"));
			}
			else
			{
				OnSelectionCleared.Broadcast();
			}
		}
	}
}

void UCommonButtonGroupBase::OnRemoveAll()
{
	// Set the selected button index to none before removing each widget so we don't
	// end up with a case of the selection change event triggering multiple times when removing all.
	SelectedButtonIndex = INDEX_NONE;

	{
		// Even if we require selection, allow for it be ignored when removing all buttons (Ex: Destructor).
		TGuardValue<bool> DisableSelectionRequiredGuard(bSelectionRequired, false);

		while (Buttons.Num() > 0)
		{
			if (UCommonButtonBase* Button = Buttons.Pop().Get())
			{
				RemoveWidget(Button);
			}
		}
	}
	
	OnSelectionCleared.Broadcast();
}

void UCommonButtonGroupBase::OnSelectionStateChangedBase( UCommonButtonBase* BaseButton, bool bIsSelected )
{
	UCommonButtonBase* PreviouslySelectedButton = GetButtonBaseAtIndex(SelectedButtonIndex);

	if (bIsSelected)
	{
		// Reset index to avoid stale leakage
		SelectedButtonIndex = INDEX_NONE;

		bool bFoundButton = false;

		for (int32 ButtonIndex = 0; ButtonIndex < Buttons.Num(); ButtonIndex++)
		{
			UCommonButtonBase* Button = GetButtonBaseAtIndex(ButtonIndex);
			if (Button)
			{
				if (Button == BaseButton)
				{
					SelectedButtonIndex = ButtonIndex;
					bFoundButton = true;
				}
				else if (Button->GetSelected())
				{
					// Make sure any other selected buttons are deselected
					Button->SetSelectedInternal(false);
				}
			}
		}

		// If the button isn't in the group, do not broadcast a stale state
		if (!bFoundButton)
		{
			return;
		}

		NativeOnSelectedButtonBaseChanged.Broadcast(BaseButton, SelectedButtonIndex);
		OnSelectedButtonBaseChanged.Broadcast(BaseButton, SelectedButtonIndex);		
		OnSelectedButtonLostSelection.Broadcast(PreviouslySelectedButton);
	}
	else
	{
		for ( auto& WeakButton : Buttons )
		{
			if ( WeakButton.IsValid() && WeakButton->GetSelected() )
			{
				return; // early out here and prevent the delegate below from being triggered
			}
		}

		PreviouslySelectedButton = GetButtonBaseAtIndex(SelectedButtonIndex);
		OnSelectedButtonLostSelection.Broadcast(PreviouslySelectedButton);

		SelectedButtonIndex = INDEX_NONE;

		NativeOnSelectionCleared.Broadcast();
		OnSelectionCleared.Broadcast();
	}
}

void UCommonButtonGroupBase::OnButtonBaseHovered(UCommonButtonBase* BaseButton)
{
	HoveredButtonIndex = FindButtonIndex(BaseButton);
	if (HoveredButtonIndex != INDEX_NONE)
	{
		NativeOnHoveredButtonBaseChanged.Broadcast(BaseButton, HoveredButtonIndex);
		OnHoveredButtonBaseChanged.Broadcast(BaseButton, HoveredButtonIndex);
	}
}

void UCommonButtonGroupBase::OnButtonBaseUnhovered(UCommonButtonBase* BaseButton)
{
	if (HoveredButtonIndex != INDEX_NONE && GetButtonBaseAtIndex(HoveredButtonIndex) == BaseButton)
	{
		UCommonButtonBase* EmptyButton = nullptr;
		HoveredButtonIndex = INDEX_NONE;
		NativeOnHoveredButtonBaseChanged.Broadcast(EmptyButton, HoveredButtonIndex);
		OnHoveredButtonBaseChanged.Broadcast(EmptyButton, HoveredButtonIndex);
	}
}

void UCommonButtonGroupBase::OnHandleButtonBaseClicked(UCommonButtonBase* BaseButton)
{
	const int32 ClickedIdx = FindButtonIndex(BaseButton);
	if (ClickedIdx != INDEX_NONE)
	{
		NativeOnButtonBaseClicked.Broadcast(BaseButton, ClickedIdx);
		OnButtonBaseClicked.Broadcast(BaseButton, ClickedIdx);
	}
}

void UCommonButtonGroupBase::OnHandleButtonBaseDoubleClicked(UCommonButtonBase* BaseButton)
{
	const int32 ClickedIdx = FindButtonIndex(BaseButton);
	if (ClickedIdx != INDEX_NONE)
	{
		NativeOnButtonBaseDoubleClicked.Broadcast(BaseButton, ClickedIdx);
		OnButtonBaseDoubleClicked.Broadcast(BaseButton, ClickedIdx);
	}
}

void UCommonButtonGroupBase::OnHandleButtonBaseLockClicked(UCommonButtonBase* BaseButton)
{
	const int32 ClickedIdx = FindButtonIndex(BaseButton);
	if (ClickedIdx != INDEX_NONE)
	{
		NativeOnButtonBaseLockClicked.Broadcast(BaseButton, ClickedIdx);
		OnButtonBaseLockClicked.Broadcast(BaseButton, ClickedIdx);
	}
}

void UCommonButtonGroupBase::OnHandleButtonBaseLockDoubleClicked(UCommonButtonBase* BaseButton)
{
	const int32 ClickedIdx = FindButtonIndex(BaseButton);
	if (ClickedIdx != INDEX_NONE)
	{
		NativeOnButtonBaseLockDoubleClicked.Broadcast(BaseButton, ClickedIdx);
		OnButtonBaseLockDoubleClicked.Broadcast(BaseButton, ClickedIdx);
	}
}

UCommonButtonBase* UCommonButtonGroupBase::GetButtonBaseAtIndex(int32 Index) const
{
	if (Buttons.IsValidIndex(Index))
	{
		return Buttons[Index].Get();
	}

	return nullptr;
}

UCommonButtonBase* UCommonButtonGroupBase::GetSelectedButtonBase() const
{
	return GetButtonBaseAtIndex(SelectedButtonIndex);
}

