// Copyright Epic Games, Inc. All Rights Reserved.

#include "Groups/CommonWidgetGroupBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonWidgetGroupBase)

UCommonWidgetGroupBase::UCommonWidgetGroupBase()
{
}

void UCommonWidgetGroupBase::AddWidget(UWidget* InWidget)
{
	if (ensure(InWidget) && InWidget->IsA(GetWidgetType()))
	{
		OnWidgetAdded(InWidget);
	}
}

void UCommonWidgetGroupBase::AddWidgets(const TArray<UWidget*>& Widgets)
{
	for (UWidget* Widget : Widgets)
	{
		AddWidget(Widget);
	}
}

void UCommonWidgetGroupBase::RemoveWidget(UWidget* InWidget)
{
	if (ensure(InWidget) && InWidget->IsA(GetWidgetType()))
	{
		OnWidgetRemoved(InWidget);
	}
}

void UCommonWidgetGroupBase::RemoveAll()
{
	OnRemoveAll();
}

