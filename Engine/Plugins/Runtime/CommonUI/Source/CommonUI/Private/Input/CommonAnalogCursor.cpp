// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/CommonAnalogCursor.h"
#include "CommonInputTypeEnum.h"
#include "CommonUIPrivate.h"
#include "Input/CommonUIActionRouterBase.h"
#include "CommonInputSubsystem.h"
#include "Framework/Application/SlateUser.h"
#include "Input/CommonUIInputSettings.h"
#include "Engine/Console.h"
#include "Widgets/SViewport.h"
#include "CommonGameViewportClient.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/PlayerController.h"

#include "Components/ListView.h"
#include "Components/ScrollBar.h"
#include "Components/ScrollBox.h"
#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"
#include "Slate/SGameLayerManager.h"
#include "UnrealClient.h"

#define LOCTEXT_NAMESPACE "CommonAnalogCursor"

const float AnalogScrollUpdatePeriod = 0.1f;
const float ScrollDeadZone = 0.2f;

static TAutoConsoleVariable<bool> CVarCommonUIVirtualPointer(
	TEXT("CommonUI.EnableVirtualPointer"),
	true,
	TEXT("Enable / Disable the common ui virtual pointer (software cursor) rendered inside the game viewport."),
	ECVF_Default
);

static const TAutoConsoleVariable<bool> CVarShouldVirtualAcceptSimulateMouseButton(
	TEXT("CommonUI.ShouldVirtualAcceptSimulateMouseButton"),
	true,
	TEXT("Controls if virtual_accept key events will be converted to left mouse button events."));

static const TAutoConsoleVariable<bool> CVarShouldRouteOffscreenMouseButton(
	TEXT("CommonUI.ShouldRouteOffscreenMouseButton"),
	false,
	TEXT("Controls if we should directly route mouse events to offscreen widgets."));

bool IsEligibleFakeKeyPointerEvent(const FPointerEvent& PointerEvent)
{
	FKey EffectingButton = PointerEvent.GetEffectingButton();
	return EffectingButton.IsMouseButton() 
		&& EffectingButton != EKeys::LeftMouseButton
		&& EffectingButton != EKeys::RightMouseButton
		&& EffectingButton != EKeys::MiddleMouseButton;
}

bool IsCursorWithinRenderBounds(TSharedRef<SWidget> Widget, FSlateApplication& SlateApp, TSharedRef<FSlateUser> SlateUser)
{
	// Cursor is outside the clipped widget (i.e. outside viewport)
	TOptional<FSlateClippingState> ClippingState = Widget->GetCurrentClippingState();
	if (ClippingState.IsSet() && !ClippingState->IsPointInside(SlateUser->GetCursorPosition()))
	{
		return false;
	}
	else
	{
		// Cursor is outside window
		TSharedPtr<SWindow> FocusedWindow = SlateApp.FindWidgetWindow(Widget);
		if (!FocusedWindow.IsValid())
		{
			return true; // Widget is outside window, as there is no window
		}

		const FGeometry Geometry = FocusedWindow->GetCachedGeometry();
		const FVector2D LocalPoint = Geometry.AbsoluteToLocal(SlateUser->GetCursorPosition());

		if (LocalPoint.X < 0.0 || LocalPoint.Y < 0.0 || LocalPoint.X > Geometry.GetLocalSize().X || LocalPoint.Y > Geometry.GetLocalSize().Y)
		{
			return false;
		}
	}

	return true;
}

TOptional<FPointerEvent> GetSimulatedMouseEventForOffscreenFocusedWidget(const FKeyEvent& InKeyEvent, FSlateApplication& SlateApp,FWidgetPath& OutWidgetPath)
{
	TSharedPtr<FSlateUser> SlateUser = SlateApp.GetUser(InKeyEvent);
	TSharedPtr<SWidget> FocusedWidget = SlateApp.GetUserFocusedWidget(InKeyEvent.GetUserIndex());
	if(!SlateUser.IsValid() || !FocusedWidget.IsValid())
	{
		return TOptional<FPointerEvent>();
	}

	if (IsCursorWithinRenderBounds(FocusedWidget.ToSharedRef(), SlateApp, SlateUser.ToSharedRef()))
	{
		return TOptional<FPointerEvent>();
	}

	FWidgetPath WidgetPathNoPointer;
	SlateApp.FindPathToWidget(FocusedWidget.ToSharedRef(), WidgetPathNoPointer);

	if (!WidgetPathNoPointer.IsValid())
	{
		return TOptional<FPointerEvent>();
	}

	TArray<FWidgetAndPointer> WidgetAndPointers;
	WidgetAndPointers.Reserve(WidgetPathNoPointer.Widgets.Num());

	for (int32 Index = 0; Index < WidgetPathNoPointer.Widgets.Num(); ++Index)
	{
		const FVirtualPointerPosition VirtualCursorPosition(SlateUser->GetCursorPosition(), SlateUser->GetPreviousCursorPosition());
		WidgetAndPointers.Add(FWidgetAndPointer(WidgetPathNoPointer.Widgets[Index], VirtualCursorPosition)) ;
	}

	OutWidgetPath = FWidgetPath(WidgetAndPointers);

	const bool bIsPrimaryUser = FSlateApplication::CursorUserIndex == SlateUser->GetUserIndex();
	FPointerEvent MouseEvent(
		SlateUser->GetUserIndex(),
		FSlateApplication::CursorPointerIndex,
		SlateUser->GetCursorPosition(),
		SlateUser->GetPreviousCursorPosition(),
		bIsPrimaryUser ? SlateApp.GetPressedMouseButtons() : FTouchKeySet::EmptySet,
		EKeys::LeftMouseButton,
		0,
		bIsPrimaryUser ? SlateApp.GetModifierKeys() : FModifierKeysState()
	);

	return MouseEvent;
}

FCommonAnalogCursor::FCommonAnalogCursor(const UCommonUIActionRouterBase& InActionRouter)
	: ActionRouter(InActionRouter)
	, ActiveInputMethod(ECommonInputType::MouseAndKeyboard)
	, bShowVirtualPointer(false)
{
	bEnableCommonUIVirtualPointer = CVarCommonUIVirtualPointer.AsVariable()->GetBool();
}

void FCommonAnalogCursor::Initialize()
{
	RefreshCursorSettings();

	PointerButtonDownKeys = FSlateApplication::Get().GetPressedMouseButtons();
	PointerButtonDownKeys.Remove(EKeys::LeftMouseButton);
	PointerButtonDownKeys.Remove(EKeys::RightMouseButton);
	PointerButtonDownKeys.Remove(EKeys::MiddleMouseButton);
	
	CVarCommonUIVirtualPointer.AsVariable()->OnChangedDelegate().AddRaw(this, &FCommonAnalogCursor::HandleOnEnableVirtualPointerChanged);

	UCommonInputSubsystem& InputSubsystem = ActionRouter.GetInputSubsystem();
	InputSubsystem.OnInputMethodChangedNative.AddSP(this, &FCommonAnalogCursor::HandleInputMethodChanged);
	HandleInputMethodChanged(InputSubsystem.GetCurrentInputType());
}

void FCommonAnalogCursor::Deinitialize()
{
	// Cancel any in-flight async load so the streamable callback does not run after we are torn down.
	if (DefaultVirtualPointerLoadHandle.IsValid())
	{
		DefaultVirtualPointerLoadHandle->CancelHandle();
		DefaultVirtualPointerLoadHandle.Reset();
	}

	if (DefaultVirtualPointerWidget)
	{
		DefaultVirtualPointerWidget = nullptr;
	}

	LastAppliedSoftwareCursorWidget.Reset();

	CVarCommonUIVirtualPointer.AsVariable()->OnChangedDelegate().RemoveAll(this);

	UCommonInputSubsystem& InputSubsystem = ActionRouter.GetInputSubsystem();
	InputSubsystem.OnInputMethodChangedNative.RemoveAll(this);
}

#if WITH_EDITOR
extern bool IsViewportWindowInFocusPath(const UCommonUIActionRouterBase& Router);
#endif

void FCommonAnalogCursor::Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor)
{
	// Refreshing visibility per tick to address multiplayer p2 cursor visibility getting stuck
	RefreshCursorVisibility();

	// UE-380247: enter when gamepad drove the cursor (ActiveInputMethod thrashes under VP).
	const bool bVPGamepadDriven = IsVirtualPointerEnabled() && bLastCursorInputWasGamepad;
	// Don't bother trying to do anything while the game viewport has capture
	if ((IsUsingGamepad() || IsUsingVirtualPointer() || bVPGamepadDriven) && IsGameViewportInFocusPathWithoutCapture())
	{
		// The game viewport can't have been focused without a user, so we're quite safe to assume/enforce validity of the user here
		const TSharedRef<FSlateUser> SlateUser = SlateApp.GetUser(GetOwnerUserIndex()).ToSharedRef();

#if WITH_EDITOR
		// Instantly acknowledge any changes to our settings when we're in the editor
		RefreshCursorSettings();
		if (!IsViewportWindowInFocusPath(ActionRouter))
		{
			LastCursorTarget.Reset();
			return;
		}
#endif
		// UE-380247: clamp cursor to viewport only when gamepad drove it.
		if (bIsAnalogMovementEnabled || (IsUsingVirtualPointer() && bLastCursorInputWasGamepad))
		{
			LastCursorTarget.Reset();
			const FVector2D NewPosition = CalculateTickedCursorPosition(DeltaTime, SlateApp, SlateUser);

			UCommonInputSubsystem& InputSubsystem = ActionRouter.GetInputSubsystem();
			// shield our gamepad flag from the synthetic mouse-move SetCursorPosition emits.
			bIgnoreNextMouseMove = true;
			InputSubsystem.SetCursorPosition(NewPosition, false);
		}
		else if (UCommonUIInputSettings::Get().ShouldLinkCursorToGamepadFocus())
		{
			TSharedPtr<SWidget> PinnedLastCursorTarget = LastCursorTarget.Pin();

			// By default the cursor target is the focused widget itself, unless we're working with a list view
			TSharedPtr<SWidget> CursorTarget = SlateUser->GetFocusedWidget();
			if (TSharedPtr<ITableViewMetadata> TableViewMetadata = CursorTarget ? CursorTarget->GetMetaData<ITableViewMetadata>() : nullptr)
			{
				// A list view is currently focused, so we actually want to make sure we are centering the cursor over the currently selected row instead
				TArray<TSharedPtr<ITableRow>> SelectedRows = TableViewMetadata->GatherSelectedRows();
				if (SelectedRows.Num() > 0 && ensure(SelectedRows[0].IsValid()))
				{
					// Just pick the first selected entry in the list - it's awfully rare to have anything other than single-selection when using gamepad
					CursorTarget = SelectedRows[0]->AsWidget();
				}
			}
			
			FGeometry TargetGeometry;
			if (CursorTarget)
			{
				if (CursorTarget == GetViewportClient()->GetGameViewportWidget())
				{
					// When the target is the game viewport as a whole, we don't want to center blindly - we want to center in the geometry of our owner's widget host layer
					TSharedPtr<IGameLayerManager> GameLayerManager = GetViewportClient()->GetGameLayerManager();
					if (ensure(GameLayerManager))
					{
						TargetGeometry = GameLayerManager->GetPlayerWidgetHostGeometry(ActionRouter.GetLocalPlayerChecked());
					}
				}
				else
				{
					TargetGeometry = CursorTarget->GetTickSpaceGeometry();
				}
			}

			// We want to try to update the cursor position when focus changes or the focused widget moves at all
			if (CursorTarget != PinnedLastCursorTarget || (CursorTarget && TargetGeometry.GetAccumulatedRenderTransform() != LastCursorTargetTransform))
			{
#if !UE_BUILD_SHIPPING
				if (CursorTarget != PinnedLastCursorTarget)
				{
					UE_LOGF(LogCommonUI, Verbose, "User[%d] cursor target changed to [%ls]", GetOwnerUserIndex(), *FReflectionMetaData::GetWidgetDebugInfo(CursorTarget.Get()));
				}
#endif

				// Release capture unless the focused widget is the captor
				if (PinnedLastCursorTarget != CursorTarget && SlateUser->HasCursorCapture() && !SlateUser->DoesWidgetHaveAnyCapture(CursorTarget))
				{
					UE_LOGF(LogCommonUI, Log, "User[%d] focus changed while the cursor is captured - releasing now before moving cursor to focused widget.", GetOwnerUserIndex());
					SlateUser->ReleaseCursorCapture();
				}

				LastCursorTarget = CursorTarget;

				bool bHasValidCursorTarget = false;
				if (CursorTarget)
				{
					if (TargetGeometry.GetLocalSize().X > UE_SMALL_NUMBER && TargetGeometry.GetLocalSize().Y > UE_SMALL_NUMBER)
					{
						LastCursorTargetTransform = TargetGeometry.GetAccumulatedRenderTransform();

						bHasValidCursorTarget = true;
						
						const FVector2D AbsoluteWidgetCenter = TargetGeometry.GetAbsolutePositionAtCoordinates(FVector2D(0.5f, 0.5f));
						SlateUser->SetCursorPosition(AbsoluteWidgetCenter);

						UE_LOGF(LogCommonUI, Verbose, "User[%d] moving cursor to target [%ls] @ (%d, %d)", GetOwnerUserIndex(), *FReflectionMetaData::GetWidgetDebugInfo(CursorTarget.Get()), (int32)AbsoluteWidgetCenter.X, (int32)AbsoluteWidgetCenter.Y);
					}
				}

				if (!bHasValidCursorTarget)
				{
					LastCursorTargetTransform = FSlateRenderTransform();
					SetNormalizedCursorPosition(FVector2D::ZeroVector);
				}
			}
		}

		if (bShouldHandleRightAnalog)
		{
			TimeUntilScrollUpdate -= DeltaTime;
			if (TimeUntilScrollUpdate <= 0.0f && GetAnalogValues(EAnalogStick::Right).SizeSquared() > FMath::Square(ScrollDeadZone))
			{
				// Generate mouse wheel events over all widgets currently registered as scroll recipients
				const TArray<const UWidget*>& AnalogScrollRecipients = ActionRouter.GatherActiveAnalogScrollRecipients();
				if (AnalogScrollRecipients.Num() > 0)
				{
					const FCommonAnalogCursorSettings& CursorSettings = UCommonUIInputSettings::Get().GetAnalogCursorSettings();
					const auto GetScrollAmountFunc = [&CursorSettings](float AnalogValue)
					{
						const float AmountBeyondDeadZone = FMath::Abs(AnalogValue) - CursorSettings.ScrollDeadZone;
						if (AmountBeyondDeadZone <= 0.f)
						{
							return 0.f;
						}
						return (AmountBeyondDeadZone / (1.f - CursorSettings.ScrollDeadZone)) * -FMath::Sign(AnalogValue) * CursorSettings.ScrollMultiplier;
					};

					const FVector2D& RightStickValues = GetAnalogValues(EAnalogStick::Right);
					const FVector2D ScrollAmounts(GetScrollAmountFunc(RightStickValues.X), GetScrollAmountFunc(RightStickValues.Y));

					for (const UWidget* ScrollRecipient : AnalogScrollRecipients)
					{
						check(ScrollRecipient);
						if (ScrollRecipient->GetCachedWidget())
						{
							const EOrientation Orientation = DetermineScrollOrientation(*ScrollRecipient);
							const float ScrollAmount = Orientation == Orient_Vertical ? ScrollAmounts.Y : ScrollAmounts.X;
							if (FMath::Abs(ScrollAmount) > SMALL_NUMBER)
							{
								const FVector2D WidgetCenter = ScrollRecipient->GetCachedGeometry().GetAbsolutePositionAtCoordinates(FVector2D(.5f, .5f));
								if (IsInViewport(WidgetCenter))
								{
									FPointerEvent MouseEvent(
										SlateUser->GetUserIndex(),
										FSlateApplication::CursorPointerIndex,
										WidgetCenter,
										WidgetCenter,
										TSet<FKey>(),
										EKeys::MouseWheelAxis,
										ScrollAmount,
										FModifierKeysState());

									UCommonInputSubsystem& InputSubsystem = ActionRouter.GetInputSubsystem();
									InputSubsystem.SetIsGamepadSimulatedClick(true);
									SlateApp.ProcessMouseWheelOrGestureEvent(MouseEvent, nullptr);
									InputSubsystem.SetIsGamepadSimulatedClick(false);
								}
							}
						}
					}
				}
			}
		}
	}
	else
	{
		// Since we're not processing cursor target this frame, the cursor position may change externally and therefore invalidate our cache
		LastCursorTarget.Reset();
	}

	// Accumulate hold time while accept is down so UpdateVirtualPointerVisualState can derive Hold flag and progress
	if (bVirtualAcceptDown)
	{
		HoldElapsedTime += DeltaTime;
	}
	else
	{
		HoldElapsedTime = 0.f;
	}

	// Always call UpdateVirtualPointerVisualState - it short-circuits when there is no widget and handles the
	// transition out of "in use" by clearing all flags and firing one final state-change event.
	UpdateVirtualPointerVisualState();
}

bool FCommonAnalogCursor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	if (IsRelevantInput(InKeyEvent))
	{
		const ULocalPlayer& LocalPlayer = *ActionRouter.GetLocalPlayerChecked();
		if (LocalPlayer.ViewportClient && LocalPlayer.ViewportClient->ViewportConsole && LocalPlayer.ViewportClient->ViewportConsole->ConsoleActive())
		{
			// Let everything through when the console is open
			return false;
		}

#if !UE_BUILD_SHIPPING
		const FKey& PressedKey = InKeyEvent.GetKey();
		if (PressedKey == EKeys::Gamepad_LeftShoulder) { ShoulderButtonStatus |= EShoulderButtonFlags::LeftShoulder; }
		if (PressedKey == EKeys::Gamepad_RightShoulder) { ShoulderButtonStatus |= EShoulderButtonFlags::RightShoulder; }
		if (PressedKey == EKeys::Gamepad_LeftTrigger) { ShoulderButtonStatus |= EShoulderButtonFlags::LeftTrigger; }
		if (PressedKey == EKeys::Gamepad_RightTrigger) { ShoulderButtonStatus |= EShoulderButtonFlags::RightTrigger; }

		if (ShoulderButtonStatus == EShoulderButtonFlags::All)
		{
			ShoulderButtonStatus = EShoulderButtonFlags::None;
			bIsAnalogMovementEnabled = !bIsAnalogMovementEnabled;
			//RefreshCursorVisibility();
		}
#endif

		// We support binding actions to the virtual accept key, so it's a special flower that gets processed right now
		const bool bIsVirtualAccept = InKeyEvent.GetKey() == EKeys::Virtual_Gamepad_Accept.GetVirtualKey();
		const EInputEvent InputEventType = InKeyEvent.IsRepeat() ? IE_Repeat : IE_Pressed;
		if (bIsVirtualAccept)
		{
			bVirtualAcceptDown = true;
			UpdateVirtualPointerVisualState();
		}
		if (bIsVirtualAccept && ActionRouter.ProcessInput(InKeyEvent.GetKey(), InputEventType) == ERouteUIInputResult::Handled)
		{
			return true;
		}
		else if (!bIsVirtualAccept || ShouldVirtualAcceptSimulateMouseButton(InKeyEvent, IE_Pressed))
		{
			// If virtually accepting the focused widget, and cursor is not within the current cliprect or window, we can not rely on the hittest grid as the cursor and widget position may not updated correctly.
			// For instance, when attemptign to forward mouse input to animating or offscreen widgets. As a workaround, directly forard the pointer down event to the focused widget path.
			if (bIsVirtualAccept && CVarShouldRouteOffscreenMouseButton.GetValueOnGameThread())
			{
				FWidgetPath FocusedWidgetPath;
				TOptional<FPointerEvent> MouseEvent = GetSimulatedMouseEventForOffscreenFocusedWidget(InKeyEvent, SlateApp, FocusedWidgetPath);
				if (MouseEvent.IsSet())
				{
					UCommonInputSubsystem& InputSubsystem = ActionRouter.GetInputSubsystem();
					InputSubsystem.SetIsGamepadSimulatedClick(bIsVirtualAccept);
					const bool bReturnValue = SlateApp.RoutePointerDownEvent(FocusedWidgetPath, MouseEvent.GetValue()).IsEventHandled();
					InputSubsystem.SetIsGamepadSimulatedClick(false);
					
					return bReturnValue;
				}
			}

			// There is no awareness on a mouse event of whether it's real or not, so mark that here.
			UCommonInputSubsystem& InputSubsystem = ActionRouter.GetInputSubsystem();
			InputSubsystem.SetIsGamepadSimulatedClick(bIsVirtualAccept);
			bool bReturnValue = FAnalogCursor::HandleKeyDownEvent(SlateApp, InKeyEvent);
			InputSubsystem.SetIsGamepadSimulatedClick(false);

			return bReturnValue;
		}
	}
	return false;
}

bool FCommonAnalogCursor::HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	// Always clear our accept-down tracking when the virtual accept key goes up, regardless of input
	// relevance or our previous belief about whether the key was down. If we gate this on bVirtualAcceptDown
	// being true, any path that silently cleared it (e.g. SetVirtualPointerVisibility) makes us miss the
	// keyup-driven recompute, leaving the BP's last state stale.
	if (InKeyEvent.GetKey() == EKeys::Virtual_Gamepad_Accept.GetVirtualKey())
	{
		bVirtualAcceptDown = false;
		HoldElapsedTime = 0.f;
		UpdateVirtualPointerVisualState();
	}

	if (IsRelevantInput(InKeyEvent))
	{
#if !UE_BUILD_SHIPPING
		const FKey& PressedKey = InKeyEvent.GetKey();
		if (PressedKey == EKeys::Gamepad_LeftShoulder) { ShoulderButtonStatus ^= EShoulderButtonFlags::LeftShoulder; }
		if (PressedKey == EKeys::Gamepad_RightShoulder) { ShoulderButtonStatus ^= EShoulderButtonFlags::RightShoulder; }
		if (PressedKey == EKeys::Gamepad_LeftTrigger) { ShoulderButtonStatus ^= EShoulderButtonFlags::LeftTrigger; }
		if (PressedKey == EKeys::Gamepad_RightTrigger) { ShoulderButtonStatus ^= EShoulderButtonFlags::RightTrigger; }
#endif

		TGuardValue<TOptional<FKeyEvent>> KeyUpEventGuard(ActiveKeyUpEvent, TOptional<FKeyEvent>(InKeyEvent));

		const bool bIsVirtualAccept = InKeyEvent.GetKey() == EKeys::Virtual_Gamepad_Accept.GetVirtualKey();
		if (bIsVirtualAccept && ActionRouter.ProcessInput(InKeyEvent.GetKey(), IE_Released) == ERouteUIInputResult::Handled)
		{
			return true;
		}
		else if (!bIsVirtualAccept || ShouldVirtualAcceptSimulateMouseButton(InKeyEvent, IE_Released))
		{
						// If virtually accepting the focused widget, and cursor is not within the current cliprect or window, we can not rely on the hittest grid as the cursor and widget position may not updated correctly.
			// For instance, when attemptign to forward mouse input to animating or offscreen widgets. As a workaround, directly forard the pointer down event to the focused widget path.
			if (bIsVirtualAccept && CVarShouldRouteOffscreenMouseButton.GetValueOnGameThread())
			{
				FWidgetPath FocusedWidgetPath;
				TOptional<FPointerEvent> MouseEvent = GetSimulatedMouseEventForOffscreenFocusedWidget(InKeyEvent, SlateApp, FocusedWidgetPath);
				if (MouseEvent.IsSet())
				{
					return SlateApp.RoutePointerUpEvent(FocusedWidgetPath, MouseEvent.GetValue()).IsEventHandled();
				}
			}

			return FAnalogCursor::HandleKeyUpEvent(SlateApp, InKeyEvent);
		}
	}
	return false;
}

bool FCommonAnalogCursor::CanReleaseMouseCapture() const
{
	EMouseCaptureMode MouseCapture = ActionRouter.GetActiveMouseCaptureMode();
	return MouseCapture == EMouseCaptureMode::CaptureDuringMouseDown || MouseCapture == EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown;
}

bool FCommonAnalogCursor::HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent)
{
	// UE-380247: gamepad now drives the cursor (zero values don't flip back - sticky between strokes).
	if (InAnalogInputEvent.GetKey().IsGamepadKey() && !FMath::IsNearlyZero(InAnalogInputEvent.GetAnalogValue()))
	{
		bLastCursorInputWasGamepad = true;
	}

	if (IsRelevantInput(InAnalogInputEvent))
	{
		bool bParentHandled = FAnalogCursor::HandleAnalogInputEvent(SlateApp, InAnalogInputEvent);
		if (bIsAnalogMovementEnabled || IsUsingVirtualPointer())
		{
			return bParentHandled;
		}
	}

	return false;
}

bool FCommonAnalogCursor::HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent)
{
#if WITH_EDITOR
	// We can leave editor cursor visibility in a bad state if the engine stops ticking to debug
	if (GIntraFrameDebuggingGameThread)
	{
		SlateApp.SetPlatformCursorVisibility(true);
		TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(GetOwnerUserIndex());
		if (SlateUser)
		{
			SlateUser->SetCursorVisibility(true);
		}
	}
#endif // WITH_EDITOR

	// UE-380247: real mouse motion hands cursor control back to the mouse, releasing the gamepad clamp.
	// Swallow the synthetic event our own SetCursorPosition emits so it doesn't masquerade as mouse input.
	if (bIgnoreNextMouseMove)
	{
		bIgnoreNextMouseMove = false;
	}
	else if (!MouseEvent.GetCursorDelta().IsNearlyZero())
	{
		bLastCursorInputWasGamepad = false;
	}

	return FAnalogCursor::HandleMouseMoveEvent(SlateApp, MouseEvent);
}

bool FCommonAnalogCursor::HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& PointerEvent)
{
	if (FAnalogCursor::IsRelevantInput(PointerEvent))
	{
#if UE_COMMONUI_PLATFORM_REQUIRES_CURSOR_HIDDEN_FOR_TOUCH	
		// Some platforms don't register as switching its input type, so detect touch input here to hide the cursor.
		if (PointerEvent.IsTouchEvent() && ShouldHideCursor())
		{
			//ClearCenterWidget();
			HideCursor();
		}
#endif 

		// Mouse buttons other than the two primaries are fair game for binding as if they were normal keys
		const FKey EffectingButton = PointerEvent.GetEffectingButton();
		if (EffectingButton.IsMouseButton()
			&& EffectingButton != EKeys::LeftMouseButton
			&& EffectingButton != EKeys::RightMouseButton
			&& EffectingButton != EKeys::MiddleMouseButton)
		{
			UGameViewportClient* ViewportClient = GetViewportClient();
			if (TSharedPtr<SWidget> ViewportWidget = ViewportClient ? ViewportClient->GetGameViewportWidget() : nullptr)
			{
				const FWidgetPath WidgetsUnderCursor = SlateApp.LocateWindowUnderMouse(PointerEvent.GetScreenSpacePosition(), SlateApp.GetInteractiveTopLevelWindows());
				if (WidgetsUnderCursor.ContainsWidget(ViewportWidget.Get()))
				{
					FKeyEvent MouseKeyEvent(EffectingButton, PointerEvent.GetModifierKeys(), PointerEvent.GetUserIndex(), false, 0, 0);
					if (SlateApp.ProcessKeyDownEvent(MouseKeyEvent))
					{
						PointerButtonDownKeys.Add(EffectingButton);
						return true;
					}
				}
			}
		}
	}
	return false;
}

bool FCommonAnalogCursor::HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& PointerEvent)
{
	if (FAnalogCursor::IsRelevantInput(PointerEvent))
	{
		const bool bHadKeyDown = PointerButtonDownKeys.Remove(PointerEvent.GetEffectingButton()) > 0;
		if (bHadKeyDown
			|| (IsEligibleFakeKeyPointerEvent(PointerEvent) && !SlateApp.HasUserMouseCapture(PointerEvent.GetUserIndex())))
		{
			// Reprocess as a key if there was no mouse capture or it was previously pressed
			FKeyEvent MouseKeyEvent(PointerEvent.GetEffectingButton(), PointerEvent.GetModifierKeys(), PointerEvent.GetUserIndex(), false, 0, 0);
			bool bHandled = SlateApp.ProcessKeyUpEvent(MouseKeyEvent);
			if (bHadKeyDown)
			{
				// Only block the mouse up if the mouse down was also blocked
				return bHandled;
			}
		}
	}

	return false;
}

int32 FCommonAnalogCursor::GetOwnerUserIndex() const
{
	return ActionRouter.GetLocalPlayerIndex();
}

void FCommonAnalogCursor::ShouldHandleRightAnalog(bool bInShouldHandleRightAnalog)
{
	bShouldHandleRightAnalog = bInShouldHandleRightAnalog;
}

bool FCommonAnalogCursor::ShouldVirtualAcceptSimulateMouseButton(const FKeyEvent& InKeyEvent, EInputEvent InputEvent) const
{
	return CVarShouldVirtualAcceptSimulateMouseButton.GetValueOnGameThread();
}

void FCommonAnalogCursor::OnVirtualAcceptHoldCanceled()
{
	if (ActiveKeyUpEvent.IsSet() && ActiveKeyUpEvent->GetKey() == EKeys::Virtual_Gamepad_Accept.GetVirtualKey() && ShouldVirtualAcceptSimulateMouseButton(ActiveKeyUpEvent.GetValue(), IE_Pressed))
	{
		UCommonInputSubsystem& InputSubsystem = ActionRouter.GetInputSubsystem();
		InputSubsystem.SetIsGamepadSimulatedClick(true);
		FAnalogCursor::HandleKeyDownEvent(FSlateApplication::Get(), ActiveKeyUpEvent.GetValue());
		InputSubsystem.SetIsGamepadSimulatedClick(false);
	}
}

TWeakPtr<SWidget> FCommonAnalogCursor::GetLastCursorTarget() const 
{ 
	return LastCursorTarget; 
}

//void FCommonAnalogCursor::SetCursorMovementStick(EAnalogStick InCursorMovementStick)
//{
//	const EAnalogStick NewStick = InCursorMovementStick == EAnalogStick::Max ? EAnalogStick::Left : InCursorMovementStick;
//	if (NewStick != CursorMovementStick)
//	{
//		ClearAnalogValues();
//		CursorMovementStick = NewStick;
//	}
//}

EOrientation FCommonAnalogCursor::DetermineScrollOrientation(const UWidget& Widget) const
{
	if (const UListView* AsListView = Cast<const UListView>(&Widget))
	{
		return AsListView->GetOrientation();
	}
	else if (const UScrollBar* AsScrollBar = Cast<const UScrollBar>(&Widget))
	{
		return AsScrollBar->GetOrientation();
	}
	else if (const UScrollBox* AsScrollBox = Cast<const UScrollBox>(&Widget))
	{
		return AsScrollBox->GetOrientation();
	}
	return EOrientation::Orient_Vertical;
}

bool FCommonAnalogCursor::IsRelevantInput(const FKeyEvent& KeyEvent) const
{
	return (IsUsingGamepad() || IsUsingVirtualPointer()) && FAnalogCursor::IsRelevantInput(KeyEvent) && (IsGameViewportInFocusPathWithoutCapture() || (KeyEvent.GetKey() == EKeys::Virtual_Gamepad_Accept.GetVirtualKey() && CanReleaseMouseCapture()));
}

bool FCommonAnalogCursor::IsRelevantInput(const FAnalogInputEvent& AnalogInputEvent) const
{
	return (IsUsingGamepad() || IsUsingVirtualPointer()) && FAnalogCursor::IsRelevantInput(AnalogInputEvent) && IsGameViewportInFocusPathWithoutCapture();
}

//EAnalogStick FCommonAnalogCursor::GetScrollStick() const
//{
//	// Scroll is on the right stick unless it conflicts with the cursor movement stick
//	return CursorMovementStick == EAnalogStick::Right ? EAnalogStick::Left : EAnalogStick::Right;
//}

UGameViewportClient* FCommonAnalogCursor::GetViewportClient() const
{
	return ActionRouter.GetLocalPlayerChecked()->ViewportClient;
}

bool FCommonAnalogCursor::IsGameViewportInFocusPathWithoutCapture() const
{
	if (const UGameViewportClient* ViewportClient = GetViewportClient())
	{
		if (TSharedPtr<SViewport> GameViewportWidget = ViewportClient->GetGameViewportWidget())
		{
			TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(GetOwnerUserIndex());
			// In gameplay-style modes the SViewport is expected to hold cursor capture, so we must override that to drive the virtual pointer
			const ECommonInputMode ActiveInputMode = ActionRouter.GetActiveInputMode();
			const bool bIsInGameInputMode = (ActiveInputMode == ECommonInputMode::Game || ActiveInputMode == ECommonInputMode::All);
			const bool bBypassCaptureCheck = bIsInGameInputMode && IsUsingVirtualPointer();
			if (SlateUser && (bBypassCaptureCheck || !SlateUser->DoesWidgetHaveCursorCapture(GameViewportWidget)))
			{
#if PLATFORM_DESKTOP
				// Not captured - is it in the focus path?
				return SlateUser->IsWidgetInFocusPath(GameViewportWidget);
#else
				// If we're not on desktop, focus on the viewport is irrelevant, as there aren't other windows around to care about
				return true;
#endif
			}
		}
	}
	return false;
}

void FCommonAnalogCursor::HandleInputMethodChanged(ECommonInputType NewInputMethod)
{
	ActiveInputMethod = NewInputMethod;
	if (IsUsingGamepad() && !IsUsingVirtualPointer())
	{
		LastCursorTarget.Reset();
	}
	RefreshCursorVisibility();
}

void FCommonAnalogCursor::RefreshCursorSettings()
{
	const FCommonAnalogCursorSettings& CursorSettings = UCommonUIInputSettings::Get().GetAnalogCursorSettings();
	Acceleration = CursorSettings.CursorAcceleration;
	MaxSpeed = CursorSettings.CursorMaxSpeed;
	DeadZone = CursorSettings.CursorDeadZone;
	StickySlowdown = CursorSettings.HoverSlowdownFactor;
	Mode = CursorSettings.bEnableCursorAcceleration ? AnalogCursorMode::Accelerated : AnalogCursorMode::Direct;
}

void FCommonAnalogCursor::RefreshCursorVisibility()
{
	FSlateApplication& SlateApp = FSlateApplication::Get();
	if (TSharedPtr<FSlateUser> SlateUser = SlateApp.GetUser(GetOwnerUserIndex()))
	{
		const bool bVirtualPointerEnabled = IsVirtualPointerEnabled();
		if (bVirtualPointerEnabled != bLastBroadcastVirtualPointerEnabled)
		{
			bLastBroadcastVirtualPointerEnabled = bVirtualPointerEnabled;
			OnVirtualPointerEnabledChanged.Broadcast(bVirtualPointerEnabled);
		}

		const bool bIsUsingVirtualPointer = IsUsingVirtualPointer();
		const bool bShowCursor = bIsAnalogMovementEnabled || ActionRouter.ShouldAlwaysShowCursor() || ActiveInputMethod == ECommonInputType::MouseAndKeyboard || bIsUsingVirtualPointer;

		if (!bShowCursor || bIsUsingVirtualPointer)
		{
			SlateApp.SetPlatformCursorVisibility(false);
		}

		if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(ActionRouter.GetLocalPlayer()))
		{
			APlayerController* PC = LocalPlayer->GetPlayerController(ActionRouter.GetWorld());
			if (PC)
			{
				PC->CurrentMouseCursor = bIsUsingVirtualPointer ? EMouseCursor::Custom : PC->DefaultMouseCursor.GetValue();
			}

			if (TObjectPtr<class UGameViewportClient> ViewportClient = LocalPlayer->ViewportClient)
			{
				if (UCommonGameViewportClient* CommonVC = Cast<UCommonGameViewportClient>(ViewportClient))
				{
					CommonVC->SetUseVirtualPointerCursor(bIsUsingVirtualPointer);
				}

				if (bIsUsingVirtualPointer)
				{
					// Skip when the desired widget is already what we last applied; TakeWidget() is the per-tick cost.
					UUserWidget* DesiredCursorWidget = GetOrLoadDefaultVirtualPointer();
					if (DesiredCursorWidget && LastAppliedSoftwareCursorWidget.Get() != DesiredCursorWidget)
					{
						ViewportClient->SetSoftwareCursorWidget(EMouseCursor::Custom, DesiredCursorWidget);
						
						ViewportClient->SetUseSoftwareCursorWidgets(true);
						LastAppliedSoftwareCursorWidget = DesiredCursorWidget;
						FSlateApplication::Get().QueryCursor();
					}
				}
				else if (LastAppliedSoftwareCursorWidget.IsValid())
				{
					ViewportClient->SetUseSoftwareCursorWidgets(false);
					LastAppliedSoftwareCursorWidget.Reset();
					FSlateApplication::Get().QueryCursor();
				}
			}
		}
		SlateUser->SetCursorVisibility(bShowCursor);
	}
}

bool FCommonAnalogCursor::IsUsingGamepad() const
{
	return ActiveInputMethod == ECommonInputType::Gamepad;
}

bool FCommonAnalogCursor::IsUsingFakeTouch() const
{
	return FSlateApplication::Get().IsFakingTouchEvents() && ActiveInputMethod == ECommonInputType::Touch;
}

bool FCommonAnalogCursor::IsVirtualPointerEnabled() const
{
	return bEnableCommonUIVirtualPointer && bShowVirtualPointer;
}

bool FCommonAnalogCursor::IsUsingVirtualPointer() const
{
	return (IsUsingGamepad() || IsUsingFakeTouch()) && IsVirtualPointerEnabled();
}

void FCommonAnalogCursor::SetVirtualPointerVisibility(bool bVisible)
{
	bShowVirtualPointer = bVisible;
	if (!bVisible)
	{
		bVirtualAcceptDown = false;
		HoldElapsedTime = 0.f;
	}
	RefreshCursorVisibility();
	// Let UpdateVirtualPointerVisualState recompute and fire a state-change event if the visibility flip
	// caused active flags to drop (so the widget can return to its idle visual). Avoid silently zeroing
	// CurrentVirtualPointerVisualState here - that desyncs the BP from our internal state.
	UpdateVirtualPointerVisualState();
}

FVirtualPointerEnabledChanged& FCommonAnalogCursor::GetOnVirtualPointerEnabledChanged()
{
	return OnVirtualPointerEnabledChanged;
}

bool FCommonAnalogCursor::ShouldHideCursor() const
{
	bool bUsingMouseForTouch = FSlateApplication::Get().IsFakingTouchEvents();
	const ULocalPlayer& LocalPlayer = *ActionRouter.GetLocalPlayerChecked();
	if (UGameViewportClient* GameViewportClient = LocalPlayer.ViewportClient)
	{
		bUsingMouseForTouch |= GameViewportClient->GetUseMouseForTouch();
	}

	return !bUsingMouseForTouch;
}

void FCommonAnalogCursor::HideCursor()
{
	const TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(GetOwnerUserIndex());
	const UWorld* World = ActionRouter.GetWorld();
	if (SlateUser && World && World->IsGameWorld())
	{
		UGameViewportClient* GameViewport = World->GetGameViewport();
		if (GameViewport && GameViewport->GetWindow().IsValid() && GameViewport->Viewport)
		{
			const FVector2D TopLeftPos = GameViewport->Viewport->ViewportToVirtualDesktopPixel(FVector2D(0.025f, 0.025f));
			SlateUser->SetCursorPosition(TopLeftPos);
			SlateUser->SetCursorVisibility(false);
		}
	}
}

void FCommonAnalogCursor::SetNormalizedCursorPosition(const FVector2D& RelativeNewPosition)
{
	TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(GetOwnerUserIndex());
	if (SlateUser)
	{
		const UGameViewportClient* ViewportClient = GetViewportClient();
		if (TSharedPtr<SViewport> ViewportWidget = ViewportClient ? ViewportClient->GetGameViewportWidget() : nullptr)
		{
			const FVector2D ClampedNewPosition(FMath::Clamp(RelativeNewPosition.X, 0.0f, 1.0f), FMath::Clamp(RelativeNewPosition.Y, 0.0f, 1.0f));
			const FVector2D AbsolutePosition = ViewportWidget->GetCachedGeometry().GetAbsolutePositionAtCoordinates(ClampedNewPosition);
			SlateUser->SetCursorPosition(AbsolutePosition);
		}
	}
}

bool FCommonAnalogCursor::IsInViewport(const FVector2D& Position) const
{
	if (const UGameViewportClient* ViewportClient = GetViewportClient())
	{
		TSharedPtr<SViewport> ViewportWidget = ViewportClient->GetGameViewportWidget();
		return ViewportWidget && ViewportWidget->GetCachedGeometry().GetLayoutBoundingRect().ContainsPoint(Position);
	}
	return false;
}

FVector2D FCommonAnalogCursor::ClampPositionToViewport(const FVector2D& InPosition) const
{
	const UGameViewportClient* ViewportClient = GetViewportClient();
	if (TSharedPtr<SViewport> ViewportWidget = ViewportClient ? ViewportClient->GetGameViewportWidget() : nullptr)
	{
		const FGeometry& ViewportGeometry = ViewportWidget->GetCachedGeometry();
		FVector2D LocalPosition = ViewportGeometry.AbsoluteToLocal(InPosition);
		LocalPosition.X = FMath::Clamp(LocalPosition.X, 1.0f, ViewportGeometry.GetLocalSize().X - 1.0f);
		LocalPosition.Y = FMath::Clamp(LocalPosition.Y, 1.0f, ViewportGeometry.GetLocalSize().Y - 1.0f);
		
		return ViewportGeometry.LocalToAbsolute(LocalPosition);
	}

	return InPosition;
}

void FCommonAnalogCursor::HandleOnEnableVirtualPointerChanged(IConsoleVariable* ConsoleVariable)
{
	if (bEnableCommonUIVirtualPointer == ConsoleVariable->GetBool())
	{
		return;
	}
	bEnableCommonUIVirtualPointer = ConsoleVariable->GetBool();
	UpdateInputPreProcessor();
	RefreshCursorVisibility();
}

void FCommonAnalogCursor::UpdateInputPreProcessor()
{
	FSlateApplication& App = FSlateApplication::Get();

	TSharedPtr<IInputProcessor> InputProcessor = SharedThis(this);

	if (bEnableCommonUIVirtualPointer)
	{
		if (App.FindInputPreProcessor(InputProcessor, EInputPreProcessorType::Engine) == INDEX_NONE)
		{
			App.RegisterInputPreProcessor(InputProcessor);
		}
	}
	else if (App.FindInputPreProcessor(InputProcessor, EInputPreProcessorType::Engine) != INDEX_NONE)
	{
		App.UnregisterInputPreProcessor(InputProcessor);
	}
}

TObjectPtr<UUserWidget> FCommonAnalogCursor::GetOrLoadDefaultVirtualPointer()
{
	// Cached widget - return immediately, no asset I/O.
	if (DefaultVirtualPointerWidget)
	{
		return DefaultVirtualPointerWidget;
	}

	// A previous async load resolved to nothing usable. Don't retry every frame.
	if (bDefaultVirtualPointerLoadFailed)
	{
		return nullptr;
	}

	const TSoftClassPtr<UUserWidget>& VirtualPointerClass = UCommonUIInputSettings::Get().GetDefaultVirtualPointerClass();
	if (VirtualPointerClass.IsNull())
	{
		return nullptr;
	}

	// Class is already resident in memory (e.g. async load completed previously, or another system loaded
	// it). Create the widget right away - no need to round-trip through the streamable manager.
	if (UClass* LoadedClass = VirtualPointerClass.Get())
	{
		if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(ActionRouter.GetLocalPlayer()))
		{
			if (APlayerController* PC = LocalPlayer->GetPlayerController(ActionRouter.GetWorld()))
			{
				DefaultVirtualPointerWidget = CreateWidget(PC, LoadedClass);
			}
		}
		return DefaultVirtualPointerWidget;
	}

	// Async load already in flight. Don't kick off another - guarantees we don't request the asset every
	// frame while waiting for the first request to complete.
	if (DefaultVirtualPointerLoadHandle.IsValid() && DefaultVirtualPointerLoadHandle->IsActive())
	{
		return nullptr;
	}

	// Kick off the one and only async load. CreateSP with SharedThis keeps the cursor alive until the
	// callback fires (cancelled in Deinitialize if the cursor is torn down before completion).
	DefaultVirtualPointerLoadHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(
		VirtualPointerClass.ToSoftObjectPath(),
		FStreamableDelegate::CreateSP(SharedThis(this), &FCommonAnalogCursor::OnDefaultVirtualPointerClassLoaded));

	return nullptr;
}

void FCommonAnalogCursor::OnDefaultVirtualPointerClassLoaded()
{
	DefaultVirtualPointerLoadHandle.Reset();

	const TSoftClassPtr<UUserWidget>& VirtualPointerClass = UCommonUIInputSettings::Get().GetDefaultVirtualPointerClass();
	UClass* LoadedClass = VirtualPointerClass.Get();
	if (!LoadedClass)
	{
		// Asset failed to resolve - flag so we never re-request. Fixing the config requires a session restart.
		bDefaultVirtualPointerLoadFailed = true;
		return;
	}

	if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(ActionRouter.GetLocalPlayer()))
	{
		if (APlayerController* PC = LocalPlayer->GetPlayerController(ActionRouter.GetWorld()))
		{
			DefaultVirtualPointerWidget = CreateWidget(PC, LoadedClass);
			// Fire an immediate state recompute so the widget transitions from its constructed default into
			// whatever the cursor's current input state is (e.g. gamepad-Pressed if accept was held while loading).
			UpdateVirtualPointerVisualState();
		}
	}
	// If LP/PC is unavailable here, leave DefaultVirtualPointerWidget null. The next call to
	// GetOrLoadDefaultVirtualPointer will hit the "class is in memory" branch and try widget creation
	// again - no further async load needed.
}

void FCommonAnalogCursor::UpdateVirtualPointerVisualState()
{
	UUserWidget* Widget = VirtualPointerWidget ? VirtualPointerWidget : DefaultVirtualPointerWidget;
	if (!Widget || !Widget->Implements<UVirtualPointerVisualStateInterface>())
	{
		return;
	}

	ECursorVisualState NewStates = ECursorVisualState::Default;

	// Only compute active flags while the virtual pointer is actually in use. Otherwise NewStates stays
	// Default and the change-detection below will fire a final event clearing any active flags so the
	// widget can return to its idle visual.
	if (IsUsingVirtualPointer())
	{
		// Hover: read Slate's cached widget-path-under-cursor (FSlateUser::GetLastWidgetsUnderCursor) instead
		// of running a fresh hittest. Slate refreshes that cache on every mouse move event - including the
		// synthetic ones generated when the analog stick drives SetCursorPosition - so it tracks our virtual
		// pointer's location without us paying for the hittest grid traversal each tick.
		// Skip the game viewport (it supports keyboard focus so the game receives input) and any descendant
		// of our own cursor widget (defensive - it is normally drawn via UGameViewportClient::MapCursor
		// outside the hittest grid, but a project-supplied widget may end up in the hit path).
		FSlateApplication& SlateApp = FSlateApplication::Get();
		if (TSharedPtr<FSlateUser> SlateUser = SlateApp.GetUser(GetOwnerUserIndex()))
		{
			const FWeakWidgetPath WeakPath = SlateUser->GetLastWidgetsUnderCursor();
			if (WeakPath.IsValid())
			{
				const UGameViewportClient* ViewportClient = GetViewportClient();
				const TSharedPtr<SViewport> GameViewport = ViewportClient ? ViewportClient->GetGameViewportWidget() : nullptr;
				const TSharedPtr<SWidget> CursorSWidget = Widget->GetCachedWidget();

				auto IsCursorWidgetOrDescendant = [&CursorSWidget](const TSharedRef<SWidget>& In) -> bool
				{
					if (!CursorSWidget.IsValid())
					{
						return false;
					}
					TSharedPtr<SWidget> Current = In;
					while (Current.IsValid())
					{
						if (Current.Get() == CursorSWidget.Get())
						{
							return true;
						}
						Current = Current->GetParentWidget();
					}
					return false;
				};

				for (const TWeakPtr<SWidget>& WeakHit : WeakPath.Widgets)
				{
					TSharedPtr<SWidget> HitWidget = WeakHit.Pin();
					if (!HitWidget.IsValid())
					{
						continue;
					}
					const bool bIsViewport = GameViewport.IsValid() && HitWidget.Get() == GameViewport.Get();
					if (HitWidget->SupportsKeyboardFocus() && !bIsViewport && !IsCursorWidgetOrDescendant(HitWidget.ToSharedRef()))
					{
						NewStates |= ECursorVisualState::Hover;
						break;
					}
				}
			}
		}

		// Pressed / Hold (time-based): progress runs from 0 to 1 over MaxHoldDuration starting at press.
		if (bVirtualAcceptDown)
		{
			NewStates |= ECursorVisualState::Pressed;

			if (HoldElapsedTime > 0.f)
			{
				NewStates |= ECursorVisualState::Hold;

				const FCommonAnalogCursorSettings& Settings = UCommonUIInputSettings::Get().GetAnalogCursorSettings();
				const float Progress = FMath::Clamp(HoldElapsedTime / FMath::Max(Settings.MaxHoldDuration, UE_SMALL_NUMBER), 0.f, 1.f);
				IVirtualPointerVisualStateInterface::Execute_OnVirtualPointerHoldProgress(Widget, Progress);
			}

			// Drag: only when the user is actively dragging - holding accept while moving the analog stick.
			if (GetAnalogValues(EAnalogStick::Left).SizeSquared() > FMath::Square(DeadZone))
			{
				NewStates |= ECursorVisualState::Drag;
			}
		}
	}

	if (NewStates == CurrentVirtualPointerVisualState)
	{
		return;
	}

	const ECursorVisualState OldStates = CurrentVirtualPointerVisualState;
	CurrentVirtualPointerVisualState = NewStates;
	IVirtualPointerVisualStateInterface::Execute_OnVirtualPointerVisualStateChanged(Widget, OldStates, NewStates);
}

#undef LOCTEXT_NAMESPACE
