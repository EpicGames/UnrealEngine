// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/CommonBoundActionBar.h"

#include "CommonInputSubsystem.h"
#include "CommonInputTypeEnum.h"
#include "CommonUITypes.h"
#include "Editor/WidgetCompilerLog.h"
#include "EnhancedInputSubsystems.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateUser.h"
#include "InputAction.h"
#include "Input/CommonBoundActionButtonInterface.h"
#include "Input/CommonUIActionRouterBase.h"
#include "Input/UIActionBinding.h"
#include "Layout/WidgetPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonBoundActionBar)

#define LOCTEXT_NAMESPACE "CommonUI"

bool bActionBarIgnoreOptOut = false;
static FAutoConsoleVariableRef CVarActionBarIgnoreOptOut(
	TEXT("ActionBar.IgnoreOptOut"),
	bActionBarIgnoreOptOut,
	TEXT("If true, the Bound Action Bar will display bindings whether or not they are configured bDisplayInReflector"),
	ECVF_Default
);

void UCommonBoundActionBar::SetDisplayOwningPlayerActionsOnly(bool bShouldOnlyDisplayOwningPlayerActions)
{
	if (bShouldOnlyDisplayOwningPlayerActions != bDisplayOwningPlayerActionsOnly)
	{
		bDisplayOwningPlayerActionsOnly = bShouldOnlyDisplayOwningPlayerActions;
		if (!IsDesignTime())
		{
			HandleBoundActionsUpdated(true);
		}
	}
}

void UCommonBoundActionBar::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();

	if (const UGameInstance* GameInstance = GetGameInstance())
	{
		if (GameInstance->GetGameViewportClient())
		{
			GameInstance->GetGameViewportClient()->OnPlayerAdded().RemoveAll(this);
			GameInstance->GetGameViewportClient()->OnPlayerRemoved().RemoveAll(this);

			GameInstance->GetGameViewportClient()->OnPlayerAdded().AddUObject(this, &UCommonBoundActionBar::HandlePlayerAdded);
			GameInstance->GetGameViewportClient()->OnPlayerRemoved().AddUObject(this, &UCommonBoundActionBar::HandlePlayerRemoved);
		}

		for (const ULocalPlayer* LocalPlayer : GameInstance->GetLocalPlayers())
		{
			MonitorPlayerActions(LocalPlayer);
		}

		// Establish entries (as needed) immediately upon construction
		HandleDeferredDisplayUpdate();
	}
}

void UCommonBoundActionBar::SynchronizeProperties()
{
	Super::SynchronizeProperties();
}

void UCommonBoundActionBar::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	
	if (const UGameInstance* GameInstance = GetGameInstance())
	{
		for (const ULocalPlayer* LocalPlayer : GameInstance->GetLocalPlayers())
		{
			if (const UCommonUIActionRouterBase* ActionRouter = ULocalPlayer::GetSubsystem<UCommonUIActionRouterBase>(LocalPlayer))
			{
				ActionRouter->OnBoundActionsUpdated().RemoveAll(this);
			}

			if (UCommonInputSubsystem* InputSubsystem = UCommonInputSubsystem::Get(LocalPlayer))
			{
				InputSubsystem->OnInputMethodChangedNative.RemoveAll(this);
			}
		}
	}
}

#if WITH_EDITOR
void UCommonBoundActionBar::ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const
{
	Super::ValidateCompiledDefaults(CompileLog);

	if (!ActionButtonClass)
	{
		CompileLog.Error(FText::Format(LOCTEXT("Error_BoundActionBar_MissingButtonClass", "{0} has no ActionButtonClass specified."), FText::FromString(GetName())));
	}
	else if (CompileLog.GetContextClass() && ActionButtonClass->IsChildOf(CompileLog.GetContextClass()))
	{
		CompileLog.Error(FText::Format(LOCTEXT("Error_BoundActionBar_RecursiveButtonClass", "{0} has a recursive ActionButtonClass specified (reference itself)."), FText::FromString(GetName())));
	}

}
#endif

void UCommonBoundActionBar::HandleBoundActionsUpdated(bool bFromOwningPlayer)
{
	if (bFromOwningPlayer || !bDisplayOwningPlayerActionsOnly)
	{
		UpdateDisplay();
	}
}

void UCommonBoundActionBar::HandleInputMappingsRebuiltUpdated()
{
	UpdateDisplay();
}

void UCommonBoundActionBar::UpdateDisplay()
{
	if (!bIsRefreshQueued)
	{
		bIsRefreshQueued = true;

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [this](float InDelta)
			{
				if (IsSafeToUpdateDisplay())
				{
					HandleDeferredDisplayUpdate();
					return false;
				}
				return true;
			}));
	}
}

bool UCommonBoundActionBar::IsSafeToUpdateDisplay() const
{
	return !DoAnyActionButtonsHaveMouseCapture();
}

void UCommonBoundActionBar::HandleDeferredDisplayUpdate()
{
	ActionBarUpdateBegin();
	
	bIsRefreshQueued = false;

	ResetInternal();

	const UGameInstance* GameInstance = GetGameInstance();
	check(GameInstance);
	const ULocalPlayer* OwningLocalPlayer = GetOwningLocalPlayer();

	// Sort the player list so our owner is at the end
	TArray<ULocalPlayer*> SortedPlayers = GameInstance->GetLocalPlayers();
	SortedPlayers.StableSort(
		[&OwningLocalPlayer](const ULocalPlayer& PlayerA, const ULocalPlayer& PlayerB)
		{
			return &PlayerA != OwningLocalPlayer;
		});

	for (const ULocalPlayer* LocalPlayer : SortedPlayers)
	{
		if (LocalPlayer == OwningLocalPlayer || !bDisplayOwningPlayerActionsOnly)
		{
			if (IsEntryClassValid(ActionButtonClass))
			{
				if (const UCommonUIActionRouterBase* ActionRouter = ULocalPlayer::GetSubsystem<UCommonUIActionRouterBase>(LocalPlayer))
				{
					const UCommonInputSubsystem& InputSubsystem = ActionRouter->GetInputSubsystem();
					const ECommonInputType PlayerInputType = InputSubsystem.GetCurrentInputType();
					const FName& PlayerGamepadName = InputSubsystem.GetCurrentGamepadName();

					TSet<FName> AcceptedBindings;
					TArray<FUIActionBindingHandle> FilteredBindings = ActionRouter->GatherActiveBindings().FilterByPredicate([ActionRouter, PlayerInputType, PlayerGamepadName, &AcceptedBindings, this](const FUIActionBindingHandle& Handle) mutable
						{
							if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(Handle))
							{
								if (!Binding->bDisplayInActionBar && !bActionBarIgnoreOptOut)
								{
									return false;
								}

								const bool bIsValidEnhancedInputAction = Binding->InputTypesExemptFromValidKeyCheck.Contains(PlayerInputType) ||
									(CommonUI::IsEnhancedInputSupportEnabled() && CommonUI::ActionValidForInputType(ActionRouter->GetLocalPlayer(), PlayerInputType, Binding->InputAction.Get()));
								if (!bIsValidEnhancedInputAction)
								{
									const bool bIsValidDataTableInputAction = CommonUI::ActionValidForInputType(ActionRouter->GetLocalPlayer(), PlayerInputType, Binding->GetLegacyInputActionData());
									if (!bIsValidDataTableInputAction)
									{
										return false;
									}
								}

								if (!bIgnoreDuplicateActions)
								{
									return true;
								}
								bool bAlreadyAccepted = false;
								AcceptedBindings.Add(Binding->ActionName, &bAlreadyAccepted);
								return !bAlreadyAccepted;
							}

							return false;
						});

					//Force Virtual_Back to one end of the list so Back actions are always consistent.
					//Otherwise, order within a node is controlled by order of add/remove.
					Algo::Sort(FilteredBindings, [ActionRouter, PlayerInputType, PlayerGamepadName](const FUIActionBindingHandle& A, const FUIActionBindingHandle& B)
					{
						TSharedPtr<FUIActionBinding> BindingA = FUIActionBinding::FindBinding(A);
						TSharedPtr<FUIActionBinding> BindingB = FUIActionBinding::FindBinding(B);

						if (ensureMsgf((BindingA && BindingB), TEXT("The array filter above should enforce that there are no null bindings")))
						{
							auto IsKeyBackAction = [ActionRouter, PlayerInputType, PlayerGamepadName](FCommonInputActionDataBase* LegacyData, const UInputAction* InputAction)
							{
								if (LegacyData)
								{
									FKey Key = LegacyData->GetInputTypeInfo(PlayerInputType, PlayerGamepadName).GetKey();

									// Fallback back to keyboard key when there is no key for touch
									if (PlayerInputType == ECommonInputType::Touch)
									{
										if (!Key.IsValid())
										{
											Key = LegacyData->GetInputTypeInfo(ECommonInputType::MouseAndKeyboard, PlayerGamepadName).GetKey();
										}
									}

									return Key == EKeys::Virtual_Gamepad_Back.GetVirtualKey() || Key == EKeys::Escape || Key == EKeys::Android_Back;
								}
								else if (InputAction)
								{
									FKey Key = CommonUI::GetFirstKeyForInputType(ActionRouter->GetLocalPlayer(), PlayerInputType, InputAction);

									// Fallback back to keyboard key when there is no key for touch
									if (PlayerInputType == ECommonInputType::Touch)
									{
										if (!Key.IsValid())
										{
											Key = CommonUI::GetFirstKeyForInputType(ActionRouter->GetLocalPlayer(), ECommonInputType::MouseAndKeyboard, InputAction);
										}
									}

									return Key == EKeys::Virtual_Gamepad_Back.GetVirtualKey() || Key == EKeys::Escape || Key == EKeys::Android_Back;
								}

								return false;
							};

							auto GetNavBarPriority = [](FCommonInputActionDataBase* LegacyData, const UInputAction* InputAction)
							{
								if (LegacyData)
								{
									return LegacyData->NavBarPriority;
								}
								else if (InputAction)
								{
									if (TObjectPtr<const UCommonInputMetadata> InputActionMetadata = CommonUI::GetEnhancedInputActionMetadata(InputAction))
									{
										return InputActionMetadata->NavBarPriority;
									}
								}

								return 0;
							};

							FCommonInputActionDataBase* LegacyDataA = BindingA->GetLegacyInputActionData();
							FCommonInputActionDataBase* LegacyDataB = BindingB->GetLegacyInputActionData();

							const UInputAction* InputActionA = nullptr;
							const UInputAction* InputActionB = nullptr;

							// InputActions will be null unless IsEnhancedInputSupportEnabled, so we don't have to check for support below
							if (CommonUI::IsEnhancedInputSupportEnabled())
							{
								InputActionA = BindingA->InputAction.Get();
								InputActionB = BindingB->InputAction.Get();
							}

							bool bIsValidActionA = LegacyDataA || InputActionA;
							bool bIsValidActionB = LegacyDataB || InputActionB;

							if (ensureMsgf(bIsValidActionA, TEXT("Binding is invalid: %s"), *BindingA->ToDebugString())
								&& ensureMsgf(bIsValidActionB, TEXT("Binding is invalid: %s"), *BindingB->ToDebugString()))
							{
								bool bAIsBack = IsKeyBackAction(LegacyDataA, InputActionA);
								bool bBIsBack = IsKeyBackAction(LegacyDataB, InputActionB);

								if (bAIsBack && !bBIsBack)
								{
									return false;
								}

								int32 NavBarPriorityA = GetNavBarPriority(LegacyDataA, InputActionA);
								int32 NavBarPriorityB = GetNavBarPriority(LegacyDataB, InputActionB);

								if (NavBarPriorityA != NavBarPriorityB)
								{
									return NavBarPriorityA < NavBarPriorityB;
								}
							}

							return GetTypeHash(BindingA->Handle) < GetTypeHash(BindingB->Handle);
						}

						return true; // A < B by default
					});


					for (FUIActionBindingHandle BindingHandle : FilteredBindings)
					{
						ICommonBoundActionButtonInterface* ActionButton = Cast<ICommonBoundActionButtonInterface>(CreateActionButton(BindingHandle));
						if (ensure(ActionButton))
						{
							ActionButton->SetRepresentedAction(BindingHandle);
							NativeOnActionButtonCreated(ActionButton, BindingHandle);
						}
					}
				}
			}
		}
	}

	OnActionBarUpdated.Broadcast();
	ActionBarUpdateEnd();
}

UUserWidget* UCommonBoundActionBar::CreateActionButton(const FUIActionBindingHandle& BindingHandle)
{
	return CreateEntryInternal(ActionButtonClass);
}

void UCommonBoundActionBar::HandlePlayerAdded(int32 PlayerIdx)
{
	const ULocalPlayer* NewPlayer = GetGameInstance()->GetLocalPlayerByIndex(PlayerIdx);
	MonitorPlayerActions(NewPlayer);
	HandleBoundActionsUpdated(NewPlayer == GetOwningLocalPlayer());
}

void UCommonBoundActionBar::HandlePlayerRemoved(int32 PlayerIdx)
{
	const ULocalPlayer* RemovedPlayer = GetGameInstance()->GetLocalPlayerByIndex(PlayerIdx);
	HandleBoundActionsUpdated(RemovedPlayer == GetOwningLocalPlayer());
}

void UCommonBoundActionBar::HandledInputTypeUpdated(ECommonInputType InputType)
{
	UpdateDisplay();
}

void UCommonBoundActionBar::MonitorPlayerActions(const ULocalPlayer* NewPlayer)
{
	if (const UCommonUIActionRouterBase* ActionRouter = ULocalPlayer::GetSubsystem<UCommonUIActionRouterBase>(NewPlayer))
	{
		ActionRouter->OnBoundActionsUpdated().AddUObject(this, &UCommonBoundActionBar::HandleBoundActionsUpdated, NewPlayer == GetOwningLocalPlayer());
	}

	/**
	 * Update available inputs anytime the input type changes;
	 * in Enhanced Input, an input action might be only bound with one device (gamepad), but not others (KBM / mouse),
	 * due to that they will be considered invalid in case the action bar is updated when using the device that doesn't
	 * have them bound to anything. When switching to a device that actually binds them to an input, the action bar
	 * has to refresh to include the input actions that were considered invalid prior to that
	 */
	if (UCommonInputSubsystem* InputSubsystem = UCommonInputSubsystem::Get(NewPlayer))
	{
		InputSubsystem->OnInputMethodChangedNative.AddUObject(this, &UCommonBoundActionBar::HandledInputTypeUpdated);
	}

	if (CommonUI::IsEnhancedInputSupportEnabled())
	{
		// need to check the owning player here rather than the in the callback because the dynamic delegates can't have extra params. So just don't subscribe if not needed
		const bool bFromOwningPlayer = NewPlayer == GetOwningLocalPlayer();
		if (bFromOwningPlayer || !bDisplayOwningPlayerActionsOnly)
		{
			if (UEnhancedInputLocalPlayerSubsystem* EnhancedInputLocalPlayerSubsystem = ULocalPlayer::GetSubsystem<UEnhancedInputLocalPlayerSubsystem>(NewPlayer))
			{
				EnhancedInputLocalPlayerSubsystem->ControlMappingsRebuiltDelegate.AddUniqueDynamic(this, &UCommonBoundActionBar::HandleInputMappingsRebuiltUpdated);
			}
		}
	}
}

void UCommonBoundActionBar::ActionBarUpdateBegin()
{
	ActionBarUpdateBeginImpl();
}

void UCommonBoundActionBar::ActionBarUpdateEnd()
{
	ActionBarUpdateEndImpl();
}

bool UCommonBoundActionBar::DoAnyActionButtonsHaveMouseCapture() const
{
	if (const ULocalPlayer* LocalPlayer = GetOwningLocalPlayer())
	{
		if (TSharedPtr<const FSlateUser> SlateUser = LocalPlayer->GetSlateUser())
		{
			if (SlateUser->HasAnyCapture())
			{
				const FWeakWidgetPath CapturePath = SlateUser->GetWeakCursorCapturePath();
				for (const UUserWidget* Element : GetAllEntries())
				{
					if (CapturePath.ContainsWidget(Element->GetCachedWidget().Get()))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
