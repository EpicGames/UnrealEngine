// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/CommonBoundActionButton.h"
#include "CommonTextBlock.h"
#include "Input/UIActionBinding.h"
#include "CommonActionWidget.h"
#include "CommonUITypes.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/LocKeyFuncs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonBoundActionButton)

#define LOCTEXT_NAMESPACE "CommonBoundActionButton"

void UCommonBoundActionButton::SetRepresentedAction(FUIActionBindingHandle InBindingHandle)
{
	if (TSharedPtr<FUIActionBinding> OldBinding = FUIActionBinding::FindBinding(BindingHandle))
	{
		OldBinding->OnHoldActionProgressed.RemoveAll(this);
	}

	BindingHandle = InBindingHandle;
	UpdateInputActionWidget();

	if (TSharedPtr<FUIActionBinding> NewBinding = FUIActionBinding::FindBinding(InBindingHandle))
	{
		NewBinding->OnHoldActionProgressed.AddUObject(this, &UCommonBoundActionButton::NativeOnActionProgress);

		if (bLinkRequiresHoldToBindingHold)
		{
			const FCommonInputActionDataBase* InputActionData = NewBinding->GetLegacyInputActionData();
			SetRequiresHold(InputActionData ? InputActionData->HasHoldBindings() : false);
		}
	}
}

void UCommonBoundActionButton::NativeOnClicked()
{
	Super::NativeOnClicked();

	if (TSharedPtr<FUIActionBinding> ActionBinding = FUIActionBinding::FindBinding(BindingHandle))
	{
		ActionBinding->OnExecuteAction.ExecuteIfBound();
	}
}

void UCommonBoundActionButton::NativeOnCurrentTextStyleChanged()
{
	Super::NativeOnCurrentTextStyleChanged();

	if (Text_ActionName)
	{
		Text_ActionName->SetStyle(GetCurrentTextStyleClass());
	}
}

void UCommonBoundActionButton::UpdateInputActionWidget()
{
	if (InputActionWidget) //optional bound widget
	{
		InputActionWidget->SetInputActionBinding(BindingHandle);
		
		FText ActionDisplayName = BindingHandle.GetDisplayName();
		if (BindingHandle.IsValid())
		{
			ULocalPlayer* BindingOwner = BindingHandle.GetBoundLocalPlayer();
			if (ensure(BindingOwner) && BindingOwner != GetOwningLocalPlayer())
			{
				TOptional<int32> BoundPlayerIndex = FSlateApplication::Get().GetUserIndexForController(BindingOwner->GetControllerId());
				if (BoundPlayerIndex.IsSet())
				{
					// This action belongs to a player that isn't our owner, so append a little indicator of the player it's for
					ActionDisplayName = FText::FormatNamed(LOCTEXT("OtherPlayerActionFormat", "[P{PlayerNum}] {ActionName}"),
						TEXT("PlayerNum"), FText::AsNumber(BoundPlayerIndex.GetValue() + 1),
						TEXT("ActionName"), ActionDisplayName);
				}
			}
		}

		if (Text_ActionName)
		{
			Text_ActionName->SetText(ActionDisplayName);
		}
		
		OnUpdateInputAction();
	}
}

void UCommonBoundActionButton::UpdateHoldData(ECommonInputType CurrentInputType)
{
	if (bLinkRequiresHoldToBindingHold)
	{
		if (const TSharedPtr<FUIActionBinding> ActionBinding = FUIActionBinding::FindBinding(BindingHandle))
		{
			if (const FCommonInputActionDataBase* DataTableRow = ActionBinding->GetLegacyInputActionData())
			{
				const FCommonInputTypeInfo InputTypeInfo = DataTableRow->GetCurrentInputTypeInfo(GetInputSubsystem());
				HoldTime = InputTypeInfo.HoldTime;
				HoldRollbackTime = InputTypeInfo.HoldRollbackTime;
			}
		}
	}
	else
	{
		Super::UpdateHoldData(CurrentInputType);
	}
}

#undef LOCTEXT_NAMESPACE
