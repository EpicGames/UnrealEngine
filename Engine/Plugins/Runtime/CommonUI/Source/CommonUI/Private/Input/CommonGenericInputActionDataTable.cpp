// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/CommonGenericInputActionDataTable.h"
#include "CommonUITypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonGenericInputActionDataTable)

UCommonGenericInputActionDataTable::UCommonGenericInputActionDataTable()
{
	RowStruct = FCommonInputActionDataBase::StaticStruct();
}

void UCommonGenericInputActionDataTable::PostLoad()
{
	Super::PostLoad();

	TArray<UClass*> ProcessorClasses;
	GetDerivedClasses(UCommonInputActionDataProcessor::StaticClass(), ProcessorClasses, false);

	for (UClass* ProcessorClass : ProcessorClasses)
	{
		if (UCommonInputActionDataProcessor* ProcessorCDO = Cast<UCommonInputActionDataProcessor>(ProcessorClass->GetDefaultObject()))
		{
			ProcessorCDO->ProcessInputActions(this);
		}
	}
}

void UCommonInputActionDataProcessor::ProcessInputActions(UCommonGenericInputActionDataTable* InputActionDataTable)
{
}

