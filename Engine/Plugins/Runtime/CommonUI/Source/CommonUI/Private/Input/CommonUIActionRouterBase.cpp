// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/CommonUIActionRouterBase.h"

#include "CommonActivatableWidget.h"
#include "CommonGameViewportClient.h"
#include "CommonInputSubsystem.h"
#include "CommonInputTypeEnum.h"
#include "CommonUIUtils.h"
#include "Engine/Canvas.h"
#include "Engine/Console.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "EnhancedInputSubsystems.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Framework/Commands/InputBindingManager.h"
#include "GameFramework/HUD.h"
#include "Input/CommonAnalogCursor.h"
#include "Input/CommonUIInputSettings.h"
#include "Input/UIActionBinding.h"
#include "Input/UIActionRouterTypes.h"
#include "NativeGameplayTags.h"
#include "Slate/SGameLayerManager.h"
#include "Slate/SObjectWidget.h"
#include "TimerManager.h"
#include "Widgets/SViewport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUIActionRouterBase)

//////////////////////////////////////////////////////////////////////////

bool bAlwaysShowCursor = false;
static const FAutoConsoleVariableRef CVarAlwaysShowCursor(
	TEXT("CommonUI.AlwaysShowCursor"),
	bAlwaysShowCursor,
	TEXT(""));

bool bAutoFlushPressedKeys = true;
static const FAutoConsoleVariableRef CVarAutoFlushInput(
	TEXT("CommonUI.AutoFlushPressedKeys"),
	bAutoFlushPressedKeys,
	TEXT("Causes the pressed keys to be flushed when the Input Mode is switched to Menu."));

bool bResetUIInputConfigOnActivatableTreeDeactivation = true;
static const FAutoConsoleVariableRef CVarResetUIInputConfigOnActivatableTreeDeactivation(
	TEXT("CommonUI.ResetUIInputConfigOnActivatableTreeDeactivation"),
	bResetUIInputConfigOnActivatableTreeDeactivation,
	TEXT("Controls if input config is reset when root is changed via deactivation."));

bool bSupportMultiUserInput = true;
static const FAutoConsoleVariableRef CVarSupportMultiUserInput(
	TEXT("CommonUI.SupportMultiUserInput"),
	bSupportMultiUserInput,
	TEXT("Whether or not action routers can forward inputs to other action routers to support widgets binding inputs for multiple local players"));

//////////////////////////////////////////////////////////////////////////

static bool bTraceInputConfig = false;
static FAutoConsoleVariableRef CVarTraceInputConfig(
	TEXT("CommonUI.Debug.TraceConfigChanges"),
	bTraceInputConfig,
	TEXT("Trace Input Config transitions (Non-shipping). Suggest use with Slate.Debug.TraceNavigationConfig (Non-shipping)."));

static bool bTraceConfigOnScreen = false;
static FAutoConsoleVariableRef CVarTraceConfigOnScreen(
	TEXT("CommonUI.Debug.TraceConfigOnScreen"),
	bTraceConfigOnScreen,
	TEXT("Trace for input configs should be displayed on screen. Requires CommonUI.Debug.TraceConfigChanges."));

static int32 TraceInputConfigNum = 5;
static FAutoConsoleVariableRef CVarTraceInputConfigNum(
	TEXT("CommonUI.Debug.TraceInputConfigNum"),
	TraceInputConfigNum,
	TEXT("Number of Input config to keep in trace history."),
	ECVF_ReadOnly);

static bool bWarnAllWidgetsDeactivated = false;
static FAutoConsoleVariableRef CVarWarnAllWidgetsDeactivated(
	TEXT("CommonUI.Debug.WarnAllWidgetsDeactivated"),
	bWarnAllWidgetsDeactivated,
	TEXT("Warn when all widgets are deactivated. A valid event, but may have leftover input configs."));

static bool bCheckGameViewportClientValid = true;
static FAutoConsoleVariableRef CVarCheckGameViewportClientValid(
	TEXT("CommonUI.Debug.CheckGameViewportClientValid"),
	bCheckGameViewportClientValid,
	TEXT("Log error when CommonUI is used without the current game viewport deriving from CommonGameViewportClient."));

//////////////////////////////////////////////////////////////////////////

UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_InputModeGame, "InputMode.Game");
UE_DEFINE_GAMEPLAY_TAG_STATIC(TAG_InputModeMenu, "InputMode.Menu");

//////////////////////////////////////////////////////////////////////////

FGlobalUITags FGlobalUITags::GUITags;

UCommonActivatableWidget* FindOwningActivatable(const UWidget& Widget)
{
	TSharedPtr<SWidget> CurWidget = Widget.GetCachedWidget();
	ULocalPlayer* OwningLocalPlayer = Widget.GetOwningLocalPlayer();

	return UCommonUIActionRouterBase::FindOwningActivatable(CurWidget, OwningLocalPlayer);
}

//////////////////////////////////////////////////////////////////////////
// FPersistentActionCollection
//////////////////////////////////////////////////////////////////////////

class FPersistentActionCollection : public FActionRouterBindingCollection
{
public:
	FPersistentActionCollection(UCommonUIActionRouterBase& ActionRouter)
		: FActionRouterBindingCollection(ActionRouter)
	{}

	void DumpActionBindings(FString& OutputStr) const
	{
		OutputStr.Append(TEXT("\nPersistent Action Collection:"));
		DebugDumpActionBindings(OutputStr, 0);
	}

	FString DumpActionBindings() const
	{
		FString OutStr;
		DumpActionBindings(OutStr);
		return OutStr;
	}
	
};

//////////////////////////////////////////////////////////////////////////
// UCommonUIActionRouterBase
//////////////////////////////////////////////////////////////////////////

namespace UE::CommonUI::Private
{
	enum class ESearchType
	{
		IncludeSelf,
		ExcludeSelf
	};
	template<ESearchType SearchType, typename TPredicate>
	void ForEachParentWidget(TSharedRef<SWidget> Widget, TPredicate Predicate)
	{
		TSharedPtr<SWidget> TestWidget = Widget;
		if constexpr (SearchType == ESearchType::ExcludeSelf)
		{
			TestWidget = Widget->GetParentWidget();
		}
		while (TestWidget)
		{
			const bool bContinueIterating = Predicate(TestWidget.ToSharedRef());
			if (!bContinueIterating)
			{
				return;
			}
			
			TestWidget = TestWidget->GetParentWidget();
		}
	}

	
	template<ESearchType SearchType>
	UCommonActivatableWidget* FindActivatableFromSlate(TSharedPtr<SWidget> SlateWidget, ULocalPlayer* OwningLocalPlayer)
	{
		UCommonActivatableWidget* OwningActivatable = nullptr;
		if (SlateWidget.IsValid())
		{
			ForEachParentWidget<SearchType>(SlateWidget.ToSharedRef(), [&OwningActivatable, OwningLocalPlayer](const TSharedRef<SWidget>& Widget)
			{
				if (Widget->GetMetaData<FCommonActivatableSlateMetaData>().IsValid())
				{
					if (UCommonActivatableWidget* CandidateActivatable = Cast<UCommonActivatableWidget>(StaticCastSharedRef<SObjectWidget>(Widget)->GetWidgetObject()))
					{
						if (OwningLocalPlayer == nullptr || CandidateActivatable->GetOwningLocalPlayer() == OwningLocalPlayer)
						{
							OwningActivatable = CandidateActivatable;
						}
						return false;
					}
				}
				return true;
			});
		}
		return OwningActivatable;
	}
}

UCommonUIActionRouterBase* UCommonUIActionRouterBase::Get(const UWidget& ContextWidget)
{
	return ULocalPlayer::GetSubsystem<UCommonUIActionRouterBase>(ContextWidget.GetOwningLocalPlayer());
}

UCommonActivatableWidget* UCommonUIActionRouterBase::FindOwningActivatable(TSharedPtr<SWidget> Widget, ULocalPlayer* OwningLocalPlayer)
{
	return UE::CommonUI::Private::FindActivatableFromSlate<UE::CommonUI::Private::ESearchType::ExcludeSelf>(Widget, OwningLocalPlayer);
}

UCommonActivatableWidget* UCommonUIActionRouterBase::FindActivatable(TSharedPtr<SWidget> Widget, ULocalPlayer* OwningLocalPlayer)
{
	return UE::CommonUI::Private::FindActivatableFromSlate<UE::CommonUI::Private::ESearchType::IncludeSelf>(Widget, OwningLocalPlayer);
}

UCommonUIActionRouterBase::UCommonUIActionRouterBase()
	: PersistentActions(MakeShared<FPersistentActionCollection>(*this))
	, InputConfigSources(TraceInputConfigNum, "None")
{
	// Non-CDO behavior
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Register "showdebug" hook.
		if (!IsRunningDedicatedServer())
		{
			AHUD::OnShowDebugInfo.AddUObject(this, &UCommonUIActionRouterBase::OnShowDebugInfo);
		}

		UConsole::RegisterConsoleAutoCompleteEntries.AddUObject(this, &UCommonUIActionRouterBase::PopulateAutoCompleteEntries);
	}
}

FUIActionBindingHandle UCommonUIActionRouterBase::RegisterUIActionBinding(const UWidget& Widget, const FBindUIActionArgs& BindActionArgs)
{
	FUIActionBindingHandle BindingHandle = FUIActionBinding::TryCreate(Widget, BindActionArgs, GetLocalPlayerIndex());
	if (BindingHandle.IsValid())
	{
		FActivatableTreeNodePtr OwnerNode = nullptr;
		if (const UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(&Widget))
		{
			// For an activatable widget, we want the node that pertains specifically to this widget.
			// We don't want to associate the action with one of its parents; we just want to wait for its node to be constructed.
			OwnerNode = FindNode(ActivatableWidget);
		}
		else
		{
			// For non-activatable widgets, we will accept the nearest parent node.
			OwnerNode = FindOwningNode(Widget);
		}

		if (OwnerNode)
		{
			if (const TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(BindingHandle))
			{
				if (Binding->bIsPersistent)
				{
					PersistentActions->AddBinding(*Binding);
				}
				else
				{
					OwnerNode->AddBinding(*Binding);
				}
			}
			
			if (UCommonActivatableWidget* ActivatableWidget = OwnerNode->GetWidget())
			{
				ActivatableWidget->RegisterInputTreeNode(OwnerNode);
			}
		}
		else if (Widget.GetCachedWidget())
		{
			// The widget is already constructed, but there's no node for it yet - defer for a frame
			FPendingWidgetRegistration& PendingRegistration = GetOrCreatePendingRegistration(Widget);
			PendingRegistration.ActionBindings.AddUnique(BindingHandle);
		}

		return BindingHandle;
	}

	return FUIActionBindingHandle();
}

bool UCommonUIActionRouterBase::RegisterLinkedPreprocessor(const UWidget& Widget, const TSharedRef<IInputProcessor>& InputPreprocessor, int32 DesiredIndex)
{
	return RegisterLinkedPreprocessor(Widget, InputPreprocessor, FInputPreprocessorRegistrationKey{ EInputPreProcessorType::Game, DesiredIndex });
}

bool UCommonUIActionRouterBase::RegisterLinkedPreprocessor(const UWidget& Widget, const TSharedRef<IInputProcessor>& InputPreprocessor)
{
	return RegisterLinkedPreprocessor(Widget, InputPreprocessor, FInputPreprocessorRegistrationKey());
}

bool UCommonUIActionRouterBase::RegisterLinkedPreprocessor(const UWidget& Widget, const TSharedRef<IInputProcessor>& InputPreprocessor, const FInputPreprocessorRegistrationKey& RegistrationInfo)
{
	if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(Widget))
	{
		OwnerNode->AddInputPreprocessor(InputPreprocessor, RegistrationInfo);
		return true;
	}
	else if (Widget.GetCachedWidget())
	{
		// The widget is already constructed, but there's no node for it yet - defer for a frame
		FPendingWidgetRegistration& PendingRegistration = GetOrCreatePendingRegistration(Widget);

		bool bAlreadyExists = false;
		for (FInputPreprocessorRegistration& Registration : PendingRegistration.InputPreProcessors)
		{
			if (Registration.InputProcessor == InputPreprocessor)
			{
				Registration.Info = RegistrationInfo;
				bAlreadyExists = true;
				break;
			}
		}

		if(!bAlreadyExists)
		{
			PendingRegistration.InputPreProcessors.Add(FInputPreprocessorRegistration{ RegistrationInfo, InputPreprocessor });
		}

		return true;
	}

	return false;
}

void UCommonUIActionRouterBase::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	UCommonInputSubsystem* InputSubsystem = Collection.InitializeDependency<UCommonInputSubsystem>();

	UCommonActivatableWidget::OnRebuilding.AddUObject(this, &UCommonUIActionRouterBase::HandleActivatableWidgetRebuilding);
	FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UCommonUIActionRouterBase::HandlePostGarbageCollect);

	if (FSlateApplication::IsInitialized())
	{
		if (ensure(InputSubsystem))
		{
			AnalogCursor = MakeAnalogCursor();
			PostAnalogCursorCreate();
			AnalogCursor->GetOnVirtualPointerEnabledChanged().AddUObject(this, &UCommonUIActionRouterBase::HandleVirtualPointerEnabled);

			if (bCheckGameViewportClientValid && !GEngine->GameViewportClientClass->IsChildOf<UCommonGameViewportClient>())
			{
				UE_LOGF(LogUIActionRouter, Error,
					"Using CommonUI without a CommonGameViewportClient derived game viewport client. CommonUI Input routing will not function correctly.\n" "To disable this warning set CommonUI.Debug.CheckGameViewportClientValid=0 under [SystemSettings] in your project's DefaultEngine.ini.");
			}
		}
		else
		{
			UE_LOGF(LogUIActionRouter, Warning, "Input system not initialized before action router!");
		}

		FSlateApplication::Get().OnFocusChanging().AddUObject(this, &UCommonUIActionRouterBase::HandleSlateFocusChanging);
	}
}

void UCommonUIActionRouterBase::PostAnalogCursorCreate()
{
	RegisterAnalogCursorTick();
}

void UCommonUIActionRouterBase::RegisterAnalogCursorTick()
{
	if (GEngine->GameViewportClientClass->IsChildOf<UCommonGameViewportClient>())
	{
		FSlateApplication::Get().RegisterInputPreProcessor(AnalogCursor, UCommonUIInputSettings::Get().GetAnalogCursorSettings().PreprocessorRegistrationInfo);
	}

	if (bIsActivatableTreeEnabled)
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UCommonUIActionRouterBase::Tick));
	}
}

void UCommonUIActionRouterBase::Deinitialize()
{
	Super::Deinitialize();

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication& SlateApplication = FSlateApplication::Get();

		SlateApplication.OnFocusChanging().RemoveAll(this);
		SlateApplication.UnregisterInputPreProcessor(AnalogCursor);
		if (AnalogCursor.IsValid())
		{
			AnalogCursor->GetOnVirtualPointerEnabledChanged().RemoveAll(this);
			AnalogCursor->Deinitialize();
			AnalogCursor.Reset();
		}

#if WITH_EDITOR
		ULocalPlayer* LocalPlayer = GetLocalPlayer();
		const bool bCursorUser = IsValid(LocalPlayer) && LocalPlayer->GetSlateUser() == SlateApplication.GetCursorUser();

		if (bCursorUser)
		{
			// This restores cursor visibility when exiting PIE while using a gamepad.
			SlateApplication.UsePlatformCursorForCursorUser(true);
		}
#endif
	}

	FTSTicker::RemoveTicker(TickHandle);
	SetActiveRoot(nullptr);
	HeldKeys.Empty();
}

bool UCommonUIActionRouterBase::ShouldCreateSubsystem(UObject* Outer) const
{
	TArray<UClass*> ChildClasses;
	GetDerivedClasses(GetClass(), ChildClasses, false);

	UE_LOGF(LogUIActionRouter, Display, "Found %i derived classes when attemping to create action router (%ls)", ChildClasses.Num(), *GetClass()->GetName());

	// Only create an instance if there is no override implementation defined elsewhere
	return ChildClasses.Num() == 0;
}

void UCommonUIActionRouterBase::SetIsActivatableTreeEnabled(bool bInIsTreeEnabled)
{
	bIsActivatableTreeEnabled = bInIsTreeEnabled;
	if (!bInIsTreeEnabled)
	{
		SetActiveRoot(nullptr);
	}
	else
	{
		RefreshActionDomainLeafNodeConfig();
	}
}

void UCommonUIActionRouterBase::RegisterScrollRecipient(const UWidget& ScrollableWidget, UWidget* OwningNodeWidget)
{
	if (OwningNodeWidget)
	{
		if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(*OwningNodeWidget))
		{
			OwnerNode->AddScrollRecipient(ScrollableWidget);
		}
	}
	else if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(ScrollableWidget))
	{
		OwnerNode->AddScrollRecipient(ScrollableWidget);
	}
	else
	{
		GetOrCreatePendingRegistration(ScrollableWidget).bIsScrollRecipient = true;
	}
}

void UCommonUIActionRouterBase::UnregisterScrollRecipient(const UWidget& ScrollableWidget, UWidget* OwningNodeWidget)
{
	if (OwningNodeWidget)
	{
		if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(*OwningNodeWidget))
		{
			OwnerNode->RemoveScrollRecipient(ScrollableWidget);
		}
	}
	else if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(ScrollableWidget))
	{
		OwnerNode->RemoveScrollRecipient(ScrollableWidget);
	}
	else if (FPendingWidgetRegistration* PendingRegistration = PendingWidgetRegistrations.FindByKey(ScrollableWidget))
	{
		PendingRegistration->bIsScrollRecipient = false;
	}
}

TArray<const UWidget*> UCommonUIActionRouterBase::GatherActiveAnalogScrollRecipients() const
{
	if (ActiveRootNode)
	{
		return ActiveRootNode->GatherScrollRecipients();
	}
	else if (FActivatableTreeRootPtr RootNode = FindActiveActionDomainRootNode())
	{
		return RootNode->GatherScrollRecipients();
	}

	return TArray<const UWidget*>();
}

TArray<FUIActionBindingHandle> UCommonUIActionRouterBase::GatherActiveBindings() const
{
	TArray<FUIActionBindingHandle> BindingHandles = PersistentActions->GetActionBindings();

	if (!bIsActivatableTreeEnabled)
	{
		// If we are ignoring the activatable tree, all active roots should be ignored.
		return BindingHandles;
	}

	if (ActiveRootNode)
	{
		ActiveRootNode->AppendAllActiveActions(BindingHandles);
	}
	
	if (const UCommonInputActionDomainTable* ActionDomainTable = GetActionDomainTable())
	{
		bool bDomainHadActiveRoots = false;

		for (const UCommonInputActionDomain* ActionDomain : ActionDomainTable->ActionDomains)
		{
			if (const FActionDomainSortedRootList* SortedRootList = ActionDomainRootNodes.Find(ActionDomain))
			{
				for (const FActivatableTreeRootRef& RootNode : SortedRootList->GetRootList())
				{
					if (RootNode->IsReceivingInput() && RootNode->IsWidgetActivated())
					{
						RootNode->AppendAllActiveActions(BindingHandles);
						bDomainHadActiveRoots = true;

						if (ActionDomain && ActionDomain->ShouldBreakInnerEventFlow(false))
						{
							break;
						}
					}
				}
			}

			if (ActionDomain && ActionDomain->ShouldBreakEventFlow(bDomainHadActiveRoots, false))
			{
				break;
			}
		}
	}

	return BindingHandles;
}

TSharedRef<FCommonAnalogCursor> UCommonUIActionRouterBase::MakeAnalogCursor() const
{
	// Override if desired and call CreateAnalogCursor<T> with a custom type
	return FCommonAnalogCursor::CreateAnalogCursor(*this);
}

ERouteUIInputResult UCommonUIActionRouterBase::ProcessInput(FKey Key, EInputEvent InputEvent) const
{
#if WITH_EDITOR
	// In PIE, check if the user is attempting to press the StopPlaySession command chord.
	if (GIsPlayInEditorWorld && InputEvent == IE_Pressed)
	{
		// @TODO: This could be more generic to be a list of command in commands to allow to be ignored by the UI action router.
		TSharedPtr<FUICommandInfo> StopCommand = FInputBindingManager::Get().FindCommandInContext("PlayWorld", "StopPlaySession");
		if (ensure(StopCommand))
		{
			const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
			const FInputChord CheckChord( Key, EModifierKey::FromBools(ModifierKeys.IsControlDown(), ModifierKeys.IsAltDown(), ModifierKeys.IsShiftDown(), ModifierKeys.IsCommandDown()) );

			// If the stop command matches the incoming key chord, let it execute.
			if (StopCommand->HasActiveChord(CheckChord))
			{
				return ERouteUIInputResult::Unhandled;
			}
		}
	}
#endif

	// Also check for repeat event here as if input is flushed when a key is being held, we will receive a released event and then continue to receive repeat events without a pressed event
	if (InputEvent == EInputEvent::IE_Pressed || InputEvent == EInputEvent::IE_Repeat)
	{
		HeldKeys.AddUnique(Key);
	}
	else if (InputEvent == EInputEvent::IE_Released)
	{
		HeldKeys.RemoveSwap(Key);
	}

	const ECommonInputMode ActiveMode = GetActiveInputMode();
	const int32 OwningUserIndex = GetLocalPlayerIndex();

	// Begin with a pass to see if the input corresponds to a hold action
	// We do this first to make sure that a higher-priority press binding doesn't prevent a hold on the same key from being triggerable
	const auto ProcessHoldInputFunc = [ActiveMode, Key, InputEvent, OwningUserIndex](const UCommonUIActionRouterBase& ActionRouter)
	{
		EProcessHoldActionResult ProcessHoldResult = ActionRouter.PersistentActions->ProcessHoldInput(ActiveMode, Key, InputEvent, OwningUserIndex);
	
		if (ProcessHoldResult == EProcessHoldActionResult::Unhandled && ActionRouter.bIsActivatableTreeEnabled)
		{
			if (ActionRouter.ActiveRootNode)
			{
				ProcessHoldResult = ActionRouter.ActiveRootNode->ProcessHoldInput(ActiveMode, Key, InputEvent, OwningUserIndex);
			}

			if (ProcessHoldResult == EProcessHoldActionResult::Unhandled)
			{
				ProcessHoldResult = ActionRouter.ProcessHoldInputOnActionDomains(ActiveMode, Key, InputEvent, OwningUserIndex);
			}
		}

		return ProcessHoldResult;
	};

	const auto ProcessNormalInputFunc = [Key, ActiveMode, OwningUserIndex](const UCommonUIActionRouterBase& ActionRouter, EInputEvent Event)
	{
		bool bHandled = ActionRouter.PersistentActions->ProcessNormalInput(ActiveMode, Key, Event, OwningUserIndex);

		if (!bHandled && ActionRouter.bIsActivatableTreeEnabled)
		{
			if (ActionRouter.ActiveRootNode)
			{
				bHandled = ActionRouter.ActiveRootNode->ProcessNormalInput(ActiveMode, Key, Event, OwningUserIndex);
			}

			if (!bHandled)
			{
				bHandled = ActionRouter.ProcessInputOnActionDomains(ActiveMode, Key, Event, OwningUserIndex);
			}
		}

		return bHandled;
	};

	const auto ProcessInputOnActionRouter = [&ProcessHoldInputFunc, &ProcessNormalInputFunc, &Key, InputEvent](const UCommonUIActionRouterBase& ActionRouter)
	{
		EProcessHoldActionResult ProcessHoldResult = ProcessHoldInputFunc(ActionRouter);
		if (ProcessHoldResult == EProcessHoldActionResult::Handled)
		{
			return true;
		}
		
		if (ProcessHoldResult == EProcessHoldActionResult::GeneratePress)
		{
			// A hold action was in progress but quickly aborted, so we want to generate a press action now for any normal bindings that are interested
			if (!ProcessNormalInputFunc(ActionRouter, IE_Pressed))
			{
				if (Key == EKeys::Virtual_Gamepad_Accept.GetVirtualKey() && ActionRouter.AnalogCursor.IsValid())
				{
					// If Virtual_Accept is bound to a Hold action, the Pressed event for it will not trigger a simulated click
					// but when a hold action is canceled fast enough, we want to trigger any "normal" actions bound to the same key
					// so we give the analog cursor a chance to trigger the simulated click here
					// This allows users to bind Hold actions to Virtual_Accept while still triggering focused button widgets with quick virtual_accept presses
					ActionRouter.AnalogCursor->OnVirtualAcceptHoldCanceled();
				}
			}
		}

		// Even if no widget cares about this input, we don't want to let anything through to the actual game while we're in menu mode
		return ProcessNormalInputFunc(ActionRouter, InputEvent);
	};

	bool bHandledInput = ProcessInputOnActionRouter(*this);
	if (bSupportMultiUserInput && !bHandledInput)
	{
		const ULocalPlayer* LocalPlayer = GetLocalPlayerChecked();
		if (const UGameInstance* GameInstance = LocalPlayer->GetGameInstance())
		{
			for (const ULocalPlayer* OtherPlayer : GameInstance->GetLocalPlayers())
			{
				if (OtherPlayer == LocalPlayer)
				{
					continue;
				}

				// If necessary, this could be sped up by caching something to indicate which action routers have bindings for which players
				if (const UCommonUIActionRouterBase* OtherActionRouter = ULocalPlayer::GetSubsystem<UCommonUIActionRouterBase>(OtherPlayer))
				{
					if (ProcessInputOnActionRouter(*OtherActionRouter))
					{
						bHandledInput = true;
						break;
					}
				}
			}
		}
	}

	if (bHandledInput)
	{
		return ERouteUIInputResult::Handled;
	}

	return CanProcessNormalGameInput() ? ERouteUIInputResult::Unhandled : ERouteUIInputResult::BlockGameInput;
}

UCommonInputSubsystem& UCommonUIActionRouterBase::GetInputSubsystem() const
{
	UCommonInputSubsystem* InputSubsytem = GetLocalPlayerChecked()->GetSubsystem<UCommonInputSubsystem>();
	check(InputSubsytem);
	return *InputSubsytem;
}

void UCommonUIActionRouterBase::FlushInput()
{
	const ECommonInputMode ActiveMode = GetActiveInputMode();
	const int32 OwningUserIndex = GetLocalPlayerIndex();
	const auto FlushInputOnActionRouter = [ActiveMode, OwningUserIndex](const UCommonUIActionRouterBase& ActionRouter, const FKey& HeldKey)
	{
		EProcessHoldActionResult ProcessHoldResult = ActionRouter.PersistentActions->ProcessHoldInput(ActiveMode, HeldKey, IE_Released, OwningUserIndex);
		if (ActionRouter.bIsActivatableTreeEnabled && ProcessHoldResult == EProcessHoldActionResult::Unhandled)
		{
			if (ActionRouter.ActiveRootNode)
			{
				ProcessHoldResult = ActionRouter.ActiveRootNode->ProcessHoldInput(ActiveMode, HeldKey, IE_Released, OwningUserIndex);
			}

			if (ProcessHoldResult == EProcessHoldActionResult::Unhandled)
			{
				ProcessHoldResult = ActionRouter.ProcessHoldInputOnActionDomains(ActiveMode, HeldKey, IE_Released, OwningUserIndex);
			}
		}

		return ProcessHoldResult;
	};

	for (const FKey& HeldKey : HeldKeys)
	{
		EProcessHoldActionResult ProcessHoldResult = FlushInputOnActionRouter(*this, HeldKey);
		
		if (bSupportMultiUserInput && ProcessHoldResult == EProcessHoldActionResult::Unhandled)
		{
			ULocalPlayer* LocalPlayer = GetLocalPlayerChecked();
			if (UGameInstance* GameInstance = LocalPlayer->GetGameInstance())
			{
				for (ULocalPlayer* OtherPlayer : GameInstance->GetLocalPlayers())
				{
					if (OtherPlayer == LocalPlayer)
					{
						continue;
					}

					// If necessary, this could be sped up by caching something to indicate which action routers have widgets with bindings for which players
					if (const UCommonUIActionRouterBase* OtherActionRouter = ULocalPlayer::GetSubsystem<UCommonUIActionRouterBase>(OtherPlayer))
					{
						ProcessHoldResult = FlushInputOnActionRouter(*OtherActionRouter, HeldKey);
						if (ProcessHoldResult != EProcessHoldActionResult::Unhandled)
						{
							break;
						}
					}
				}
			}
		}
	}

	HeldKeys.Empty();
}

bool UCommonUIActionRouterBase::IsWidgetInActiveRoot(const UCommonActivatableWidget* Widget) const
{
	FActivatableTreeRootPtr RootNode = ActiveRootNode ? ActiveRootNode : FindActiveActionDomainRootNode();
	if (Widget && RootNode)
	{
		TSharedPtr<SWidget> WidgetWalker = Widget->GetCachedWidget();
		while (WidgetWalker)
		{
			if (WidgetWalker->GetMetaData<FCommonActivatableSlateMetaData>().IsValid())
			{
				if (UCommonActivatableWidget* CandidateActivatable = Cast<UCommonActivatableWidget>(StaticCastSharedPtr<SObjectWidget>(WidgetWalker)->GetWidgetObject()))
				{
					if (CandidateActivatable == RootNode->GetWidget())
					{
						return true;
					}
				}
			}
			WidgetWalker = WidgetWalker->GetParentWidget();
		}
	}

	return false;
}

bool UCommonUIActionRouterBase::IsWidgetInLeafmostNodeHierarchy(const UWidget& Widget) const
{
	FActivatableTreeRootPtr RootNode = ActiveRootNode ? ActiveRootNode : FindActiveActionDomainRootNode();
	if (!RootNode)
	{
		return false;
	}

	return RootNode->IsWidgetInLeafmostNodeHierarchy(Widget);
}

UCommonActivatableWidget* UCommonUIActionRouterBase::GetLeafmostActivatableWidget() const
{
	FActivatableTreeRootPtr RootNode = ActiveRootNode ? ActiveRootNode : FindActiveActionDomainRootNode();
	if (!RootNode)
	{
		return nullptr;
	}

	return RootNode->GetLeafmostActivatableWidget();
}

void UCommonUIActionRouterBase::NotifyUserWidgetConstructed(const UCommonUserWidget& Widget)
{
	check(Widget.GetCachedWidget().IsValid());

	if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(Widget))
	{
		RegisterWidgetBindings(OwnerNode, Widget.GetActionBindings());
	}
	else if (Widget.GetActionBindings().Num() > 0)
	{
		GetOrCreatePendingRegistration(Widget).ActionBindings.Append(Widget.GetActionBindings());
	}
}

void UCommonUIActionRouterBase::NotifyUserWidgetDestructed(const UCommonUserWidget& Widget)
{
	const int32 PendingRegistrationIdx = PendingWidgetRegistrations.IndexOfByKey(Widget);
	if (PendingRegistrationIdx == INDEX_NONE)
	{
		// The widget wasn't pending registration, so the bindings need to be removed
		// Not worth splitting out which bindings are persistent vs. normal, just have both collections try to remove all the bindings on the widget
		PersistentActions->RemoveBindings(Widget.GetActionBindings());
		if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(Widget))
		{
			OwnerNode->RemoveBindings(Widget.GetActionBindings());
		}
	}
	else
	{
		PendingWidgetRegistrations.RemoveAt(PendingRegistrationIdx);
	}
}

void UCommonUIActionRouterBase::AddBinding(FUIActionBindingHandle Handle)
{
	if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(Handle))
	{
		if (const UWidget* BoundWidget = Binding->BoundWidget.Get())
		{
			if (FActivatableTreeNodePtr OwnerNode = FindOwningNode(*BoundWidget))
			{
				if (Binding->bIsPersistent)
				{
					PersistentActions->AddBinding(*Binding);
				}
				else
				{
					OwnerNode->AddBinding(*Binding);
				}
			}
			else if (BoundWidget->GetCachedWidget())
			{
				GetOrCreatePendingRegistration(*BoundWidget).ActionBindings.AddUnique(Handle);
			}
		}
	}
}

void UCommonUIActionRouterBase::RemoveBinding(FUIActionBindingHandle Handle)
{
	if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(Handle))
	{
		if (Binding->OwningCollection.IsValid())
		{
			Binding->OwningCollection.Pin()->RemoveBinding(Handle);
		}
		else if (FPendingWidgetRegistration* PendingRegistration = PendingWidgetRegistrations.FindByKey(Binding->BoundWidget.Get()))
		{
			PendingRegistration->ActionBindings.Remove(Handle);
		}
	}
}

int32 UCommonUIActionRouterBase::GetLocalPlayerIndex() const
{
	ULocalPlayer* LocalPlayer = GetLocalPlayerChecked();
	if (UGameInstance* GameInstance = LocalPlayer->GetGameInstance())
	{
		return GameInstance->GetLocalPlayers().Find(LocalPlayer);
	}
	return INDEX_NONE;
}

bool UCommonUIActionRouterBase::ShouldAlwaysShowCursor() const
{
	bool bUsingMouseForTouch = FSlateApplication::Get().IsFakingTouchEvents();
	ULocalPlayer& LocalPlayer = *GetLocalPlayerChecked();
	if (UGameViewportClient* GameViewportClient = LocalPlayer.ViewportClient)
	{
		bUsingMouseForTouch |= GameViewportClient->GetUseMouseForTouch();
	}
	bool bVirtualPointerEnabled = AnalogCursor.IsValid() ? AnalogCursor->IsVirtualPointerEnabled() : false;

	return bAlwaysShowCursor || bUsingMouseForTouch || bVirtualPointerEnabled;
}

ECommonInputMode UCommonUIActionRouterBase::GetActiveInputMode(ECommonInputMode DefaultInputMode) const
{
	return ActiveInputConfig.IsSet() ? ActiveInputConfig.GetValue().GetInputMode() : DefaultInputMode;
}

EMouseCaptureMode UCommonUIActionRouterBase::GetActiveMouseCaptureMode(EMouseCaptureMode DefaultMouseCapture) const
{
	EMouseCaptureMode Mode = ActiveInputConfig.IsSet() ? ActiveInputConfig.GetValue().GetMouseCaptureMode() : DefaultMouseCapture;
	
	const bool bVirtualPointerEnabled = AnalogCursor.IsValid() ? AnalogCursor->IsVirtualPointerEnabled() : false;
	if (bVirtualPointerEnabled && Mode == EMouseCaptureMode::CapturePermanently)
	{
		// While VP is enabled, mouse-up releases capture each click, so plain CapturePermanently drops
		// every subsequent mouse-down in FSceneViewport (HasMouseCapture()=false on entry).
		Mode = EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown;
	}

	return Mode;
}

void UCommonUIActionRouterBase::HandleRootWidgetSlateReleased(TWeakPtr<FActivatableTreeRoot> WeakRoot)
{
	if (!WeakRoot.IsValid())
	{
		return;
	}

	FActivatableTreeRootRef RootNode = WeakRoot.Pin().ToSharedRef();
	if (UCommonActivatableWidget* ActivatableWidget = RootNode->GetWidget())
	{
		ActivatableWidget->OnSlateReleased().RemoveAll(this);
	}

	if (RootNodes.Contains(RootNode))
	{
		// It's possible that the widget is destructed as a result of some other deactivation handler, causing us to get here before hearing about
		// the deactivation. Not a big deal, just need to process the deactivation right here if the node in question is the active root.
		if (ActiveRootNode == RootNode)
		{
			if (ActiveRootNode->IsWidgetActivated() && ActiveRootNode->GetWidget())
			{
				ActiveRootNode->GetWidget()->DeactivateWidget();
			}
			HandleRootNodeDeactivated(ActiveRootNode);
		}

		RootNodes.RemoveSwap(RootNode);
		// Cannot actually have this ensure here, because we may be in a function called on the ToBeRemoved node itself, keeping one remaining strong reference
		// ensureAlways(!ToBeRemoved.IsValid());

		if (RootNodes.Num() == 0)
		{
			// Clear any input locks before clearing our active config so they don't leak
			if (ActiveInputConfig.IsSet())
			{
				if (ULocalPlayer* LocalPlayer = GetLocalPlayer())
				{
					if (APlayerController* PC = LocalPlayer->GetPlayerController(GetWorld()))
					{
						if (ActiveInputConfig->bIgnoreMoveInput)
						{
							PC->SetIgnoreMoveInput(false);
						}

						if (ActiveInputConfig->bIgnoreLookInput)
						{
							PC->SetIgnoreLookInput(false);
						}
					}
				}
				ActiveInputConfig.Reset();
			}

			RefreshActionDomainLeafNodeConfig();
		}
	}
	else
	{
		int32 NumRemoved = 0;
		for (TPair<TObjectPtr<UCommonInputActionDomain>, FActionDomainSortedRootList>& Pair : ActionDomainRootNodes)
		{
			FActionDomainSortedRootList& ActionDomainRootList = Pair.Value;
			NumRemoved += ActionDomainRootList.Remove(RootNode);
			RootNode->OnLeafmostActiveNodeChanged.Unbind();
			RootNode->SetCanReceiveInput(false);
			RootNode->UpdateLeafNode();
		}

		if (NumRemoved <= 0)
		{
			UCommonActivatableWidget* ActivatableWidget = RootNode->GetWidget();
			ensureAlwaysMsgf(false, TEXT("Root node could not be found during deactivation [%s]"), ActivatableWidget ? *ActivatableWidget->GetName() : TEXT("Unknown"));
		}
	}
}

void UCommonUIActionRouterBase::HandleRootNodeActivated(TWeakPtr<FActivatableTreeRoot> WeakActivatedRoot)
{
	FActivatableTreeRootRef ActivatedRoot = WeakActivatedRoot.Pin().ToSharedRef();
	UCommonActivatableWidget* NodeWidget = ActivatedRoot->GetWidget();

	if (RootNodes.Contains(ActivatedRoot))
	{
		if (ActivatedRoot->GetLastPaintLayer() > 0)
		{
			const int32 CurrentRootLayer = ActiveRootNode ? ActiveRootNode->GetLastPaintLayer() : INDEX_NONE;
			if (ActivatedRoot->GetLastPaintLayer() > CurrentRootLayer)
			{
				// Ensure we have a local player so the action router local player subsystem will handle root change
				const ULocalPlayer* LocalPlayer = GetLocalPlayer();
				if (LocalPlayer && LocalPlayer->ViewportClient)
				{
					SetActiveRoot(ActivatedRoot);
				}
			}
		}
	}
	else if (UCommonInputActionDomain* WidgetActionDomain = NodeWidget ? NodeWidget->GetCalculatedActionDomain().Get() : nullptr)
	{
		const FActionDomainSortedRootList* ActionDomainRootList = ActionDomainRootNodes.Find(WidgetActionDomain);
		if (ActionDomainRootList && ensure(ActionDomainRootList->Contains(ActivatedRoot)))
		{
			ActivatedRoot->SetCanReceiveInput(true);
			ActivatedRoot->OnLeafmostActiveNodeChanged.BindUObject(this, &UCommonUIActionRouterBase::HandleLeafmostActiveNodeChanged);

			if (!ActiveRootNode.IsValid())
			{
				if (ActivatedRoot->GetLastPaintLayer() > 0)
				{
					if (ActivatedRoot == FindActiveActionDomainRootNode())
					{
						// This will allow us to update the leaf node config on the activated root.
						ActivatedRoot->UpdateLeafNode();
						OnBoundActionsUpdated().Broadcast();
					}
				}
				else
				{
					ActiveActionDomainRootsPendingPaint.Add(ActivatedRoot);
				}
			}
		}
	}
}

void UCommonUIActionRouterBase::HandleRootNodeDeactivated(TWeakPtr<FActivatableTreeRoot> WeakDeactivatedRoot)
{
	FActivatableTreeRootPtr DeactivatedRoot = WeakDeactivatedRoot.Pin();
	if (ActiveRootNode.IsValid() && ActiveRootNode == DeactivatedRoot)
	{
		// Reset the active root widget - we'll re-establish it on the next tick
		SetActiveRoot(nullptr);
	}

	// In the case that the activatable tree was enabled we need to re-establish input for the highest paint layer node in action domain nodes.
	if (bIsActivatableTreeEnabled)
	{
		if (bWarnAllWidgetsDeactivated)
		{
			UE_LOGF(LogUIActionRouter, Warning, "All widgets deactivated. Existing input config set: %ls",
				ActiveInputConfig.IsSet() ? TEXT("Yes - the current input config is lingering from a deactivated widget.") : TEXT("No."));
		}

		RefreshActionDomainLeafNodeConfig();
		OnBoundActionsUpdated().Broadcast();
	}
}

void UCommonUIActionRouterBase::HandleLeafmostActiveNodeChanged()
{
	OnBoundActionsUpdated().Broadcast();
}

void UCommonUIActionRouterBase::HandleSlateFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget)
{
	if (FocusEvent.GetUser() == GetLocalPlayerIndex())
	{
		FActivatableTreeRootPtr RootNode = ActiveRootNode ? ActiveRootNode : FindActiveActionDomainRootNode();
		if (RootNode && RootNode->IsParentOfWidget(OldFocusedWidget, FActivatableTreeNode::IncludeSelf))
		{
			RootNode->RefreshCachedRestorationTarget();
		}
	}
}

void UCommonUIActionRouterBase::HandlePostGarbageCollect()
{
	FUIActionBinding::CleanRegistrations();

	// GC may result in root widget being purged while conditional slate resource release skips HandleRootWidgetSlateReleased, handle this scenario
	for (TArray<FActivatableTreeRootRef>::TIterator Iter = RootNodes.CreateIterator(); Iter; ++Iter)
	{
		if (!Iter->Get().IsWidgetValid())
		{
			Iter.RemoveCurrent();
		}
	}

	for (TPair<TObjectPtr<UCommonInputActionDomain>, FActionDomainSortedRootList>& Pair : ActionDomainRootNodes)
	{
		for (TArray<FActivatableTreeRootRef>::TIterator Iter = Pair.Value.GetRootList().CreateIterator(); Iter; ++Iter)
		{
			if (!Iter->Get().IsWidgetValid())
			{
				Iter.RemoveCurrent();
			}
		}
	}
}

void UCommonUIActionRouterBase::HandleVirtualPointerEnabled(bool bEnabled)
{
	RefreshUIInputConfig();
}

const UCommonInputActionDomainTable* UCommonUIActionRouterBase::GetActionDomainTable() const
{
	UCommonInputSubsystem* InputSubsytem = GetLocalPlayerChecked()->GetSubsystem<UCommonInputSubsystem>();
	return InputSubsytem ? InputSubsytem->GetActionDomainTable() : nullptr;
}

bool UCommonUIActionRouterBase::ProcessInputOnActionDomains(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const
{
	const UCommonInputActionDomainTable* ActionDomainTable = GetActionDomainTable();
	if (!ActionDomainTable)
	{
		return false;
	}

	bool bInputEventHandledAtLeastOnce = false;

	for (const UCommonInputActionDomain* ActionDomain : ActionDomainTable->ActionDomains)
	{
		const FActionDomainSortedRootList* SortedRootList = ActionDomainRootNodes.Find(ActionDomain);
		if (!SortedRootList)
		{
			// No widget with this Domain was added
			continue;
		}

		bool bInputEventHandledInDomain = false;
		bool bDomainHadActiveRoots = false;

		for (const FActivatableTreeRootRef& RootNode : SortedRootList->GetRootList())
		{
			if (RootNode->IsWidgetActivated())
			{
				bool bInputEventHandled = RootNode->ProcessNormalInput(ActiveInputMode, Key, InputEvent, UserIndex);
				bInputEventHandledInDomain |= bInputEventHandled;
				bDomainHadActiveRoots = true;

				if (ActionDomain->ShouldBreakInnerEventFlow(bInputEventHandled))
				{
					break;
				}
			}
		}

		bInputEventHandledAtLeastOnce |= bInputEventHandledInDomain;

		if (ActionDomain->ShouldBreakEventFlow(bDomainHadActiveRoots, bInputEventHandledInDomain))
		{
			break;
		}
	}

	return bInputEventHandledAtLeastOnce;
}

EProcessHoldActionResult UCommonUIActionRouterBase::ProcessHoldInputOnActionDomains(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const
{
	EProcessHoldActionResult HoldActionResult = EProcessHoldActionResult::Unhandled;
	
	const UCommonInputActionDomainTable* ActionDomainTable = GetActionDomainTable();
	if (!ActionDomainTable)
	{
		return HoldActionResult;
	}

	for (const UCommonInputActionDomain* ActionDomain : ActionDomainTable->ActionDomains)
	{
		const FActionDomainSortedRootList* SortedRootList = ActionDomainRootNodes.Find(ActionDomain);
		if (!SortedRootList)
		{
			// No widget with this Domain was added
			continue;
		}

		bool bInputEventHandledInDomain = false;
		bool bDomainHadActiveRoots = false;

		for (const FActivatableTreeRootRef& RootNode : SortedRootList->GetRootList())
		{
			if (RootNode->IsReceivingInput() && HoldActionResult == EProcessHoldActionResult::Unhandled)
			{
				HoldActionResult = RootNode->ProcessHoldInput(ActiveInputMode, Key, InputEvent, UserIndex);
				bInputEventHandledInDomain |= HoldActionResult == EProcessHoldActionResult::Handled;
				bDomainHadActiveRoots = true;

				if (ActionDomain->ShouldBreakInnerEventFlow(HoldActionResult == EProcessHoldActionResult::Handled))
				{
					break;
				}
			}

		}

		if (ActionDomain->ShouldBreakEventFlow(bDomainHadActiveRoots, bInputEventHandledInDomain))
		{
			break;
		}
	}

	return HoldActionResult;
}

FGameplayTagContainer UCommonUIActionRouterBase::GetGameplayTagsForInputMode(const ECommonInputMode Mode) const
{
	FGameplayTagContainer Tags;
	
	switch (Mode)
	{
	case ECommonInputMode::Game:
		Tags.AddTag(TAG_InputModeGame);
		break;

	case ECommonInputMode::Menu:
		Tags.AddTag(TAG_InputModeMenu);
		break;

	case ECommonInputMode::All:
		Tags.AddTag(TAG_InputModeGame);
		Tags.AddTag(TAG_InputModeMenu);
		break;
	}

	return Tags;
}

void UCommonUIActionRouterBase::DebugDumpActionDomainRootNodes(int32 UserIndex, int32 ControllerId, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const
{
	FString ActionDomainsOutputStr;
	ActionDomainsOutputStr.Append(TEXT("******** Start Debugging ActionDomainRootNodes ********\n"));
	for (const TPair<TObjectPtr<UCommonInputActionDomain>, FActionDomainSortedRootList>& Pair : ActionDomainRootNodes)
	{
		ActionDomainsOutputStr.Append(TEXT("\n****** Dumping ActionDomainRootNodes for ActionDomain: "));
		ActionDomainsOutputStr.Append(GetNameSafe(Pair.Key) + TEXT(" ******\n"));
		if (Pair.Value.GetRootList().Num() > 0)
		{
			Pair.Value.DebugDumpRootList(ActionDomainsOutputStr, bIncludeActions, bIncludeChildren, bIncludeInactive);
			ActionDomainsOutputStr.Append(TEXT("\n\n"));
		}
		else
		{
			ActionDomainsOutputStr.Append(TEXT("-No root nodes found\n"));
		}
	}
	ActionDomainsOutputStr.Append(TEXT("\n******** End Debugging ActionDomainRootNodes ********\n"));
	UE_LOGF(LogUIActionRouter, Display, "Dumping ActionDomainRootNodes for LocalPlayer [User %d, ControllerId %d]:\n%ls\n", UserIndex, ControllerId, *ActionDomainsOutputStr);
}

void UCommonUIActionRouterBase::ProcessRebuiltWidgets()
{
	RefreshRootNodes();
	RefreshBoundActions();
}

void UCommonUIActionRouterBase::RefreshRootNodes()
{
	// Begin by organizing all of the widgets that need nodes according to their direct parent
	TArray<UCommonActivatableWidget*> RootCandidates;
	TMap<UCommonActivatableWidget*, TArray<UCommonActivatableWidget*>> WidgetsByDirectParent;
	for (TWeakObjectPtr<UCommonActivatableWidget> RebuiltWidget : RebuiltWidgetsPendingNodeAssignment)
	{
		if (RebuiltWidget.IsValid() && RebuiltWidget->GetCachedWidget())
		{
			if (UCommonActivatableWidget* ActivatableParent = !RebuiltWidget->IsModal() ? ::FindOwningActivatable(*RebuiltWidget) : nullptr)
			{
				WidgetsByDirectParent.FindOrAdd(ActivatableParent).Add(RebuiltWidget.Get());
			}
			else
			{
				// Parent-less (or modal), so add an entry for it as a root candidate
				RootCandidates.Add(RebuiltWidget.Get());
			}
		}
	}
	RebuiltWidgetsPendingNodeAssignment.Reset();

	// Build a new tree for any new roots
	for (UCommonActivatableWidget* RootWidget : RootCandidates)
	{
		FActivatableTreeRootRef RootNode = FActivatableTreeRoot::Create(*this, *RootWidget);

		TWeakPtr<FActivatableTreeRoot> WeakRoot(RootNode);
		RootNode->OnActivated.BindUObject(this, &UCommonUIActionRouterBase::HandleRootNodeActivated, WeakRoot);
		RootNode->OnDeactivated.BindUObject(this, &UCommonUIActionRouterBase::HandleRootNodeDeactivated, WeakRoot);
		RootWidget->OnSlateReleased().AddUObject(this, &UCommonUIActionRouterBase::HandleRootWidgetSlateReleased, WeakRoot);

		UCommonInputActionDomain* ActionDomain = RootWidget->GetCalculatedActionDomain();
		if (ActionDomain)
		{
			ActionDomainRootNodes.FindOrAdd(ActionDomain).Add(RootNode);
		}
		else
		{
			RootNodes.Add(RootNode);
		}

		AssembleTreeRecursive(RootNode, WidgetsByDirectParent);

		if (RootWidget->IsActivated())
		{
			// If we've created a root for a widget that's already active, process that activation now (ensures we have an appropriate active root)
			HandleRootNodeActivated(WeakRoot);
		}
	}

	// Now process any remaining entries - these are widgets that were rebuilt but should be appended to an existing node
	int32 NumWidgetsLeft = INDEX_NONE;
	while (WidgetsByDirectParent.Num() > 0 && NumWidgetsLeft != WidgetsByDirectParent.Num())
	{
		// If we run this loop twice without removing any entries from the map, we're in trouble
		NumWidgetsLeft = WidgetsByDirectParent.Num();

		// The keys in here fall into one of two categories - either they should be appended directly to an existing node, or they are a child of another key here
		// So, we can just go through looking for keys with an owner that already has a node. Then we can BuildTreeFunc from there with ease.
		for (const TPair<UCommonActivatableWidget*, TArray<UCommonActivatableWidget*>>& ParentChildrenPair : WidgetsByDirectParent)
		{
			if (FActivatableTreeNodePtr ExistingNode = FindNode(ParentChildrenPair.Key))
			{
				AssembleTreeRecursive(ExistingNode.ToSharedRef(), WidgetsByDirectParent);
				break;
			}
		}
	}

	if (WidgetsByDirectParent.Num() > 0)
	{
		ensureAlwaysMsgf(false, TEXT("Somehow we rebuilt a widget that is owned by an activatable, but no node exists for that activatable. This *should* be completely impossible."));
	}
}
	
void UCommonUIActionRouterBase::RefreshBoundActions()
{
	// Now, we account for all the widgets that would like their actions bound
	for (const FPendingWidgetRegistration& PendingRegistration : PendingWidgetRegistrations)
	{
		const UWidget* Widget = PendingRegistration.Widget.Get();
		if (Widget && Widget->GetCachedWidget().IsValid())
		{
			FActivatableTreeNodePtr OwnerNode = FindOwningNode(*Widget);
			RegisterWidgetBindings(OwnerNode, PendingRegistration.ActionBindings);

			if (UCommonActivatableWidget* OwnerWidget = OwnerNode ? OwnerNode->GetWidget() : nullptr)
			{
				OwnerWidget->RegisterInputTreeNode(OwnerNode);
			}

			if ((PendingRegistration.bIsScrollRecipient || PendingRegistration.InputPreProcessors.Num() > 0) && ensureMsgf(OwnerNode, TEXT("Widget [%s] does not have a parent activatable widget at any level - cannot register preprocessors or as a scroll recipient"), *Widget->GetName()))
			{
				if (PendingRegistration.bIsScrollRecipient)
				{
					OwnerNode->AddScrollRecipient(*Widget);
				}

				for (const auto& PreprocessorInfo : PendingRegistration.InputPreProcessors)
				{
					OwnerNode->AddInputPreprocessor(PreprocessorInfo.InputProcessor.ToSharedRef(), PreprocessorInfo.Info);
				}
			}
		}
	}

	PendingWidgetRegistrations.Reset();
}

void UCommonUIActionRouterBase::AssembleTreeRecursive(const FActivatableTreeNodeRef& CurNode, TMap<UCommonActivatableWidget*, TArray<UCommonActivatableWidget*>>& WidgetsByDirectParent)
{
	TArray<UCommonActivatableWidget*> Children;
	if (WidgetsByDirectParent.RemoveAndCopyValue(CurNode->GetWidget(), Children))
	{
		for (UCommonActivatableWidget* ActivatableWidget : Children)
		{
			FActivatableTreeNodeRef NewNode = CurNode->AddChildNode(*ActivatableWidget);
			AssembleTreeRecursive(NewNode, WidgetsByDirectParent);
		}
	}
}

bool UCommonUIActionRouterBase::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UCommonUIActionRouter_Tick);

	// Sort our action domain roots to match the most recent paint layers
	for (TPair<TObjectPtr<UCommonInputActionDomain>, FActionDomainSortedRootList>& Pair : ActionDomainRootNodes)
	{
		Pair.Value.Sort();
	}
	
	if (PendingWidgetRegistrations.Num() > 0 || RebuiltWidgetsPendingNodeAssignment.Num() > 0)
	{
		ProcessRebuiltWidgets();
	}

	if (bIsActivatableTreeEnabled)
	{
		int32 HighestPaintLayer = ActiveRootNode ? ActiveRootNode->GetLastPaintLayer() : INDEX_NONE;
		FActivatableTreeRootPtr NewActiveRoot = ActiveRootNode;
		for (const FActivatableTreeRootRef& Root : RootNodes)
		{
			if (Root->IsWidgetActivated())
			{
				int32 CurrentRootLayer = Root->GetLastPaintLayer();
				if (CurrentRootLayer > HighestPaintLayer)
				{
					HighestPaintLayer = CurrentRootLayer;
					NewActiveRoot = Root;
				}
			}
		}

		if (NewActiveRoot != ActiveRootNode)
		{
			SetActiveRoot(NewActiveRoot);
		}

		// Check for any newly painted active action domain nodes that could become the active node
		// Work on a copy in-case ActiveActionDomainRootsPendingPaint changes during iteration
		// We iterate even if ActiveRootNode.IsValid() to maintain the list of unpainted widgets that could become the active node if ActiveRootNode deactivates before then
		TSet<TWeakPtr<FActivatableTreeRoot>> ActiveActionDomainRootsPendingPaintCopy = MoveTemp(ActiveActionDomainRootsPendingPaint);
		for (auto It = ActiveActionDomainRootsPendingPaintCopy.CreateIterator(); It; ++It)
		{
			if (FActivatableTreeRootPtr ActiveRoot = It->Pin())
			{
				if (ActiveRoot->GetLastPaintLayer() > 0)
				{
					It.RemoveCurrent();
					
					if (!ActiveRootNode.IsValid() && ActiveRoot == FindActiveActionDomainRootNode())
					{
						ActiveRoot->UpdateLeafNode();
						OnBoundActionsUpdated().Broadcast();
					}
				}
			}
			else
			{
				It.RemoveCurrent();
			}
		}

		ActiveActionDomainRootsPendingPaint.Append(ActiveActionDomainRootsPendingPaintCopy);
	}
	
	const ECommonInputMode ActiveMode = GetActiveInputMode();
	const int32 OwningUserIndex = GetLocalPlayerIndex();
	const auto TickInputOnActionRouter = [ActiveMode, OwningUserIndex](const UCommonUIActionRouterBase& ActionRouter, const FKey& HeldKey)
	{
		EProcessHoldActionResult ProcessHoldResult = ActionRouter.PersistentActions->ProcessHoldInput(ActiveMode, HeldKey, IE_Repeat, OwningUserIndex);
		if (ActionRouter.bIsActivatableTreeEnabled && ProcessHoldResult == EProcessHoldActionResult::Unhandled)
		{
			if (ActionRouter.ActiveRootNode)
			{
				ProcessHoldResult = ActionRouter.ActiveRootNode->ProcessHoldInput(ActiveMode, HeldKey, IE_Repeat, OwningUserIndex);
			}

			if (ProcessHoldResult == EProcessHoldActionResult::Unhandled)
			{
				ProcessHoldResult = ActionRouter.ProcessHoldInputOnActionDomains(ActiveMode, HeldKey, IE_Repeat, OwningUserIndex);
			}
		}
		return ProcessHoldResult;
	};
	
	for (const FKey& HeldKey : HeldKeys)
	{
		EProcessHoldActionResult ProcessHoldResult = TickInputOnActionRouter(*this, HeldKey);
		if (bSupportMultiUserInput && ProcessHoldResult == EProcessHoldActionResult::Unhandled)
		{
			ULocalPlayer* LocalPlayer = GetLocalPlayerChecked();
			if (UGameInstance* GameInstance = LocalPlayer->GetGameInstance())
			{
				for (ULocalPlayer* OtherPlayer : GameInstance->GetLocalPlayers())
				{
					if (OtherPlayer == LocalPlayer)
					{
						continue;
					}

					// If necessary, this could be sped up by caching something to indicate which action routers have widgets with bindings for which players
					if (const UCommonUIActionRouterBase* OtherActionRouter = ULocalPlayer::GetSubsystem<UCommonUIActionRouterBase>(OtherPlayer))
					{
						ProcessHoldResult = TickInputOnActionRouter(*OtherActionRouter, HeldKey);
						if (ProcessHoldResult != EProcessHoldActionResult::Unhandled)
						{
							break;
						}
					}
				}
			}
		}
	}

	return true; //continue ticking
}

void UCommonUIActionRouterBase::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static const FName NAME_ActionRouter("ActionRouter");
	if (!Canvas || !HUD->ShouldDisplayDebug(NAME_ActionRouter))
	{
		return;
	}
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	DisplayDebugManager.SetFont(GEngine->GetSmallFont());


	static UEnum* InputModeEnum = StaticEnum<ECommonInputMode>();
	static UEnum* MouseCaptureModeEnum = StaticEnum<EMouseCaptureMode>();
	static UEnum* InputTypeEnum = StaticEnum<ECommonInputType>();

	check(InputModeEnum);
	check(MouseCaptureModeEnum);
	check(InputTypeEnum);

	const UCommonInputSubsystem& InputSystem = GetInputSubsystem();
	ECommonInputType CurrentInputType = InputSystem.GetCurrentInputType();

	ULocalPlayer* LocalPlayer = GetLocalPlayerChecked();
	int32 ControllerId = LocalPlayer->GetControllerId();

	DisplayDebugManager.SetDrawColor(FColor::White);
	DisplayDebugManager.DrawString(FString::Printf(TEXT("Action Router - Player [%d]: Input Type[%s]"), ControllerId, *InputTypeEnum->GetNameStringByValue((int64)CurrentInputType)));
	if (ActiveInputConfig.IsSet())
	{
		FString InputModeStr = InputModeEnum->GetNameStringByValue((int64)ActiveInputConfig->GetInputMode());
		FString MouseCaptureStr = MouseCaptureModeEnum->GetNameStringByValue((int64)ActiveInputConfig->GetMouseCaptureMode());

		DisplayDebugManager.DrawString(FString::Printf(TEXT("    Input Mode [%s] Mouse Capture [%s]"), *InputModeStr, *MouseCaptureStr));
	}
	else
	{
		DisplayDebugManager.SetDrawColor(FColor::Red);
		DisplayDebugManager.DrawString(FString(TEXT("    No Input Config")));
	}

	DisplayDebugManager.SetDrawColor(FColor::White);
	DisplayDebugManager.DrawString(PersistentActions->DumpActionBindings());
}

void UCommonUIActionRouterBase::PopulateAutoCompleteEntries(TArray<FAutoCompleteCommand>& AutoCompleteList)
{
	const UConsoleSettings* ConsoleSettings = GetDefault<UConsoleSettings>();

	AutoCompleteList.AddDefaulted();

	FAutoCompleteCommand& AutoCompleteCommand = AutoCompleteList.Last();

	AutoCompleteCommand.Command = TEXT("showdebug ActionRouter");
	AutoCompleteCommand.Desc = TEXT("Toggles display of Action Router");
	AutoCompleteCommand.Color = ConsoleSettings->AutoCompleteCommandColor;
}

bool UCommonUIActionRouterBase::CanProcessNormalGameInput() const
{
	if (GetActiveInputMode() == ECommonInputMode::Menu)
	{
		// We still process normal game input in menu mode if the game viewport has mouse capture
		// This allows manipulation of preview items and characters in the world while in menus. 
		// If this is not desired, disable viewport mouse capture in your desired input config.
		const ULocalPlayer& LocalPlayer = *GetLocalPlayerChecked();
		if (TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(GetLocalPlayerIndex()))
		{
			return LocalPlayer.ViewportClient && SlateUser->DoesWidgetHaveCursorCapture(LocalPlayer.ViewportClient->GetGameViewportWidget());
		}
	}
	return true;
}

bool UCommonUIActionRouterBase::IsPendingTreeChange() const
{
	return RebuiltWidgetsPendingNodeAssignment.Num() > 0;
}

void UCommonUIActionRouterBase::RegisterWidgetBindings(const FActivatableTreeNodePtr& TreeNode, const TArray<FUIActionBindingHandle>& BindingHandles)
{
	for (FUIActionBindingHandle Handle : BindingHandles)
	{
		if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(Handle))
		{
			if (Binding->bIsPersistent)
			{
				PersistentActions->AddBinding(*Binding);
			}
			else if (ensureMsgf(TreeNode, TEXT("Widget [%s] does not have a parent activatable widget at any level - cannot register standard binding to action [%s]. UserWidget parent(s): %s"),
				*Binding->BoundWidget->GetName(),
				*Binding->ActionName.ToString(),
				*CommonUIUtils::PrintAllOwningUserWidgets(Binding->BoundWidget.Get())))
			{
				TreeNode->AddBinding(*Binding);
			}
		}
	}
}

void UCommonUIActionRouterBase::RefreshActiveRootFocusRestorationTarget() const
{
	FActivatableTreeRootPtr RootNode = ActiveRootNode ? ActiveRootNode : FindActiveActionDomainRootNode();
	if (RootNode)
	{
		RootNode->RefreshCachedRestorationTarget();
	}
}

void UCommonUIActionRouterBase::RefreshActiveRootFocus()
{
	FActivatableTreeRootPtr RootNode = ActiveRootNode ? ActiveRootNode : FindActiveActionDomainRootNode();
	if (RootNode)
	{
		RootNode->FocusLeafmostNode();
	}
}

void UCommonUIActionRouterBase::RefreshUIInputConfig()
{
	if (ActiveInputConfig.IsSet())
	{
		ApplyUIInputConfig(ActiveInputConfig.GetValue(), /*bForceRefresh*/ true);
	}
}

TWeakPtr<FActivatableTreeRoot> UCommonUIActionRouterBase::GetActiveRoot() const
{
	return ActiveRootNode;
}

void UCommonUIActionRouterBase::SetActiveRoot(FActivatableTreeRootPtr NewActiveRoot)
{
	if (ActiveRootNode)
	{
		ActiveRootNode->OnLeafmostActiveNodeChanged.Unbind();
		ActiveRootNode->SetCanReceiveInput(false);
		ActiveRootNode->UpdateLeafNode();
	}

	if (bForceResetActiveRoot || !bIsActivatableTreeEnabled)
	{
		// Never activate a root while dormant or the tree is disabled
		bForceResetActiveRoot = false;
		ActiveRootNode.Reset();

		if (bForceResetActiveRoot || bResetUIInputConfigOnActivatableTreeDeactivation)
		{
			// Reset the input config when dormant so we don't get stuck in a non-default input mode when layout is dormant
			SetActiveUIInputConfig(FUIInputConfig(ECommonInputMode::All, EMouseCaptureMode::NoCapture));
		}
	}
	else
	{
		ActiveRootNode = NewActiveRoot;
		if (NewActiveRoot)
		{
			NewActiveRoot->SetCanReceiveInput(true);
			NewActiveRoot->UpdateLeafNode();
			NewActiveRoot->OnLeafmostActiveNodeChanged.BindUObject(this, &UCommonUIActionRouterBase::HandleLeafmostActiveNodeChanged);
		}
	}

	OnBoundActionsUpdated().Broadcast();
}

void UCommonUIActionRouterBase::SetForceResetActiveRoot(bool bInForceResetActiveRoot)
{
	bForceResetActiveRoot = bInForceResetActiveRoot;
}

void UCommonUIActionRouterBase::UpdateLeafNodeAndConfig(FActivatableTreeRootPtr DesiredRoot, FActivatableTreeNodePtr DesiredLeafNode)
{
	if (DesiredRoot.IsValid())
	{
		if (DesiredRoot == ActiveRootNode)
		{
			// We're updating both the leaf node and it's config if we're the active root.
			if (!DesiredRoot->UpdateLeafmostActiveNode(DesiredLeafNode))
			{
				UE_LOGF(LogUIActionRouter, Warning, "LeafmostActiveNode not updated.");
			}
		}
		else
		{
			RefreshActionDomainLeafNodeConfig();
		}
	}
}

void UCommonUIActionRouterBase::FlushPressedKeys() const
{
	ULocalPlayer& LocalPlayer = *GetLocalPlayerChecked();
	if (APlayerController* PC = LocalPlayer.GetPlayerController(GetWorld()))
	{
		PC->FlushPressedKeys();
	}
}

UCommonUIActionRouterBase::FPendingWidgetRegistration& UCommonUIActionRouterBase::GetOrCreatePendingRegistration(const UWidget& Widget)
{
	if (FPendingWidgetRegistration* ExistingEntry = PendingWidgetRegistrations.FindByKey(Widget))
	{
		return *ExistingEntry;
	}

	FPendingWidgetRegistration NewEntry;
	NewEntry.Widget = &Widget;
	return PendingWidgetRegistrations[PendingWidgetRegistrations.Add(NewEntry)];;
}

FActivatableTreeNodePtr UCommonUIActionRouterBase::FindNode(const UCommonActivatableWidget* Widget) const
{
	if (Widget)
	{
		const bool bIsModal = Widget->IsModal();
		for (const FActivatableTreeRootPtr RootNode : RootNodes)
		{
			FActivatableTreeNodePtr FoundNode;
			if (!bIsModal)
			{
				FoundNode = FindNodeRecursive(RootNode, *Widget);
			}
			else if (Widget == RootNode->GetWidget())
			{
				// If we're looking for a modal's node, we only need to check the roots
				FoundNode = RootNode;
			}

			if (FoundNode.IsValid())
			{
				return FoundNode;
			}
		}
				
		for (const TPair<TObjectPtr<UCommonInputActionDomain>, FActionDomainSortedRootList>& Pair : ActionDomainRootNodes)
		{
			for (const FActivatableTreeRootRef& RootNode : Pair.Value.GetRootList())
			{
				FActivatableTreeNodePtr FoundNode = FindNodeRecursive(RootNode, *Widget);
				if (FoundNode.IsValid())
				{
					return FoundNode;
				}
			}
		}
	}

	return nullptr;
}

FActivatableTreeNodePtr UCommonUIActionRouterBase::FindOwningNode(const UWidget& Widget) const
{
	const UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(&Widget);
	FActivatableTreeNodePtr FoundNode = FindNode(ActivatableWidget);

	// Don't search beyond the roots if we're looking for a modal activatable
	if (!FoundNode && (!ActivatableWidget || !ActivatableWidget->IsModal()))
	{
		if (UCommonActivatableWidget* OwningActivatable = ::FindOwningActivatable(Widget))
		{
			FoundNode = FindNode(OwningActivatable);
		}
	}
	return FoundNode;
}

FActivatableTreeNodePtr UCommonUIActionRouterBase::FindNodeRecursive(const FActivatableTreeNodePtr& CurrentNode, const UCommonActivatableWidget& Widget) const
{
	FActivatableTreeNodePtr FoundNode;
	if (CurrentNode)
	{
		if (CurrentNode->GetWidget() == &Widget)
		{
			FoundNode = CurrentNode;
		}
		else
		{
			for (const FActivatableTreeNodePtr Child : CurrentNode->GetChildren())
			{
				FoundNode = FindNodeRecursive(Child, Widget);
				if (FoundNode)
				{
					break;
				}
			}
		}
	}
	return FoundNode;
}

FActivatableTreeNodePtr UCommonUIActionRouterBase::FindNodeRecursive(const FActivatableTreeNodePtr& CurrentNode, const TSharedPtr<SWidget>& Widget) const
{
	FActivatableTreeNodePtr FoundNode;
	if (CurrentNode)
	{
		TSharedPtr<SWidget> CachedWidget = CurrentNode->GetWidget() ? CurrentNode->GetWidget()->GetCachedWidget() : nullptr;

		// only want to check leaf nodes
		if (CurrentNode->GetChildren().Num() == 0)
		{
			if (CurrentNode->IsExclusiveParentOfWidget(Widget))
			{
				FoundNode = CurrentNode;
			}
		}
		else
		{
			for (const FActivatableTreeNodePtr Child : CurrentNode->GetChildren())
			{
				FoundNode = FindNodeRecursive(Child, Widget);
				if (FoundNode)
				{
					break;
				}
			}
		}
	}
	return FoundNode;
}

void UCommonUIActionRouterBase::SetActiveUIInputConfig(const FUIInputConfig& NewConfig, const UObject* InConfigSource)
{
#if WITH_SLATE_DEBUGGING
	if (bTraceInputConfig && InConfigSource && (!ActiveInputConfig.IsSet() || NewConfig != ActiveInputConfig.GetValue()))
	{
		InputConfigSources[InputConfigSourceIndex] = InConfigSource->GetName();

		TStringBuilder<1024> Builder;
		Builder.Appendf(TEXT("Input Config Change:\n%s\n"), *NewConfig.ToString());
		Builder.Appendf(TEXT("--- Input Config Source History (Newest Last) ---\n"));
		for (uint32 Index = 0; Index < InputConfigSources.Capacity(); Index++)
		{
			uint32 CircularIndex = InputConfigSources.GetNextIndex(Index + InputConfigSourceIndex);
			Builder.Appendf(TEXT("%s\n"), *InputConfigSources[CircularIndex]);
		}
		FString InputConfigTrace = Builder.ToString();

		UE_LOGF(LogUIActionRouter, Log, "%ls", *InputConfigTrace);
		if (bTraceConfigOnScreen)
		{
			uint64 OnScreenTraceKey = 202013;
			GEngine->AddOnScreenDebugMessage(OnScreenTraceKey, 15.f, FColor::Cyan, InputConfigTrace);
		}

		InputConfigSourceIndex = InputConfigSources.GetNextIndex(InputConfigSourceIndex);
	}
#endif // WITH_SLATE_DEBUGGING

	ApplyUIInputConfig(NewConfig, !ActiveInputConfig.IsSet());
}

void UCommonUIActionRouterBase::RefreshActionDomainLeafNodeConfig()
{
	// We don't want to refresh if there is an activated root node as we don't want input mode changes
	for (const FActivatableTreeRootRef& Root : RootNodes)
	{
		if (Root->IsWidgetActivated())
		{
			return;
		}
	}

	if (const UCommonInputActionDomainTable* ActionDomainTable = GetActionDomainTable())
	{
		if (RebuiltWidgetsPendingNodeAssignment.Num() > 0)
		{
			RefreshRootNodes();
		}

		if (FActivatableTreeRootPtr RootNode = FindActiveActionDomainRootNode())
		{
			if (!RootNode->UpdateLeafmostActiveNode(RootNode))
			{
				RootNode->ApplyLeafmostNodeConfig();
			}
		}
		else
		{
			SetActiveUIInputConfig(FUIInputConfig(ActionDomainTable->InputMode, ActionDomainTable->MouseCaptureMode, ActionDomainTable->bHideCursorDuringViewportCapture), ActionDomainTable);
		}
	}
}

void UCommonUIActionRouterBase::ApplyUIInputConfig(const FUIInputConfig& NewConfig, bool bForceRefresh)
{
	if (bForceRefresh || NewConfig != ActiveInputConfig.GetValue())
	{
		UE_LOGF(LogUIActionRouter, Display, "UIInputConfig being changed. bForceRefresh: %d, UserIndex: %d", bForceRefresh ? 1 : 0, GetLocalPlayerIndex());
		UE_LOGF(LogUIActionRouter, Display, "\tInputMode: Previous (%ls), New (%ls)",
			ActiveInputConfig.IsSet() ? *StaticEnum<ECommonInputMode>()->GetValueAsString(ActiveInputConfig->GetInputMode()) : TEXT("None"), *StaticEnum<ECommonInputMode>()->GetValueAsString(NewConfig.GetInputMode()));

		const ECommonInputMode PreviousInputMode = GetActiveInputMode();

		TOptional<FUIInputConfig> OldConfig = ActiveInputConfig;
		ActiveInputConfig = NewConfig;

		ULocalPlayer& LocalPlayer = *GetLocalPlayerChecked();

		// Note: may not work for splitscreen. We need per-player viewport client settings for mouse capture
		if (UGameViewportClient* GameViewportClient = LocalPlayer.ViewportClient)
		{
			if (TSharedPtr<SViewport> ViewportWidget = GameViewportClient->GetGameViewportWidget())
			{
				if (APlayerController* PC = LocalPlayer.GetPlayerController(GetWorld()))
				{
					// Ensure we don't ever call SetIgnoreMoveInput(false) unless we previously called SetIgnoreMoveInput(true)
					const bool bOldIgnoreMoveValue = OldConfig.IsSet() && OldConfig.GetValue().bIgnoreMoveInput;
					if (bOldIgnoreMoveValue != NewConfig.bIgnoreMoveInput)
					{
						PC->SetIgnoreMoveInput(NewConfig.bIgnoreMoveInput);						
					}
					// Ensure we don't ever call SetIgnoreLookInput(false) unless we previously called SetIgnoreLookInput(true)
					const bool bOldIgnoreLookValue = OldConfig.IsSet() && OldConfig.GetValue().bIgnoreLookInput;
					if (bOldIgnoreLookValue != NewConfig.bIgnoreLookInput)
					{
						PC->SetIgnoreLookInput(NewConfig.bIgnoreLookInput);						
					}

					if (bAutoFlushPressedKeys && NewConfig.GetInputMode() == ECommonInputMode::Menu && PreviousInputMode != NewConfig.GetInputMode())
					{
						// Flushing pressed keys after switching to the Menu InputMode. This prevents the inputs from being artificially "held down".
						// This needs to be delayed by one frame to successfully clear input captured at the end of this frame
						GetWorld()->GetTimerManager().SetTimerForNextTick(this, &ThisClass::FlushPressedKeys);
					}

					const bool bWasCursorHidden = !PC->ShouldShowMouseCursor();

					GameViewportClient->SetMouseCaptureMode(GetActiveMouseCaptureMode()); // ActiveInputConfig has already been updated with NewConfig
					GameViewportClient->SetHideCursorDuringCapture(NewConfig.HideCursorDuringViewportCapture() && !ShouldAlwaysShowCursor());
					GameViewportClient->SetMouseLockMode(NewConfig.GetMouseLockMode());

					FReply& SlateOperations = LocalPlayer.GetSlateOperations();
					const EMouseCaptureMode CaptureMode = NewConfig.GetMouseCaptureMode();
					switch (CaptureMode)
					{
					case EMouseCaptureMode::CapturePermanently:
					case EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown:
					{
						PC->SetShowMouseCursor(ShouldAlwaysShowCursor() || !NewConfig.HideCursorDuringViewportCapture());

						TSharedRef<SViewport> ViewportWidgetRef = ViewportWidget.ToSharedRef();
						SlateOperations.UseHighPrecisionMouseMovement(ViewportWidgetRef);
						SlateOperations.SetUserFocus(ViewportWidgetRef);
						SlateOperations.CaptureMouse(ViewportWidgetRef);

						if (GameViewportClient->ShouldAlwaysLockMouse() || GameViewportClient->LockDuringCapture() || !PC->ShouldShowMouseCursor())
						{
							SlateOperations.LockMouseToWidget(ViewportWidget.ToSharedRef());
						}
						else
						{
							SlateOperations.ReleaseMouseLock();
						}
					}
					break;
					case EMouseCaptureMode::NoCapture:
					case EMouseCaptureMode::CaptureDuringMouseDown:
					case EMouseCaptureMode::CaptureDuringRightMouseDown:
					{
						PC->SetShowMouseCursor(true);

						SlateOperations.ReleaseMouseCapture();

						if (GameViewportClient->ShouldAlwaysLockMouse())
						{
							SlateOperations.LockMouseToWidget(ViewportWidget.ToSharedRef());
						}
						else
						{
							SlateOperations.ReleaseMouseLock();
						}
					}
					break;
					}

					// If the mouse was hidden previously, set it back to the center of the viewport now that we're showing it again 
					if (!bForceRefresh && bWasCursorHidden && PC->ShouldShowMouseCursor())
					{
						const ECommonInputType CurrentInputType = GetInputSubsystem().GetCurrentInputType();
						
						bool bCenterCursor = true;
						switch (CurrentInputType)
						{
							// Touch - Don't do it - the cursor isn't really relevant there.
							case ECommonInputType::Touch:
								bCenterCursor = false;
								break;
							// Gamepad - Let the settings tell us if we should center it.
							case ECommonInputType::Gamepad:
								break;
						}

						if (bCenterCursor)
						{
							TSharedPtr<FSlateUser> SlateUser = LocalPlayer.GetSlateUser();
							TSharedPtr<IGameLayerManager> GameLayerManager = GameViewportClient->GetGameLayerManager();
							if (ensure(SlateUser) && ensure(GameLayerManager))
							{
								FGeometry PlayerViewGeometry = GameLayerManager->GetPlayerWidgetHostGeometry(&LocalPlayer);
								const FVector2D AbsoluteViewCenter = PlayerViewGeometry.GetAbsolutePositionAtCoordinates(FVector2D(0.5f, 0.5f));
								SlateUser->SetCursorPosition(AbsoluteViewCenter);

								UE_LOGF(LogUIActionRouter, Verbose, "Moving the cursor to the viewport center.");
							}
						}
					}
				}
				else
				{
					UE_LOGF(LogUIActionRouter, Warning, "\tFailed to commit change! Local player controller was null.");
				}
			}
			else
			{
				UE_LOGF(LogUIActionRouter, Warning, "\tFailed to commit change! ViewportWidget was null.");
			}
		}
		else
		{
			UE_LOGF(LogUIActionRouter, Warning, "\tFailed to commit change! GameViewportClient was null.");
		}

		if (PreviousInputMode != NewConfig.GetInputMode())
		{
			if (UEnhancedInputLocalPlayerSubsystem* IE = LocalPlayer.GetSubsystem<UEnhancedInputLocalPlayerSubsystem>())
			{
				IE->RemoveTagsFromInputMode(GetGameplayTagsForInputMode(PreviousInputMode));
				IE->AppendTagsToInputMode(GetGameplayTagsForInputMode(NewConfig.GetInputMode()));
			}
			OnActiveInputModeChanged().Broadcast(NewConfig.GetInputMode());
		}

		OnActiveInputConfigChanged().Broadcast(NewConfig);
	}
}

void UCommonUIActionRouterBase::SetActiveActivationMetadata(const FActivationMetadata& NewConfig)
{
	OnActivationMetadataChanged().Broadcast(NewConfig);
}

FActivatableTreeRootPtr UCommonUIActionRouterBase::FindActiveActionDomainRootNode() const
{
	if (const UCommonInputActionDomainTable* ActionDomainTable = GetActionDomainTable())
	{
		for (const UCommonInputActionDomain* ActionDomain : ActionDomainTable->ActionDomains)
		{
			if (const FActionDomainSortedRootList* SortedRootList = ActionDomainRootNodes.Find(ActionDomain))
			{
				for (const FActivatableTreeRootRef& RootNode : SortedRootList->GetRootList())
				{
					if (RootNode->IsReceivingInput() && RootNode->DoesWidgetSupportActivationFocus())
					{
						return RootNode;
					}
				}
			}
		}
	}

	return nullptr;
}

void UCommonUIActionRouterBase::HandleActivatableWidgetRebuilding(UCommonActivatableWidget& RebuildingWidget)
{
	if (RebuildingWidget.GetOwningLocalPlayer() == GetLocalPlayerChecked())
	{
		RebuiltWidgetsPendingNodeAssignment.AddUnique(&RebuildingWidget);
	}
}

//////////////////////////////////////////////////////////////////////////
// Debug Utils - may merit its own file?
//////////////////////////////////////////////////////////////////////////

extern const TCHAR* InputEventToString(EInputEvent InputEvent);
class FActionRouterDebugUtils
{
public:
	static void HandleDebugDumpTree(const TArray<FString>& Args, UWorld* World)
	{
		if (World == nullptr)
		{
			UE_LOGF(LogUIActionRouter, Error, "No World, unable to run CommonUI.DumpActivatableTree");
			return;
		}

		const bool bIncludeActions = !Args.IsValidIndex(0) || Args[0].ToBool();
		const bool bIncludeChildren = !Args.IsValidIndex(1) || Args[1].ToBool();
		const bool bIncludeInactive = !Args.IsValidIndex(2) || Args[2].ToBool();
		const int LocalPlayerIndex = Args.IsValidIndex(3) ? FCString::Atoi(*Args[3]) : -1;

		UGameInstance* GameInstance = World->GetGameInstance();
		if (GameInstance == nullptr)
		{
			UE_LOGF(LogUIActionRouter, Error, "No GameInstance, unable to run CommonUI.DumpActivatableTree");
			return;
		}

		const TArray<ULocalPlayer*>& LocalPlayers = GameInstance->GetLocalPlayers();
		for (int32 CurrIdx = 0; CurrIdx < LocalPlayers.Num(); ++CurrIdx)
		{
			if (LocalPlayerIndex != -1 && LocalPlayerIndex != CurrIdx)
			{
				continue;
			}
			ULocalPlayer* LocalPlayer = LocalPlayers[CurrIdx];
			if (UCommonUIActionRouterBase* ActionRouter = LocalPlayer->GetSubsystem<UCommonUIActionRouterBase>())
			{
				FString TreeOutputStr;
				
				if (ActionRouter->ActiveRootNode)
				{
					TreeOutputStr.Append(TEXT("** Active Root **"));
					ActionRouter->ActiveRootNode->DebugDump(TreeOutputStr, bIncludeActions, bIncludeChildren, bIncludeInactive);
					TreeOutputStr.Append(TEXT("\n*****************\n"));
				}
				
				for (const FActivatableTreeRootRef& RootNode : ActionRouter->RootNodes)
				{
					if (RootNode != ActionRouter->ActiveRootNode)
					{
						RootNode->DebugDump(TreeOutputStr, bIncludeActions, bIncludeChildren, bIncludeInactive);
					}
				}

				if (bIncludeActions)
				{
					ActionRouter->PersistentActions->DumpActionBindings(TreeOutputStr);
				}

				UE_LOGF(LogUIActionRouter, Display, "Dumping ActivatableWidgetTree for LocalPlayer [User %d, ControllerId %d]:\n\n%ls\n\n", CurrIdx, LocalPlayer->GetControllerId(), *TreeOutputStr);
				
				ActionRouter->DebugDumpActionDomainRootNodes(CurrIdx, LocalPlayer->GetControllerId(), bIncludeActions, bIncludeChildren, bIncludeInactive);
			}
		}
	}

	static void HandleDumpCurrentInputConfig(UWorld* World)
	{
		if (World == nullptr)
		{
			UE_LOGF(LogUIActionRouter, Error, "No World, unable to run CommonUI.DumpInputConfig");
			return;
		}

		static UEnum* InputModeEnum = StaticEnum<ECommonInputMode>();
		static UEnum* MouseCaptureModeEnum = StaticEnum<EMouseCaptureMode>();

		check(InputModeEnum);
		check(MouseCaptureModeEnum);

		UGameInstance* GameInstance = World->GetGameInstance();
		if (!GameInstance)
		{
			UE_LOGF(LogUIActionRouter, Error, "No GameInstance, unable to run CommonUI.DumpInputConfig");
			return;
		}

		FString OutStr;
		const TArray<ULocalPlayer*> LocalPlayers = GameInstance->GetLocalPlayers();
		for (int32 i = 0; i < LocalPlayers.Num(); ++i)
		{
			if (ULocalPlayer* LocalPlayer = LocalPlayers[i])
			{
				const int32 ControllerId = LocalPlayer->GetControllerId();
				if (UCommonUIActionRouterBase* ActionRouter = LocalPlayer->GetSubsystem<UCommonUIActionRouterBase>())
				{
					if (ActionRouter->ActiveInputConfig.IsSet())
					{
						FString InputModeStr = InputModeEnum->GetNameStringByValue((int64)ActionRouter->ActiveInputConfig->GetInputMode());
						FString MouseCaptureStr = MouseCaptureModeEnum->GetNameStringByValue((int64)ActionRouter->ActiveInputConfig->GetMouseCaptureMode());
						FString HideStr = ActionRouter->ActiveInputConfig->HideCursorDuringViewportCapture() ? TEXT("Yes") : TEXT("No");
						OutStr += FString::Printf(TEXT("\tLocalPlayer[User %d, ControllerId %d] ActiveInputConfig: Input Mode [%s] Mouse Capture [%s] Hide Cursor During Capture [%s]\n"), i, ControllerId, *InputModeStr, *MouseCaptureStr, *HideStr);
					}
					else
					{
						OutStr += FString::Printf(TEXT("LocalPlayer [User %d, ControllerId %d] no ActiveInputConfig\n"), i, ControllerId);

					}
				}
				else
				{
					OutStr += FString::Printf(TEXT("LocalPlayer [User %d, Controller %d] has no ActionRouter\n"), i, ControllerId);
				}
			}
		}
		UE_LOGF(LogUIActionRouter, Display, "Dumping all Input configs:\n%ls", *OutStr);
	}
};

static const FAutoConsoleCommandWithWorldAndArgs DumpActivatableTreeCommand(
	TEXT("CommonUI.DumpActivatableTree"),
	TEXT("Outputs the current state of the activatable tree. 4 args: bIncludeActions, bIncludeChildren, bIncludeInactive, LocalPlayerId (optional, defaults to -1 or all)"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FActionRouterDebugUtils::HandleDebugDumpTree)
);

static const FAutoConsoleCommandWithWorld DumpInputConfigCommand(
	TEXT("CommonUI.DumpInputConfig"),
	TEXT("Outputs the current Input Config for each player"),
	FConsoleCommandWithWorldDelegate::CreateStatic(&FActionRouterDebugUtils::HandleDumpCurrentInputConfig)
);

void UCommonUIActionRouterBase::FActionDomainSortedRootList::Add(FActivatableTreeRootRef RootNode)
{
	RootList.Add(RootNode);
	Sort();
}

int32 UCommonUIActionRouterBase::FActionDomainSortedRootList::Remove(FActivatableTreeRootRef RootNode)
{
	return RootList.Remove(RootNode);
}

bool UCommonUIActionRouterBase::FActionDomainSortedRootList::Contains(FActivatableTreeRootRef RootNode) const
{
	return RootList.Contains(RootNode);
}

void UCommonUIActionRouterBase::FActionDomainSortedRootList::Sort()
{
	auto SortFunc = [](const FActivatableTreeRootRef& A, const FActivatableTreeRootRef& B) 
	{
		return A->GetLastPaintLayer() > B->GetLastPaintLayer();
	};

	RootList.Sort(SortFunc);
}

void UCommonUIActionRouterBase::FActionDomainSortedRootList::DebugDumpRootList(FString& OutputStr, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const
{
	for (const FActivatableTreeRootRef& Root : RootList)
	{
		Root->DebugDump(OutputStr, bIncludeActions, bIncludeChildren, bIncludeInactive);
	}
}

