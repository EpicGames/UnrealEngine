// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/CommonUIInputSettings.h"
#include "GameplayTagContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonUIInputSettings)

const UCommonUIInputSettings& UCommonUIInputSettings::Get()
{
	return *GetDefault<UCommonUIInputSettings>();
}

void UCommonUIInputSettings::PostInitProperties()
{
	Super::PostInitProperties();
}

const FUIInputAction* UCommonUIInputSettings::FindAction(FUIActionTag ActionTag) const
{
	return InputActions.FindByPredicate([ActionTag](const FUIInputAction& Action) { return Action.ActionTag == ActionTag; });
}

