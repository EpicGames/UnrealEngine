// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/UIActionRouterTypes.h"
#include "CommonUITypes.h"
#include "Input/CommonUIActionRouterBase.h"
#include "CommonActivatableWidget.h"
#include "Input/CommonUIInputTypes.h"
#include "InputAction.h"
#include "InputActionValue.h"
#include "CommonInputSettings.h"
#include "CommonInputSubsystem.h"
#include "ICommonInputModule.h"
#include "Input/CommonUIInputSettings.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Input/UIActionBinding.h"
#include "Widgets/SViewport.h"
#include "CommonButtonBase.h"
#include "CommonUIEditorSettings.h"
#include "EnhancedInputSubsystems.h"
#include "Engine/GameInstance.h"
#include "ICommonUIModule.h"

DEFINE_LOG_CATEGORY(LogUIActionRouter);

static const TAutoConsoleVariable<bool> CVarGamepadFocusHoveredWidget(
	TEXT("CommonUI.GamepadFocusHoveredWidget"),
	false,
	TEXT("If enabled, the mouse hovered widget, will be focused when switching to gamepad input"));

static const TAutoConsoleVariable<bool> CVarSyncCursorToFocusedWidgetOnInputMethodChange(
	TEXT("CommonUI.SyncCursorToFocusedWidgetOnInputMethodChange"),
	true,
	TEXT("Whether or not to move the cursor to the currently focused widget when input method changes to gamepad (fix for losing focus when switching between touch and gamepad inputs)."),
	ECVF_Default);

static const TAutoConsoleVariable<bool> CVarCheckKeyboardFocusAndParentage(
	TEXT("CommonUI.CheckKeyboardFocusAndParentage"),
	false,
	TEXT("If enabled, we will check for keyboard focus and parentage before setting focus"));

static const TAutoConsoleVariable<bool> CVarFallbackToDesiredOnAutoRestoreFailure(
	TEXT("CommonUI.FallbackToDesiredOnAutoRestoreFailure"),
	false,
	TEXT("If enabled, we set focus to Desired Widget if Auto Restore Focus Fails"));

static const TAutoConsoleVariable<bool> CVarDisallowUserFocusedWidgetForPendingFocusRecipient(
	TEXT("CommonUI.DisallowUserFocusedWidgetForPendingFocusRecipient"),
	false,
	TEXT("If enabled, we only check the currently focused widget when there is no pending focus recipient"));

static bool bEnhancedInputDynamicallyResolveHoldKeysForInputAction = false;
static FAutoConsoleVariableRef CVarEnhancedInputDynamicallyResolveHoldKeysForInputAction(
	TEXT("CommonUI.bEnhancedInputDynamicallyResolveHoldKeysForInputAction"),
	bEnhancedInputDynamicallyResolveHoldKeysForInputAction,
	TEXT("If enabled, triggering keys assigned to InputAction for hold triggers will be resolved dynamically during key press processing (this allows remapped keys)."));
	
static bool bAlwaysWaitForHoldActionToComplete = false;
static FAutoConsoleVariableRef CVarAlwaysWaitForHoldActionToComplete(
	TEXT("CommonUI.AlwaysWaitForHoldActionToComplete"),
	bAlwaysWaitForHoldActionToComplete,
	TEXT("If enabled, always wait for input hold time to reach triggering threshold before generating a press event (otherwise a press event will be triggered before threshold is reached if press and quick release key)."));

const TCHAR* InputEventToString(EInputEvent InputEvent)
{
	switch (InputEvent)
	{
	case IE_Pressed: return TEXT("Pressed");
	case IE_Released: return TEXT("Released");
	case IE_Repeat: return TEXT("Repeat");
	case IE_DoubleClick: return TEXT("DoubleClick");
	case IE_Axis: return TEXT("Axis");
	}
	return TEXT("Invalid");
}

bool IsWidgetInNodeHierarchy(UCommonActivatableWidget* TargetWidget, const FActivatableTreeNode& TargetNode)
{
	if (TargetNode.GetWidget() == TargetWidget)
		return true;
	else
	{
		bool bFound = false;
		for (const FActivatableTreeNodeRef& ChildNode : TargetNode.GetChildren())
		{
			const FActivatableTreeNode& ChildNodeRef = ChildNode.Get();
			bFound |= IsWidgetInNodeHierarchy(TargetWidget, ChildNodeRef);
			if (bFound)
				break;
		}
		return bFound;
	}
}

#if WITH_EDITOR
bool IsViewportWindowInFocusPath(const UCommonUIActionRouterBase& ActionRouter)
{
	ULocalPlayer& LocalPlayer = *ActionRouter.GetLocalPlayerChecked();
	if (const TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(ActionRouter.GetLocalPlayerIndex()))
	{
		if (TSharedPtr<SWindow> Window = LocalPlayer.ViewportClient ? LocalPlayer.ViewportClient->GetWindow() : nullptr)
		{
			return SlateUser->HasFocus(Window) || SlateUser->HasFocusedDescendants(Window.ToSharedRef());
		}
	}
	return true;
}
#endif

bool CanWidgetReceiveInput(SWidget& SlateWidget)
{
	if (!SlateWidget.IsEnabled())
	{
		return false;
	}

	const EVisibility WidgetVisibility = SlateWidget.GetVisibility();
	if (!WidgetVisibility.IsVisible() || !WidgetVisibility.AreChildrenHitTestVisible())
	{
		return false;
	}

	TSharedPtr<SWidget> ParentWidget = SlateWidget.GetParentWidget();
	if (ParentWidget && !ParentWidget->ValidatePathToChild(&SlateWidget))
	{
		// The parent doesn't have a valid path to the child (i.e. it's an inactive child on a switcher), so bail
		return false;
	}

	TSharedPtr<FCommonButtonMetaData> CommonButtonMetaData = SlateWidget.GetMetaData<FCommonButtonMetaData>();
	if (CommonButtonMetaData && !(CommonButtonMetaData->OwningCommonButton->IsInteractionEnabled()))
	{
		return false;
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////
// FUIActionBinding
//////////////////////////////////////////////////////////////////////////

int32 FUIActionBinding::IdCounter = 0;
TMap<FUIActionBindingHandle, TSharedPtr<FUIActionBinding>> FUIActionBinding::AllRegistrationsByHandle;

FUIActionBinding::FUIActionBinding(const UWidget& InBoundWidget, const FBindUIActionArgs& BindArgs)
	: ActionName(BindArgs.GetActionName())
	, InputEvent(BindArgs.KeyEvent)
	, bConsumesInput(BindArgs.bConsumeInput)
	, bIsPersistent(BindArgs.bIsPersistent)
	, PriorityWithinCollection(BindArgs.PriorityWithinCollection)
	, BoundWidget(&InBoundWidget)
	, InputMode(BindArgs.InputMode)
	, bDisplayInActionBar(BindArgs.bDisplayInActionBar)
	, InputTypesExemptFromValidKeyCheck(BindArgs.InputTypesExemptFromValidKeyCheck)
	, ActionDisplayName(BindArgs.OverrideDisplayName)
	, OnExecuteAction(BindArgs.OnExecuteAction)
	, Handle(IdCounter++)
	, LegacyActionTableRow(BindArgs.LegacyActionTableRow)
	, InputAction(BindArgs.InputAction)
{
	OnHoldActionProgressed.Add(BindArgs.OnHoldActionProgressed);
	OnHoldActionPressed.Add(BindArgs.OnHoldActionPressed);
	OnHoldActionReleased.Add(BindArgs.OnHoldActionReleased);

	const auto RegisterKeyMappingFunc = 
		[this](const FUIActionKeyMapping& KeyMapping)
		{
			if (KeyMapping.Key.IsValid())
			{
				if (KeyMapping.HoldTime > 0.f)
				{
					HoldMappings.Add(KeyMapping);
				}
				else
				{
					NormalMappings.Add(KeyMapping);
				}
			}
		};

	if (BindArgs.ActionTag.IsValid())
	{
		const FUIInputAction* ActionMapping = UCommonUIInputSettings::Get().FindAction(BindArgs.ActionTag);
		check(ActionMapping);
		
		if (ActionDisplayName.IsEmpty())
		{
			ActionDisplayName = ActionMapping->DefaultDisplayName;
		}

		for (const FUIActionKeyMapping& KeyMapping : ActionMapping->KeyMappings)
		{
			RegisterKeyMappingFunc(KeyMapping);
		}
	}
	else if (CommonUI::IsEnhancedInputSupportEnabled() && BindArgs.InputAction.IsValid())
	{
		if (ActionDisplayName.IsEmpty())
		{
			ActionDisplayName = BindArgs.InputAction->ActionDescription;
		}

		// Nothing else to do if we have an enhanced input action,
		// the input action itself will be queried against for keys later
	}
	else
	{
		const FCommonInputActionDataBase* LegacyActionData = GetLegacyInputActionData();
		check(LegacyActionData);
		
		if (ActionDisplayName.IsEmpty())
		{
			ActionDisplayName = LegacyActionData->DisplayName;
		}

		FName CurrentGamepadName = UCommonUIActionRouterBase::Get(InBoundWidget)->GetInputSubsystem().GetCurrentGamepadName();
		
		// KB/M
		const FCommonInputTypeInfo& LegacyMapping_KBM = LegacyActionData->GetInputTypeInfo(ECommonInputType::MouseAndKeyboard, FCommonInputDefaults::GamepadGeneric);
		FUIActionKeyMapping KeyMapping_KBM(LegacyMapping_KBM.GetKey(), LegacyMapping_KBM.bActionRequiresHold ? LegacyMapping_KBM.HoldTime : 0.f, LegacyMapping_KBM.bActionRequiresHold ? LegacyMapping_KBM.HoldRollbackTime : 0.f);

		// Touch
		const FCommonInputTypeInfo& LegacyMapping_Touch = LegacyActionData->GetInputTypeInfo(ECommonInputType::Touch, FCommonInputDefaults::GamepadGeneric);
		FUIActionKeyMapping KeyMapping_Touch(LegacyMapping_Touch.GetKey(), LegacyMapping_Touch.bActionRequiresHold ? LegacyMapping_Touch.HoldTime : 0.f, LegacyMapping_Touch.bActionRequiresHold ? LegacyMapping_Touch.HoldRollbackTime : 0.f);

		// Gamepad
		// Note: This is definitely a wonky and roundabout way to get the gamepad type, but given that it's for legacy fixup we'll let it go until this can be deleted
		UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(InBoundWidget);
		check(ActionRouter);
		const FCommonInputTypeInfo& LegacyMapping_Gamepad = LegacyActionData->GetInputTypeInfo(ECommonInputType::Gamepad, ActionRouter->GetInputSubsystem().GetCurrentGamepadName());
		FUIActionKeyMapping KeyMapping_Gamepad(LegacyMapping_Gamepad.GetKey(), LegacyMapping_Gamepad.bActionRequiresHold ? LegacyMapping_Gamepad.HoldTime : 0.f, LegacyMapping_Gamepad.bActionRequiresHold ? LegacyMapping_Gamepad.HoldRollbackTime : 0.f);
		
		if (BindArgs.bForceHold && ICommonInputModule::GetSettings().GetDefaultHoldData())
        {
        	UCommonUIHoldData* CommonUIHoldBehaviorValues = ICommonInputModule::GetSettings().GetDefaultHoldData().GetDefaultObject();
        			
        	KeyMapping_KBM.HoldTime = KeyMapping_KBM.HoldTime > 0.f ? KeyMapping_KBM.HoldTime : CommonUIHoldBehaviorValues->KeyboardAndMouse.HoldTime;
        	KeyMapping_Touch.HoldTime = KeyMapping_Touch.HoldTime > 0.f ? KeyMapping_Touch.HoldTime : CommonUIHoldBehaviorValues->Touch.HoldTime;
        	KeyMapping_Gamepad.HoldTime = KeyMapping_Gamepad.HoldTime > 0.f ? KeyMapping_Gamepad.HoldTime : CommonUIHoldBehaviorValues->Gamepad.HoldTime;
        }
        		
        RegisterKeyMappingFunc(KeyMapping_KBM);
        RegisterKeyMappingFunc(KeyMapping_Touch);
		RegisterKeyMappingFunc(KeyMapping_Gamepad);

		// Register additional keys for each input type
		auto RegisterAdditionalKeysForTypeInfo = [&](const FCommonInputTypeInfo& TypeInfo, float ForceHoldTime)
		{
			const float HoldTime = TypeInfo.bActionRequiresHold ? TypeInfo.HoldTime : ForceHoldTime;
			const float RollbackTime = TypeInfo.bActionRequiresHold ? TypeInfo.HoldRollbackTime : 0.f;
			for (const FKey& AdditionalKey : TypeInfo.GetAllKeys())
			{
				if (AdditionalKey != TypeInfo.GetKey())
				{
					RegisterKeyMappingFunc(FUIActionKeyMapping(AdditionalKey, HoldTime, RollbackTime));
				}
			}
		};

		float ForceHoldTime_KBM = 0.f;
		float ForceHoldTime_Touch = 0.f;
		float ForceHoldTime_Gamepad = 0.f;
		if (BindArgs.bForceHold && ICommonInputModule::GetSettings().GetDefaultHoldData())
		{
			UCommonUIHoldData* CommonUIHoldBehaviorValues = ICommonInputModule::GetSettings().GetDefaultHoldData().GetDefaultObject();
			ForceHoldTime_KBM = CommonUIHoldBehaviorValues->KeyboardAndMouse.HoldTime;
			ForceHoldTime_Touch = CommonUIHoldBehaviorValues->Touch.HoldTime;
			ForceHoldTime_Gamepad = CommonUIHoldBehaviorValues->Gamepad.HoldTime;
		}

		RegisterAdditionalKeysForTypeInfo(LegacyMapping_KBM, ForceHoldTime_KBM);
		RegisterAdditionalKeysForTypeInfo(LegacyMapping_Touch, ForceHoldTime_Touch);
		RegisterAdditionalKeysForTypeInfo(LegacyMapping_Gamepad, ForceHoldTime_Gamepad);
	}

#if !UE_BUILD_SHIPPING
	Handle.CachedDebugActionName = BindArgs.GetActionName().ToString();
#endif
}

FUIActionBindingHandle FUIActionBinding::TryCreate(const UWidget& InBoundWidget, const FBindUIActionArgs& BindArgs)
{
	const ULocalPlayer* OwningPlayer = InBoundWidget.GetOwningLocalPlayer();
	const int32 UserIndex = OwningPlayer ? OwningPlayer->GetLocalPlayerIndex() : INDEX_NONE;
	return TryCreate(InBoundWidget, BindArgs, UserIndex);
}

FUIActionBindingHandle FUIActionBinding::TryCreate(const UWidget& InBoundWidget, const FBindUIActionArgs& BindArgs, int32 UserIndex)
{
	bool bIsEnhancedInputSupportEnabled = CommonUI::IsEnhancedInputSupportEnabled();
	const FName BindActionName = BindArgs.GetActionName();
	if (BindActionName.IsNone() && (!bIsEnhancedInputSupportEnabled || !BindArgs.InputAction.IsValid()))
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot create action binding for widget [%ls] - no action provided.", *InBoundWidget.GetName());
		return FUIActionBindingHandle();
	}
	else if (!BindArgs.OnExecuteAction.IsBound())
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot bind widget [%ls] to action [%ls] - empty handler delegate.", *InBoundWidget.GetName(), *BindActionName.ToString());
		return FUIActionBindingHandle();
	}
	else if (BindArgs.ActionTag.IsValid() && !UCommonUIInputSettings::Get().FindAction(BindArgs.ActionTag))
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot bind widget [%ls] to action [%ls] - provided tag does not map to an existing UI input action. It can be added under Project Settings->UI Input.", *InBoundWidget.GetName(), *BindArgs.GetActionName().ToString());
		return FUIActionBindingHandle();
	}
	else if (!BindArgs.LegacyActionTableRow.IsNull() && !BindArgs.LegacyActionTableRow.GetRow<FCommonInputActionDataBase>(TEXT("")))
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot bind widget [%ls] to action [%ls] - provided legacy data table row does not resolve to valid data.", *InBoundWidget.GetName(), *BindActionName.ToString());
		return FUIActionBindingHandle();
	}
	else if (!BindArgs.ActionTag.IsValid() && BindArgs.LegacyActionTableRow.IsNull() && bIsEnhancedInputSupportEnabled && !BindArgs.InputAction.IsValid())
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot bind widget [%ls] to action [%ls] - provided input action is invalid.", *InBoundWidget.GetName(), *BindActionName.ToString());
		return FUIActionBindingHandle();
	}
	else if (UserIndex == INDEX_NONE)
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot bind widget [%ls] to action [%ls] [%ls] - invalid UserIndex.", *BindActionName.ToString(), InputEventToString(BindArgs.KeyEvent), *InBoundWidget.GetName());
		return FUIActionBindingHandle();
	}
	
	// Make sure there is no existing binding for the same action associated with the same widget
	for (const TPair<FUIActionBindingHandle, TSharedPtr<FUIActionBinding>>& HandleBindingPair : AllRegistrationsByHandle)
	{
		if (HandleBindingPair.Value->ActionName == BindActionName &&
			HandleBindingPair.Value->InputEvent == BindArgs.KeyEvent &&
			HandleBindingPair.Value->BoundWidget.Get() == &InBoundWidget &&
			HandleBindingPair.Value->UserIndex == UserIndex)
		{
			UE_LOGF(LogUIActionRouter, Error, "Widget [%ls] is already bound to action [%ls] [%ls] for user [%d]! A given widget can only bind the same action once per user. Unregister existing binding first if you wish to change any aspect of the binding", 
				*InBoundWidget.GetName(), *HandleBindingPair.Value->ActionName.ToString(), InputEventToString(BindArgs.KeyEvent), UserIndex);
			return FUIActionBindingHandle();
		}
	}
	
	TSharedPtr<FUIActionBinding> NewRegistration = MakeShareable(new FUIActionBinding(InBoundWidget, BindArgs));
	NewRegistration->UserIndex = UserIndex;
	FUIActionBindingHandle Handle = NewRegistration->Handle;
	AllRegistrationsByHandle.Add(Handle, MoveTemp(NewRegistration));

	return Handle;
}

TSharedPtr<FUIActionBinding> FUIActionBinding::FindBinding(FUIActionBindingHandle Handle)
{
	if (TSharedPtr<FUIActionBinding>* Ptr = AllRegistrationsByHandle.Find(Handle))
	{
		return *Ptr;
	}

	return nullptr;
}

void FUIActionBinding::CleanRegistrations()
{
	int32 NumRemoved = 0;
	for (auto Iter = AllRegistrationsByHandle.CreateIterator(); Iter; ++Iter)
	{
		if (!Iter->Value->BoundWidget.IsValid() || !ensureMsgf(Iter->Value->OnExecuteAction.IsBound(), TEXT("Unbound Action: %s"), *Iter->Value->ActionName.ToString()))
		{
			if (Iter->Value->OwningCollection.IsValid())
			{
				Iter->Value->OwningCollection.Pin()->RemoveBinding(Iter->Key);
			}

			Iter.RemoveCurrent();
			++NumRemoved;
		}
	}

	if (NumRemoved > 0)
	{
		UE_LOGF(LogUIActionRouter, Log, "Cleaned out [%d] inactive UI action bindings", NumRemoved);
	}
}

FCommonInputActionDataBase* FUIActionBinding::GetLegacyInputActionData() const
{
	return LegacyActionTableRow.GetRow<FCommonInputActionDataBase>(TEXT(""));
}

FString FUIActionBinding::ToDebugString() const
{
	return FString::Printf(TEXT("%s: Owner [%s], Mode [%s], Displayed? [%s], UserIndex [%d]"),
		*ActionName.ToString(),
		BoundWidget.IsValid() ? *BoundWidget->GetName() : TEXT("Invalid"),
		LexToString(InputMode),
		*LexToString(bDisplayInActionBar),
		UserIndex);
}

void FUIActionBinding::BeginHold()
{
	if (HoldProgressRollbackTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(HoldProgressRollbackTickerHandle);
		HoldProgressRollbackTickerHandle = nullptr;
	}
	
	UE_LOGF(LogUIActionRouter, VeryVerbose, "Hold pressed: %ls", *ActionName.ToString());

	HoldStartTime = FPlatformTime::Seconds();
	
	const float HoldStartPercent = HoldTime > 0.f ? HoldStartSecond / HoldTime : 0.f;
    OnHoldActionProgressed.Broadcast(HoldStartPercent);
}

bool FUIActionBinding::UpdateHold(float TargetHoldTime)
{
	CurrentHoldSecond = FMath::Clamp(GetSecondsHeld() + HoldStartSecond, 0.f, TargetHoldTime);
	UE_LOGF(LogUIActionRouter, VeryVerbose, "Hold repeating: %ls for %f", *ActionName.ToString(), CurrentHoldSecond);
	
	const float HeldPercent = FMath::Clamp(CurrentHoldSecond / TargetHoldTime, 0.f, 1.f);
	
	OnHoldActionProgressed.Broadcast(HeldPercent);

	if (HeldPercent >= 1.f)
	{
		CancelHold();
		OnExecuteAction.ExecuteIfBound();
		OnHoldActionProgressed.Broadcast(0.0f); //approximate CommonButton::OnActionComplete without adding extra bind
		HoldStartSecond = 0.0f;
		CurrentHoldSecond = 0.0f;

		UE_LOGF(LogUIActionRouter, VeryVerbose, "Hold repeating: Fired completed action! %ls after %f", *ActionName.ToString(), CurrentHoldSecond);
		return true;
	}
	return false;
}

double FUIActionBinding::GetSecondsHeld() const
{
	double Duration = 0.0;
	if (IsHoldActive())
	{
		Duration = FPlatformTime::Seconds() - HoldStartTime;
	}
	return Duration;
}

void FUIActionBinding::CancelHold()
{
	HoldStartTime = -1.0;
}

void FUIActionBinding::BeginRollback(float TargetHoldRollbackTime, float InHoldTime, FUIActionBindingHandle BindingHandle)
{
	HoldRollbackMultiplier = InHoldTime / TargetHoldRollbackTime;
	HoldTime = InHoldTime;
	
	HoldProgressRollbackTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([BindingHandle](float DeltaTime)
	{
		if (TSharedPtr<FUIActionBinding> InBinding = FindBinding(BindingHandle))
		{
			const float HoldRollbackTime = InBinding->HoldTime / InBinding->HoldRollbackMultiplier;
			InBinding->CurrentHoldSecond = FMath::Clamp(InBinding->CurrentHoldSecond - (DeltaTime * InBinding->HoldRollbackMultiplier), 0.f, HoldRollbackTime);
			const float HoldRollbackPercent = FMath::Clamp(InBinding->CurrentHoldSecond / InBinding->HoldTime, 0.f, 1.f);
			InBinding->OnHoldActionProgressed.Broadcast(HoldRollbackPercent);
			InBinding->HoldStartSecond = InBinding->CurrentHoldSecond;
            			
			if (HoldRollbackPercent <= 0.f)
			{
				FTSTicker::RemoveTicker(InBinding->HoldProgressRollbackTickerHandle);
				InBinding->HoldProgressRollbackTickerHandle = nullptr;
				InBinding->HoldStartSecond = 0.f;
				InBinding->CurrentHoldSecond = 0.f;
            						
				return false;
			}
            				
			return true;
		}
		
		return false;			
	}));
}

bool FUIActionBinding::IsHoldActive() const
{
	return HoldStartTime >= 0.0;
}

void FUIActionBinding::ResetHold()
{
	if (HoldProgressRollbackTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(HoldProgressRollbackTickerHandle);
		HoldProgressRollbackTickerHandle = nullptr;
	}
	HoldStartSecond = 0.0f;
	CurrentHoldSecond = 0.0f;
}

//////////////////////////////////////////////////////////////////////////
// FBindUIActionArgs
//////////////////////////////////////////////////////////////////////////

FName GetActionNameFromLegacyRow(const FDataTableRowHandle& LegacyRow)
{
	if (LegacyRow.DataTable)
	{
		return *FString::Printf(TEXT("Legacy_%s_%s"), *LegacyRow.DataTable->GetName(), *LegacyRow.RowName.ToString());
	}
	return FName();
}

FName FBindUIActionArgs::GetActionName() const
{
	if (InputAction.IsValid())
	{
		return InputAction->GetFName();
	}
	
	return LegacyActionTableRow.IsNull() ? ActionTag.GetTagName() : GetActionNameFromLegacyRow(LegacyActionTableRow);
}

bool FBindUIActionArgs::ActionHasHoldMappings() const
{
	bool bHasHolds = false;

	if (!LegacyActionTableRow.IsNull())
	{
		if (FCommonInputActionDataBase* InputActionData = LegacyActionTableRow.GetRow<FCommonInputActionDataBase>(TEXT("")))
		{
			bHasHolds = InputActionData->HasHoldBindings();
		}
	}

	return bHasHolds;
}

//////////////////////////////////////////////////////////////////////////
// FUIActionBindingHandle
//////////////////////////////////////////////////////////////////////////

bool FUIActionBindingHandle::IsValid() const
{
	return RegistrationId >= 0 && FUIActionBinding::FindBinding(*this) != nullptr;
}

void FUIActionBindingHandle::Unregister()
{
	if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(*this))
	{
		if (Binding->OwningCollection.IsValid())
		{
			Binding->OwningCollection.Pin()->RemoveBinding(*this);
		}
	}

	FUIActionBinding::AllRegistrationsByHandle.Remove(*this);
	RegistrationId = INDEX_NONE;
}

void FUIActionBindingHandle::ResetHold()
{
	if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(*this))
	{
		Binding->ResetHold();
	}
}

FName FUIActionBindingHandle::GetActionName() const
{
	if (TSharedPtr<const FUIActionBinding> Binding = FUIActionBinding::FindBinding(*this))
	{
		return Binding->ActionName;
	}
	return NAME_None;
}

FText FUIActionBindingHandle::GetDisplayName() const
{
	if (TSharedPtr<const FUIActionBinding> Binding = FUIActionBinding::FindBinding(*this))
	{
		if (const UCommonInputSubsystem* CommonInputSubsystem = Binding->BoundWidget.IsValid() ? UCommonInputSubsystem::Get(Binding->BoundWidget->GetOwningLocalPlayer()) : nullptr)
		{
			const ECommonInputType CurrentInputType = CommonInputSubsystem->GetCurrentInputType();

			for (const FUIActionKeyMapping& HoldMapping : Binding->HoldMappings)
			{
				const bool bIsGamepadKey = HoldMapping.Key.IsGamepadKey();
				const bool bIsTouch = HoldMapping.Key.IsTouch();

				if ((bIsGamepadKey && CurrentInputType == ECommonInputType::Gamepad) ||
					(bIsTouch && CurrentInputType == ECommonInputType::Touch) ||
					(!bIsGamepadKey && !bIsTouch && CurrentInputType == ECommonInputType::MouseAndKeyboard))
				{
					// We've got a hold mapping that's relevant to the current input method
					return FText::Format(NSLOCTEXT("UIActionRouter", "HoldActionNameFormat", "{0} (Hold)"), Binding->ActionDisplayName);
				}
			}
		}
		return Binding->ActionDisplayName;
	}
	return FText();
}

void FUIActionBindingHandle::SetDisplayName(const FText& DisplayName)
{
	FUIActionBinding* Binding = FUIActionBinding::FindBinding(*this).Get();

	if (Binding && !Binding->ActionDisplayName.EqualTo(DisplayName))
	{
		Binding->ActionDisplayName = DisplayName;

		if (const UWidget* BoundWidget = Binding->BoundWidget.Get())
		{
			if (const UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*BoundWidget))
			{
				ActionRouter->OnBoundActionsUpdated().Broadcast();
			}
		}
	}
}

bool FUIActionBindingHandle::GetDisplayInActionBar() const
{
	bool bDisplayInActionBar = false;

	if (const FUIActionBinding* Binding = FUIActionBinding::FindBinding(*this).Get())
	{
		bDisplayInActionBar = Binding->bDisplayInActionBar;
	}

	return bDisplayInActionBar;
}

void FUIActionBindingHandle::SetDisplayInActionBar(const bool bDisplayInActionBar)
{
	FUIActionBinding* Binding = FUIActionBinding::FindBinding(*this).Get();

	if (Binding && Binding->bDisplayInActionBar != bDisplayInActionBar)
	{
		Binding->bDisplayInActionBar = bDisplayInActionBar;

		if (const UWidget* BoundWidget = Binding->BoundWidget.Get())
		{
			if (const UCommonUIActionRouterBase* ActionRouter = UCommonUIActionRouterBase::Get(*BoundWidget))
			{
				ActionRouter->OnBoundActionsUpdated().Broadcast();
			}
		}
	}
}

const UWidget* FUIActionBindingHandle::GetBoundWidget() const
{
	if (TSharedPtr<const FUIActionBinding> Binding = FUIActionBinding::FindBinding(*this))
	{
		return Binding->BoundWidget.Get();
	}
	return nullptr;
}

ULocalPlayer* FUIActionBindingHandle::GetBoundLocalPlayer() const
{
	if (TSharedPtr<const FUIActionBinding> Binding = FUIActionBinding::FindBinding(*this))
	{
		if (const UWidget* BoundWidget = Binding->BoundWidget.Get())
		{
			const UGameInstance* GameInstance = BoundWidget->GetGameInstance();
			if (GameInstance && Binding->UserIndex != INDEX_NONE)
			{
				return GameInstance->GetLocalPlayerByIndex(Binding->UserIndex);
			}
			else
			{
				return BoundWidget->GetOwningLocalPlayer();
			}
		}
	}
	return nullptr;
}


//////////////////////////////////////////////////////////////////////////
// FUIInputConfig
//////////////////////////////////////////////////////////////////////////

FUIInputConfig::FUIInputConfig()
	: InputMode(ECommonInputMode::Menu)
	, MouseCaptureMode(EMouseCaptureMode::NoCapture)
	, MouseLockMode(EMouseLockMode::DoNotLock)
{}

FUIInputConfig::FUIInputConfig(ECommonInputMode InInputMode, EMouseCaptureMode InMouseCaptureMode, bool bInHideCursorDuringViewportCapture)
	: InputMode(InInputMode)
	, MouseCaptureMode(InMouseCaptureMode)
	, bHideCursorDuringViewportCapture(bInHideCursorDuringViewportCapture)
{
	switch (MouseCaptureMode)
	{
	case EMouseCaptureMode::CapturePermanently:
	case EMouseCaptureMode::CapturePermanently_IncludingInitialMouseDown:
		MouseLockMode = EMouseLockMode::LockOnCapture;
		break;
	default:
		MouseLockMode = EMouseLockMode::DoNotLock;
		break;
	}
}

FUIInputConfig::FUIInputConfig(ECommonInputMode InInputMode, EMouseCaptureMode InMouseCaptureMode, EMouseLockMode InMouseLockMode, bool bInHideCursorDuringViewportCapture)
	: InputMode(InInputMode)
	, MouseCaptureMode(InMouseCaptureMode)
	, MouseLockMode(InMouseLockMode)
	, bHideCursorDuringViewportCapture(bInHideCursorDuringViewportCapture)
{}

FString FUIInputConfig::ToString() const
{
	TStringBuilder<1024> Builder;

	Builder.Appendf(TEXT("bIgnoreMoveInput: %u\n"), bIgnoreMoveInput);
	Builder.Appendf(TEXT("bIgnoreLookInput: %u\n"), bIgnoreLookInput);
	Builder.Appendf(TEXT("InputMode: %s\n"), *UEnum::GetValueAsString(InputMode));
	Builder.Appendf(TEXT("MouseCaptureMode: %s\n"), *UEnum::GetValueAsString(MouseCaptureMode));
	Builder.Appendf(TEXT("bHideCursorDuringViewportCapture: %u\n"), bHideCursorDuringViewportCapture);

	return Builder.ToString();
}

//////////////////////////////////////////////////////////////////////////
// FActionRouterBindingCollection
//////////////////////////////////////////////////////////////////////////

FActionRouterBindingCollection::FActionRouterBindingCollection(UCommonUIActionRouterBase& OwningRouter)
	: ActionRouterPtr(&OwningRouter)
{}

EProcessHoldActionResult FActionRouterBindingCollection::ProcessHoldInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const
{
	for (FUIActionBindingHandle BindingHandle : ActionBindings)
	{
		if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(BindingHandle))
		{
			if (Binding->UserIndex == UserIndex && (ActiveInputMode == ECommonInputMode::All || ActiveInputMode == Binding->InputMode))
			{
				// Create dynamic key mappings. This also ensures that remapped keys are properly processed.
				if (CommonUI::IsEnhancedInputSupportEnabled() && bEnhancedInputDynamicallyResolveHoldKeysForInputAction && Binding->InputAction.IsValid())
				{
					Binding->HoldMappings.Empty();
					
					if (const UInputAction* InputAction = Binding->InputAction.Get())
					{
						// find hold trigger and corresponding keys for this action if a hold trigger is found
						for (const TObjectPtr<UInputTrigger>& Trigger : InputAction->Triggers)
						{
							if (const UInputTriggerHold* HoldTrigger = Cast<UInputTriggerHold>(Trigger))
							{
								if (HoldTrigger->HoldTimeThreshold <= 0.0f)
								{
									continue;
								}
							
								TArray<FKey> InputActionKeys;
								CommonUI::GetEnhancedInputActionKeys(BindingHandle.GetBoundLocalPlayer(), InputAction, InputActionKeys);
							
								for (const FKey& InputActionKey : InputActionKeys)
								{
									if (InputActionKey.IsSameResolvedKey(Key))
									{
										Binding->HoldMappings.Emplace(FUIActionKeyMapping(InputActionKey, HoldTrigger->HoldTimeThreshold, 0.f));
										break;
									}
								}
								
								break;
							}
						}
					}
				}

				for (const FUIActionKeyMapping& HoldMapping : Binding->HoldMappings)
				{
					EProcessHoldActionResult ProcessResult = EProcessHoldActionResult::Unhandled;

					check(HoldMapping.HoldTime > 0.f);

					// A persistent displayed action skips the normal rules for reachability, since it'll always appear in a bound action bar
					const bool bIsDisplayedPersistentAction = Binding->bIsPersistent && Binding->bDisplayInActionBar;
					if (HoldMapping.Key.IsSameResolvedKey(Key) && (bIsDisplayedPersistentAction || IsWidgetReachableForInput(Binding->BoundWidget.Get())))
					{
						if (InputEvent == IE_Pressed)
						{
							Binding->BeginHold();
							ProcessResult = EProcessHoldActionResult::Handled;
							Binding->OnHoldActionPressed.Broadcast();
						}
						else if (Binding->IsHoldActive())
						{
							if (InputEvent == IE_Repeat)
							{
								if (Binding->UpdateHold(HoldMapping.HoldTime))
								{
									ProcessResult = EProcessHoldActionResult::Handled;
								}
							}
							else if (InputEvent == IE_Released)
							{
								const float SecondsHeld = Binding->GetSecondsHeld();

								Binding->OnHoldActionReleased.Broadcast();

								UE_LOGF(LogUIActionRouter, VeryVerbose, "Hold released: %ls after %f", *Binding->ActionName.ToString(), SecondsHeld);

								if (HoldMapping.HoldRollbackTime <= UE_SMALL_NUMBER)
								{
									Binding->OnHoldActionProgressed.Broadcast(0.f);
									Binding->CancelHold();
								}
								else
								{
									Binding->CancelHold();
									Binding->BeginRollback(HoldMapping.HoldRollbackTime, HoldMapping.HoldTime, Binding->Handle);
								}

								if (!bAlwaysWaitForHoldActionToComplete)
								{
									static const float PressToHoldThreshold = 0.25f;
									if (SecondsHeld <= PressToHoldThreshold && SecondsHeld < HoldMapping.HoldTime)
									{
										ProcessResult = EProcessHoldActionResult::GeneratePress;
									}
								}
							}
						}

						if (Binding->bConsumesInput)
						{
							return ProcessResult;
						}
					}
				}
			}
		}
	}

	return EProcessHoldActionResult::Unhandled;
}

bool FActionRouterBindingCollection::ProcessNormalInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const
{
	for (FUIActionBindingHandle BindingHandle : ActionBindings)
	{
		if (TSharedPtr<FUIActionBinding> Binding = FUIActionBinding::FindBinding(BindingHandle))
		{
			if (Binding->UserIndex == UserIndex && (ActiveInputMode == ECommonInputMode::All || ActiveInputMode == Binding->InputMode))
			{
				auto TryConsumeInput = [&](const FKey& InKey, const UInputAction* InInputAction)
				{
					// A persistent displayed action skips the normal rules for reachability, since it'll always appear in a bound action bar
					const bool bIsDisplayedPersistentAction = Binding->bIsPersistent && Binding->bDisplayInActionBar;
					if (InKey.IsSameResolvedKey(Key) && Binding->InputEvent == InputEvent && (bIsDisplayedPersistentAction || IsWidgetReachableForInput(Binding->BoundWidget.Get())))
					{
						UE_LOGF(LogUIActionRouter, VeryVerbose, "Process normal input: %ls (key: %ls)", *Binding->ActionName.ToString(), *Key.ToString());
						
						// Just in case this was in the middle of a hold process with a different key, reset now
						Binding->CancelHold();

						// If injecting enhanced input. don't fire 'OnExecuteAction' since that can be manually done if desired in BP
						bool bEnhancedInputInjected = false;
						if (InInputAction)
						{
							if (TObjectPtr<const UCommonInputMetadata> CommonInputMetadata = CommonUI::GetEnhancedInputActionMetadata(InInputAction))
							{
								// Non generic actions should inject enhanced input so users can bind to enhanced input events
								if (!CommonInputMetadata->bIsGenericInputAction)
								{
									FInputActionValue RawValue = FInputActionValue(true);
									CommonUI::InjectEnhancedInputForAction(GetActionRouter().GetLocalPlayerChecked(), InInputAction, RawValue);
									bEnhancedInputInjected = true;
								}
							}
						}
						
						if (!bEnhancedInputInjected)
						{
							Binding->OnExecuteAction.ExecuteIfBound();
						}

						if (Binding->bConsumesInput)
						{
							return true;
						}
					}

					return false;
				};


				if (CommonUI::IsEnhancedInputSupportEnabled() && Binding->InputAction.IsValid())
				{
					if (const UInputAction* InputAction = Binding->InputAction.Get())
					{
						TArray<FKey> InputActionKeys;
						CommonUI::GetEnhancedInputActionKeys(BindingHandle.GetBoundLocalPlayer(), InputAction, InputActionKeys);
						for (const FKey& InputActionKey : InputActionKeys)
						{
							if (TryConsumeInput(InputActionKey, InputAction))
							{
								return true;
							}
						}
					}
				}

				for (const FUIActionKeyMapping& KeyMapping : Binding->NormalMappings)
				{
					if (TryConsumeInput(KeyMapping.Key, nullptr))
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

bool FActionRouterBindingCollection::IsWidgetReachableForInput(const UWidget* Widget) const
{
	if (TSharedPtr<SWidget> SlateWidget = Widget ? Widget->GetCachedWidget() : nullptr)
	{
		// Hop through the given widget's parentage to see if it's actually reachable
		while (SlateWidget)
		{
			if (!CanWidgetReceiveInput(*SlateWidget))
			{
				return false;
			}
			SlateWidget = SlateWidget->GetParentWidget();
		}
		return true;
	}

	return false;
}

void FActionRouterBindingCollection::RemoveBindings(const TArray<FUIActionBindingHandle>& WidgetBindings)
{
	for (FUIActionBindingHandle BindingToRemove : WidgetBindings)
	{
		RemoveBinding(BindingToRemove);
	}
}

void FActionRouterBindingCollection::AddBinding(FUIActionBinding& Binding)
{
	if (ensure(!ActionBindings.Contains(Binding.Handle)))
	{
		int32 IndexToInsertBinding = ActionBindings.Num();
		for (int32 Index = ActionBindings.Num() - 1; Index >= 0; Index--)
		{
			if (TSharedPtr<const FUIActionBinding> UIActionBinding = FUIActionBinding::FindBinding(ActionBindings[Index]))
			{
				if (UIActionBinding->PriorityWithinCollection < Binding.PriorityWithinCollection)
				{
					IndexToInsertBinding = Index;
					continue;
				}
			}
			break;
		}
		ActionBindings.Insert(Binding.Handle, IndexToInsertBinding);

		Binding.OwningCollection = AsShared();

		if (Binding.HoldMappings.Num() > 0)
		{
			++HoldBindingsCount;
		}

		if (IsReceivingInput())
		{
			GetActionRouter().OnBoundActionsUpdated().Broadcast();
		}
	}
}

void FActionRouterBindingCollection::RemoveBinding(FUIActionBindingHandle BindingHandle)
{
	if (ActionBindings.Remove(BindingHandle) > 0)
	{
		if (TSharedPtr<FUIActionBinding> UIBinding = FUIActionBinding::FindBinding(BindingHandle))
		{
			UIBinding->OwningCollection.Reset();

			if (UIBinding->HoldMappings.Num() > 0)
			{
				--HoldBindingsCount;
				ensure(HoldBindingsCount >= 0);
			}
		}

		if (IsReceivingInput())
		{
			GetActionRouter().OnBoundActionsUpdated().Broadcast();
		}
	}
}

int32 FActionRouterBindingCollection::GetOwnerControllerId() const
{
	return GetActionRouter().GetLocalPlayerChecked()->GetControllerId();
}

int32 FActionRouterBindingCollection::GetOwnerUserIndex() const
{
	return GetActionRouter().GetLocalPlayerIndex();
}

void FActionRouterBindingCollection::DebugDumpActionBindings(FString& OutputStr, int32 IndentSpaces) const
{
	for (FUIActionBindingHandle Handle : ActionBindings)
	{
		if (TSharedPtr<const FUIActionBinding> UIActionBinding = FUIActionBinding::FindBinding(Handle))
		{
			OutputStr.Appendf(TEXT("\n%s-%s"), FCString::Spc(IndentSpaces), *UIActionBinding->ToDebugString());
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FActivatableTreeNode
//////////////////////////////////////////////////////////////////////////

FActivatableTreeNode::FActivatableTreeNode(UCommonUIActionRouterBase& OwningRouter, UCommonActivatableWidget& ActivatableWidget)
	: FActionRouterBindingCollection(OwningRouter)
	, RepresentedWidget(&ActivatableWidget)
{
#if !UE_BUILD_SHIPPING
	DebugWidgetName = ActivatableWidget.GetName();
#endif
}

FActivatableTreeNode::FActivatableTreeNode(UCommonUIActionRouterBase& OwningRouter, UCommonActivatableWidget& ActivatableWidget, const FActivatableTreeNodeRef& InParent)
	: FActivatableTreeNode(OwningRouter, ActivatableWidget)
{
	Parent = InParent;
}

bool FActivatableTreeNode::IsWidgetReachableForInput(const UWidget* Widget) const
{
	// The widget is a child of the widget this node represents, so we only need to check that the widget is reachable 
	// by the activatable widget itself - anything beyond that is under the purview of activation. In a perfect world, 
	// the activatable tree would be granular enough such that there would be no widget relying on  pure visibility to 
	// affect its eligibility for receiving input actions. In reality, though, there are plenty of one-off cases where
	// a single button (justifiably) doesn't merit being wrapped in an activatable widget. It also maps to user expectation
	// that collapsing the widget should stop it from receiving actions.
	// So the ultimate result here is that the activatable tree is the course organization of the full widget tree, and then we do a
	// (relatively) quick validation of reachability within the activatable's personal hierarchy.
	if (TSharedPtr<SWidget> SlateWidget = Widget ? Widget->GetCachedWidget() : nullptr)
	{
		check(RepresentedWidget.IsValid());
		TSharedPtr<SWidget> RepresentedSlateWidget = RepresentedWidget->GetCachedWidget();
		if (ensure(RepresentedSlateWidget))
		{
			while (SlateWidget && SlateWidget != RepresentedSlateWidget)
			{
				if (!CanWidgetReceiveInput(*SlateWidget))
				{
					return false;
				}
				SlateWidget = SlateWidget->GetParentWidget();
			}
			return true;
		}
	}

	return false;
}

FActivatableTreeNode::~FActivatableTreeNode()
{
	if (bCanReceiveInput && ActionRouterPtr.IsValid())
	{
		// It's possible for a widget to announce deactivation and be destructed as a result before this node's deactivation handler is hit
		//	In that case, the destruction of the widget will correctly cause the immediate destruction of this node, so this node will never process the
		//	deactivation of its associated widget.
		// Therefore, if the node is being destroyed while action handlers are bound, we simply need to process the deactivation right here.
		HandleWidgetDeactivated();
	}
}

EProcessHoldActionResult FActivatableTreeNode::ProcessHoldInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const
{
	if (IsReceivingInput())
	{
		for (const FActivatableTreeNodeRef& ChildNode : Children)
		{
			EProcessHoldActionResult ChildResult = ChildNode->ProcessHoldInput(ActiveInputMode, Key, InputEvent, UserIndex);
			if (ChildResult != EProcessHoldActionResult::Unhandled)
			{
				return ChildResult;
			}
		}
		return FActionRouterBindingCollection::ProcessHoldInput(ActiveInputMode, Key, InputEvent, UserIndex);
	}
	return EProcessHoldActionResult::Unhandled;
}

bool FActivatableTreeNode::ProcessNormalInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const
{
	if (IsReceivingInput())
	{
		for (const FActivatableTreeNodeRef& ChildNode : Children)
		{
			if (ChildNode->ProcessNormalInput(ActiveInputMode, Key, InputEvent, UserIndex))
			{
				return true;
			}
		}
		return FActionRouterBindingCollection::ProcessNormalInput(ActiveInputMode, Key, InputEvent, UserIndex);
	}
	return false;
}

bool FActivatableTreeNode::IsWidgetValid() const
{ 
	return RepresentedWidget.IsValid();
}

bool FActivatableTreeNode::IsWidgetActivated() const
{
#if !UE_BUILD_SHIPPING
	UE_CLOGF(!RepresentedWidget.IsValid(), LogUIActionRouter, Warning, 
		"Represented Widget not Valid: %ls - %ls", 
		*DebugWidgetName, 
		Parent.IsValid() ? *Parent.Pin()->DebugWidgetName : TEXT("No Parent"));
#endif
	return ensure(RepresentedWidget.IsValid()) && RepresentedWidget->IsActivated();
}

bool FActivatableTreeNode::DoesWidgetSupportActivationFocus() const
{
	return ensure(RepresentedWidget.IsValid()) && RepresentedWidget->SupportsActivationFocus();
}

FActivatableTreeNodeRef FActivatableTreeNode::AddChildNode(UCommonActivatableWidget& InActivatableWidget)
{
	InActivatableWidget.OnSlateReleased().AddSP(this, &FActivatableTreeNode::HandleChildSlateReleased, &InActivatableWidget);
	
	FActivatableTreeNodeRef NewNode = MakeShareable(new FActivatableTreeNode(GetActionRouter(), InActivatableWidget, SharedThis(this)));
	NewNode->Init();
	Children.Add(NewNode);
	
	return NewNode;
}

void FActivatableTreeNode::CacheFocusRestorationTarget()
{
	TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetUserFocusedWidget(GetOwnerUserIndex());
	UCommonActivatableWidget* FocusedActivatableWidget = FocusedWidget ? UCommonUIActionRouterBase::FindActivatable(FocusedWidget, GetActionRouter().GetLocalPlayerChecked()) : nullptr;

	if (FocusedWidget != FocusRestorationTarget.Pin() && (!FocusedWidget || (FocusedActivatableWidget && IsWidgetInNodeHierarchy(FocusedActivatableWidget, *this))))
	{
		UE_LOGF(LogUIActionRouter, VeryVerbose, "FocusRestorationTarget for [%ls] changed from [%ls] to [%ls]", *GetNameSafe(GetWidget()), *FReflectionMetaData::GetWidgetDebugInfo(FocusRestorationTarget.Pin().Get()), *FReflectionMetaData::GetWidgetDebugInfo(FocusedWidget.Get()));
		FocusRestorationTarget = FocusedWidget;
	}
}

TSharedPtr<SWidget> FActivatableTreeNode::GetFocusFallbackTarget() const
{
	return FocusRestorationTarget.Pin();
}

int32 FActivatableTreeNode::GetLastPaintLayer() const
{
	if (TSharedPtr<SWidget> CachedSlate = RepresentedWidget.IsValid() ? RepresentedWidget->GetCachedWidget() : nullptr)
	{
		if (!CachedSlate->GetVisibility().IsVisible())
		{
			return INDEX_NONE;
		}
		return CachedSlate->GetPersistentState().LayerId;
	}
	return INDEX_NONE;
}

TOptional<FUIInputConfig> FActivatableTreeNode::FindDesiredInputConfig() const
{
	TOptional<FUIInputConfig> DesiredConfig = ensure(RepresentedWidget.IsValid()) ? RepresentedWidget->GetDesiredInputConfig() : TOptional<FUIInputConfig>();
	TOptional<FUIInputConfig> ActionDomainDesiredConfig = FindDesiredActionDomainInputConfig();

	if (ActionDomainDesiredConfig.IsSet() && !DesiredConfig.IsSet())
	{
		return ActionDomainDesiredConfig;
	}
	
	if (!DesiredConfig.IsSet() && Parent.IsValid())
	{
		DesiredConfig = Parent.Pin()->FindDesiredInputConfig();
	}
	
	return DesiredConfig;
}

TOptional<FUIInputConfig> FActivatableTreeNode::FindDesiredActionDomainInputConfig() const
{
	UCommonInputActionDomain* ActionDomain = ensure(RepresentedWidget.IsValid()) ? RepresentedWidget->GetCalculatedActionDomain().Get() : nullptr;
	const bool bHasActionDomainConfig = ActionDomain && ActionDomain->bUseActionDomainDesiredInputConfig;
	return  bHasActionDomainConfig ? FUIInputConfig(ActionDomain->InputMode, ActionDomain->MouseCaptureMode) : TOptional<FUIInputConfig>();
}

FActivationMetadata FActivatableTreeNode::FindActivationMetadata() const
{
	TOptional<FActivationMetadata> DesiredConfig = ensure(RepresentedWidget.IsValid()) ? RepresentedWidget->GetActivationMetadata() : TOptional<FActivationMetadata>();
	if (!DesiredConfig.IsSet() && Parent.IsValid())
	{
		DesiredConfig = Parent.Pin()->FindActivationMetadata();
	}

	if (DesiredConfig.IsSet())
	{
		return DesiredConfig.GetValue();
	}

	return FActivationMetadata();
}

void FActivatableTreeNode::AddScrollRecipient(const UWidget& ScrollRecipient)
{
	ScrollRecipients.AddUnique(&ScrollRecipient);
}

void FActivatableTreeNode::RemoveScrollRecipient(const UWidget& ScrollRecipient)
{
	ScrollRecipients.Remove(&ScrollRecipient);
}

void FActivatableTreeNode::AddInputPreprocessor(const TSharedRef<IInputProcessor>& InputPreprocessor, const FInputPreprocessorRegistrationKey& RegistrationInfo)
{
	RegisteredPreprocessors.Emplace(FInputPreprocessorRegistration{ RegistrationInfo, InputPreprocessor });
	if (IsReceivingInput())
	{
		FSlateApplication::Get().RegisterInputPreProcessor(InputPreprocessor, RegistrationInfo);
	}
}

FActivatableTreeRootRef FActivatableTreeNode::GetRoot() const
{
	if (Parent.IsValid())
	{
		return Parent.Pin()->GetRoot();
	}
	// Similar to UWorld::GetWorld() - it's a const getter that potentially returns itself, so we const cast
	return StaticCastSharedRef<FActivatableTreeRoot>(SharedThis(const_cast<FActivatableTreeNode*>(this)));
}

void FActivatableTreeNode::AppendAllActiveActions(TArray<FUIActionBindingHandle>& BoundActions) const
{
	if (IsReceivingInput())
	{
		BoundActions.Append(ActionBindings);
		
		for (const FActivatableTreeNodeRef& ChildNode : Children)
		{
			ChildNode->AppendAllActiveActions(BoundActions);
		}
	}
}

void FActivatableTreeNode::AppendValidScrollRecipients(TArray<const UWidget*>& AllScrollRecipients) const
{
	if (IsReceivingInput())
	{
		for (int32 RecipientIdx = 0; RecipientIdx < ScrollRecipients.Num(); ++RecipientIdx)
		{
			if (const UWidget* ScrollRecipient = ScrollRecipients[RecipientIdx].Get())
			{
				AllScrollRecipients.Add(ScrollRecipient);
			}
			else
			{
				ScrollRecipients.RemoveAt(RecipientIdx);
				--RecipientIdx;
			}
		}

		for (const FActivatableTreeNodeRef& ChildNode : Children)
		{
			ChildNode->AppendValidScrollRecipients(AllScrollRecipients);
		}
	}
}

void FActivatableTreeNode::DebugDumpRecursive(FString& OutputStr, int32 Depth, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const
{
	if (!bIncludeInactive && !IsWidgetActivated())
	{
		return;
	}

	static int32 SpacesPerDepthLevel = 4;
	const int32 DepthSpacing = FMath::Min(TCStringSpcHelper<TCHAR>::MAX_SPACES - 1, Depth * SpacesPerDepthLevel);

	if (RepresentedWidget.IsValid())
	{
		TSharedPtr<SWidget> CachedWidget = RepresentedWidget->GetCachedWidget();

		const int32 LayerId = CachedWidget.IsValid()
			? CachedWidget->GetPersistentState().LayerId
			: INDEX_NONE;

		OutputStr.Appendf(
			TEXT("\n%s%s: IsActivated? [%s]. LayerId=[%d]. [%d] Normal Bindings. SupportsActivationFocus [%s]"),
			FCString::Spc(DepthSpacing),
			*RepresentedWidget->GetName(),
			*LexToString(RepresentedWidget->IsActivated()),
			LayerId,
			ActionBindings.Num(),
			*LexToString(RepresentedWidget->SupportsActivationFocus())
		);
	}
	else
	{
		OutputStr.Appendf(TEXT("\n%s!!INVALID NODE!! [%d] Action Bindings."), FCString::Spc(DepthSpacing), ActionBindings.Num());
	}

	if (bIncludeActions)
	{
		DebugDumpActionBindings(OutputStr, DepthSpacing);
	}
	
	if (bIncludeChildren)
	{
		for (const FActivatableTreeNodeRef& ChildNode : Children)
		{
			ChildNode->DebugDumpRecursive(OutputStr, Depth + 1, bIncludeActions, true, bIncludeInactive);
		}
	}
}

bool FActivatableTreeNode::IsParentOfWidget(const TSharedPtr<SWidget>& SlateWidget, EIsParentSearchType ParentSearchType) const
{
	if (SlateWidget && ensure(RepresentedWidget.IsValid()))
	{
		TSharedPtr<SWidget> CachedWidget = RepresentedWidget->GetCachedWidget();
		TSharedPtr<SWidget> ParentWidget = (ParentSearchType == ExcludeSelf) ? SlateWidget->GetParentWidget() : SlateWidget;
		while (ParentWidget && ParentWidget != CachedWidget)
		{
			ParentWidget = ParentWidget->GetParentWidget();
		}
		return ParentWidget != nullptr;
	}
	return false;
}

bool FActivatableTreeNode::IsExclusiveParentOfWidget(const TSharedPtr<SWidget>& SlateWidget) const
{
	if (IsParentOfWidget(SlateWidget, ExcludeSelf))
	{
		for (const FActivatableTreeNodeRef& ChildNode : GetChildren())
		{
			if (ChildNode->DoesWidgetSupportActivationFocus() && ChildNode->IsParentOfWidget(SlateWidget, ExcludeSelf))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void FActivatableTreeNode::Init()
{
	RepresentedWidget->OnActivated().AddSP(this, &FActivatableTreeNode::HandleWidgetActivated);
	RepresentedWidget->OnDeactivated().AddSP(this, &FActivatableTreeNode::HandleWidgetDeactivated);
	if (RepresentedWidget->IsActivated())
	{
		HandleWidgetActivated();
	}
}

void FActivatableTreeNode::SetCanReceiveInput(bool bInCanReceiveInput)
{
	if (bInCanReceiveInput != bCanReceiveInput && (!bInCanReceiveInput || IsWidgetActivated()))
	{
		// Either we're setting it to false or our widget is active, which means it's ok to set to true
		bCanReceiveInput = bInCanReceiveInput;
		if (bCanReceiveInput)
		{
			RegisterPreprocessors();
		}
		else
		{
			UnregisterPreprocessors();
		}

		for (const FActivatableTreeNodePtr ChildNode : Children)
		{
			ChildNode->SetCanReceiveInput(bInCanReceiveInput);
		}
	}
}

void FActivatableTreeNode::HandleWidgetActivated()
{
	OnActivated.ExecuteIfBound();

	FActivatableTreeNodePtr ParentNode = GetParentNode();
	if (ParentNode && ParentNode->IsReceivingInput())
	{
		SetCanReceiveInput(true);
		
		if (DoesPathSupportActivationFocus())
		{
			GetRoot()->UpdateLeafmostActiveNode(SharedThis(this));
		}
		else if (RepresentedWidget.IsValid())
		{
			UCommonUIActionRouterBase::Get(*RepresentedWidget.Get())->OnBoundActionsUpdated().Broadcast();
		}
	}
}

void FActivatableTreeNode::HandleWidgetDeactivated()
{
	OnDeactivated.ExecuteIfBound();

	if (bCanReceiveInput)
	{
		SetCanReceiveInput(false);

		if (Parent.IsValid())
		{
			if (DoesPathSupportActivationFocus())
			{
				// Search for the nearest parent that's still receiving input to give it focus
				if (FActivatableTreeNodePtr NearestActiveParent = GetParentNode())
				{
					while (NearestActiveParent && !NearestActiveParent->IsReceivingInput())
					{
						NearestActiveParent = NearestActiveParent->GetParentNode();
					}

					GetActionRouter().UpdateLeafNodeAndConfig(GetRoot(), NearestActiveParent);
				}
			}
			else
			{
				GetActionRouter().OnBoundActionsUpdated().Broadcast();
			}
		}
	}
}

void FActivatableTreeNode::HandleChildSlateReleased(UCommonActivatableWidget* ChildWidget)
{
	ChildWidget->OnSlateReleased().RemoveAll(this);

	const int32 ChildToRemoveIdx = Children.IndexOfByPredicate([ChildWidget](const FActivatableTreeNodeRef& Node) { return Node->GetWidget() == ChildWidget; });
	if (ensure(ChildToRemoveIdx != INDEX_NONE))
	{
		Children.RemoveAtSwap(ChildToRemoveIdx);
	}
}

void FActivatableTreeNode::RegisterPreprocessors()
{
	check(IsReceivingInput());
	for (const FInputPreprocessorRegistration& Registration : RegisteredPreprocessors)
	{
		FSlateApplication::Get().RegisterInputPreProcessor(Registration.InputProcessor, Registration.Info);
	}
}

void FActivatableTreeNode::UnregisterPreprocessors()
{
	check(!IsReceivingInput());
	for (const FInputPreprocessorRegistration& Registration : RegisteredPreprocessors)
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(Registration.InputProcessor);
	}
}

bool FActivatableTreeNode::DoesPathSupportActivationFocus() const
{
	// True if the full path from this node back to the root supports activation focus
	bool bPathSupportsActivationFocus = DoesWidgetSupportActivationFocus();

	FActivatableTreeNodePtr ParentNode = Parent.Pin();
	while (ParentNode && bPathSupportsActivationFocus)
	{
		bPathSupportsActivationFocus = ParentNode->DoesWidgetSupportActivationFocus();
		ParentNode = ParentNode->GetParentNode();
	}
	return bPathSupportsActivationFocus;
}

//////////////////////////////////////////////////////////////////////////
// FActivatableTreeRoot
//////////////////////////////////////////////////////////////////////////

FActivatableTreeNodePtr FindLeafmostActiveChild(const FActivatableTreeNodeRef& CurNode)
{
	if (!CurNode->IsWidgetActivated() || !CurNode->DoesWidgetSupportActivationFocus())
	{
		return nullptr;
	}

	FActivatableTreeNodePtr LeafmostActiveChild;
	for (const FActivatableTreeNodeRef& ChildNode : CurNode->GetChildren())
	{
		FActivatableTreeNodePtr Candidate = FindLeafmostActiveChild(ChildNode);
		if (Candidate && (!LeafmostActiveChild || Candidate->GetLastPaintLayer() > LeafmostActiveChild->GetLastPaintLayer()))
		{
			LeafmostActiveChild = Candidate;
		}
	}

	if (LeafmostActiveChild)
	{
		return LeafmostActiveChild;
	}
	return CurNode;
}

FActivatableTreeRootRef FActivatableTreeRoot::Create(UCommonUIActionRouterBase& OwningRouter, UCommonActivatableWidget& ActivatableWidget)
{
	FActivatableTreeRootRef NewRoot = MakeShareable(new FActivatableTreeRoot(OwningRouter, ActivatableWidget));
	NewRoot->Init();
	return NewRoot;
}

void FActivatableTreeRoot::UpdateLeafNode()
{
	if (!CanReceiveInput())
	{
		if (LeafmostActiveNode.IsValid())
		{
			LeafmostActiveNode.Pin()->CacheFocusRestorationTarget();
			UpdateLeafmostActiveNode(nullptr);
		}
	}
	else if (ensure(IsWidgetActivated()))
	{
		if (!UpdateLeafmostActiveNode(SharedThis(this)))
		{
			// Our leafmost active node didn't change (good!), so we make sure apply its desired config
			// We only bother when the update doesn't do anything to avoid calling Apply twice.
			ApplyLeafmostNodeConfig();
		}
	}
}

TArray<const UWidget*> FActivatableTreeRoot::GatherScrollRecipients() const
{
	TArray<const UWidget*> AllScrollRecipients;
	AppendValidScrollRecipients(AllScrollRecipients);
	return AllScrollRecipients;
}

bool FActivatableTreeRoot::UpdateLeafmostActiveNode(FActivatableTreeNodePtr BaseCandidateNode, bool bInApplyConfig)
{
	bool bValidBaseCandidate = !BaseCandidateNode || BaseCandidateNode->IsReceivingInput();
	if (!bValidBaseCandidate)
	{
#if !UE_BUILD_SHIPPING
		UWidget* CandidateWidget = BaseCandidateNode->GetWidget();

		UE_LOGF(LogUIActionRouter, Log,
			"[FActivatableTreeRoot::UpdateLeafmostActiveNode] Node for %ls is our base candidate, but isn't receiving input. This probably means its widget was Deactivated in response to the events in its NativeOnActivated: harmless if intentional.",
			CandidateWidget ? *CandidateWidget->GetName() : TEXT(""));
#endif
		return false;
	}

	// Starting from the provided candidate, find the leafmost active node (which may well just be the candidate itself)
	FActivatableTreeNodePtr NewLeafmostNode = BaseCandidateNode ? FindLeafmostActiveChild(BaseCandidateNode.ToSharedRef()) : nullptr;

	// Always trigger on valid canidates to allow to bound actions to update
	OnLeafmostActiveNodeChanged.ExecuteIfBound();

	if (NewLeafmostNode != LeafmostActiveNode)
	{
		TSharedPtr<SWidget> OldLeafmostSlateWidget;

		if (LeafmostActiveNode.IsValid())
		{
			TSharedPtr<FActivatableTreeNode> PinnedLeafmostActiveNode = LeafmostActiveNode.Pin();
			PinnedLeafmostActiveNode->CacheFocusRestorationTarget();

			if (UCommonActivatableWidget* LeafmostActivatableWidget = PinnedLeafmostActiveNode->GetWidget())
			{
				OldLeafmostSlateWidget = LeafmostActivatableWidget->GetCachedWidget();
				LeafmostActivatableWidget->OnRequestRefreshFocus().RemoveAll(this);
			}
		}

		LeafmostActiveNode = NewLeafmostNode;
		if (bInApplyConfig)
		{
			ApplyLeafmostNodeConfig();
		}

		if (LeafmostActiveNode.IsValid())
		{
			if (UCommonActivatableWidget* NewLeafmostWidget = LeafmostActiveNode.Pin()->GetWidget())
			{
				const TSharedPtr<SWidget> NewLeafmostSlateWidget = NewLeafmostWidget->GetCachedWidget();

				NewLeafmostWidget->OnRequestRefreshFocus().AddSP(this, &FActivatableTreeRoot::HandleRequestRefreshLeafmostFocus);
				
				UE_LOGF(
					LogUIActionRouter,
					VeryVerbose,
					"Leafmost activatable widget has changed from [%ls] to [%ls]",
					OldLeafmostSlateWidget.IsValid()
					? *FReflectionMetaData::GetWidgetDebugInfo(OldLeafmostSlateWidget.Get())
					: TEXT("None"),
					NewLeafmostSlateWidget.IsValid()
					? *FReflectionMetaData::GetWidgetDebugInfo(NewLeafmostSlateWidget.Get())
					: TEXT("None")
				);
			}
		}

		return true;
	}
	return false;
}

void FActivatableTreeRoot::RefreshCachedRestorationTarget()
{
	if (TSharedPtr<FActivatableTreeNode> LeafmostPtr = LeafmostActiveNode.Pin())
	{
		LeafmostPtr->CacheFocusRestorationTarget();
	}
}

void FActivatableTreeRoot::Init()
{
	FActivatableTreeNode::Init();

	UCommonInputSubsystem& InputSubsystem = GetActionRouter().GetInputSubsystem();
	InputSubsystem.OnInputMethodChangedNative.AddSP(this, &FActivatableTreeRoot::HandleInputMethodChanged);
}

void FActivatableTreeRoot::HandleInputMethodChanged(ECommonInputType InputMethod)
{
	if (IsReceivingInput() && LeafmostActiveNode.IsValid() && ensure(LeafmostActiveNode.Pin()->IsReceivingInput()))
	{
		const bool bRetainFocus = (InputMethod == ECommonInputType::Gamepad) && CVarGamepadFocusHoveredWidget.GetValueOnAnyThread();
		
		// If focus was not retained, let leafmost node determine which widget should be focused
		ApplyLeafmostNodeConfig(bRetainFocus);

		if (InputMethod != ECommonInputType::Gamepad)
		{
			LeafmostActiveNode.Pin()->CacheFocusRestorationTarget();
		}

		if (InputMethod == ECommonInputType::Gamepad && CVarSyncCursorToFocusedWidgetOnInputMethodChange.GetValueOnAnyThread())
		{
			const int32 OwnerSlateId = GetOwnerUserIndex();

 			TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(OwnerSlateId);

			if (SlateUser)
			{
				TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetUserFocusedWidget(OwnerSlateId);

				// ApplyLeafmostNodeConfig may update the currently focused widget, but it doesn't change the cursor position.
				// It means that next time the user presses the confirmation button on the controller, the mouse click event
				// will be sent to the last touched position, and not the focused widget. It may make us lose focus and semi-lock inputs.
				// To avoid this, we update the cursor position to match the focused widget.
				if (FocusedWidget)
				{
					const FGeometry& Geometry = FocusedWidget->GetCachedGeometry();
					const FVector2D AbsoluteWidgetCenter = Geometry.GetAbsolutePositionAtCoordinates(FVector2D(0.5f, 0.5f));

					SlateUser->SetCursorPosition(AbsoluteWidgetCenter);
				}
			}
		}
	}
}

void FActivatableTreeRoot::ApplyLeafmostNodeConfig(bool bAttemptRetainFocus)
{
#if WITH_EDITOR
	if (!IsViewportWindowInFocusPath(GetActionRouter()))
	{
		return;
	}
#endif
	if (!CanSetInputConfigAndFocus())
	{
		UE_LOGF(LogUIActionRouter, Log, "Didn't apply input config for leaf-most node of root node [%ls] because it's not the active action domain node", *GetNameSafe(GetWidget()));
		return;
	}


	if (FActivatableTreeNodePtr PinnedLeafmostNode = LeafmostActiveNode.Pin())
	{
		GetActionRouter().SetActiveActivationMetadata(PinnedLeafmostNode->FindActivationMetadata());

		if (ensure(PinnedLeafmostNode->IsReceivingInput()))
		{
			UE_LOGF(LogUIActionRouter, Display, "Applying input config for leaf-most node [%ls]", *PinnedLeafmostNode->GetWidget()->GetName());

			TOptional<FUIInputConfig> DesiredConfig = PinnedLeafmostNode->FindDesiredInputConfig();
			if(DesiredConfig.IsSet())
			{
				GetActionRouter().SetActiveUIInputConfig(DesiredConfig.GetValue(), PinnedLeafmostNode->GetWidget());
			}
			else if(ICommonInputModule::GetSettings().GetEnableDefaultInputConfig())
			{
				// Nobody in the entire tree cares about the config and the default is enabled so fall back to the default
				GetActionRouter().SetActiveUIInputConfig(FUIInputConfig());
			}

			bool bFocusLeafMost = true;
			if (bAttemptRetainFocus)
			{
				// Focus if could not retain focus
				bFocusLeafMost = !GamepadFocusHoveredWidget();
			}
			
			if(bFocusLeafMost)
			{ 
				FocusLeafmostNode();
			}
		}
		else
		{
			UE_LOGF(LogUIActionRouter, Log, "Didn't apply input config for leaf-most node [%ls] because it's not receiving input right now", *PinnedLeafmostNode->GetWidget()->GetName());
		}
	}
}

bool FActivatableTreeRoot::IsAnActionDomainRoot() const
{
	for (const TPair<TObjectPtr<UCommonInputActionDomain>, UCommonUIActionRouterBase::FActionDomainSortedRootList>& Pair : GetActionRouter().ActionDomainRootNodes)
	{
		if (Pair.Value.Contains(SharedThis(const_cast<FActivatableTreeRoot*>(this))))
		{
			return true;
		}
	}

	return false;
}

bool FActivatableTreeRoot::IsActiveActionDomainRoot() const
{
	return GetActionRouter().FindActiveActionDomainRootNode().Get() == this;
}

bool FActivatableTreeRoot::CanSetInputConfigAndFocus() const
{
	return !IsAnActionDomainRoot() || IsActiveActionDomainRoot();
}

bool FActivatableTreeRoot::IsWidgetInLeafmostNodeHierarchy(const UWidget& Widget) const
{
	if (FActivatableTreeNodePtr PinnedLeaf = LeafmostActiveNode.Pin())
	{
		// If your leaf node has an SWidget-level helper, use it; otherwise adapt to UWidget
		TSharedPtr<SWidget> Cached = Widget.GetCachedWidget();
		if (Cached.IsValid())
		{
			return PinnedLeaf->IsExclusiveParentOfWidget(Cached);
		}
	}
	return false;
}

UCommonActivatableWidget* FActivatableTreeRoot::GetLeafmostActivatableWidget() const
{
	if (FActivatableTreeNodePtr PinnedLeaf = LeafmostActiveNode.Pin())
	{
		return PinnedLeaf->GetWidget();
	}

	return nullptr;
}

void FActivatableTreeRoot::FocusLeafmostNode()
{
	if (!CanSetInputConfigAndFocus())
	{
		UE_LOGF(LogUIActionRouter, Log, "%ls cannot focus its leaf most node because another node has priority", *GetNameSafe(GetWidget()));
		return;
	}

	if (!ensure(LeafmostActiveNode.IsValid()))
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot focus leaf most node - invalid LeafmostActiveNode");
		return;
	}
	FActivatableTreeNodePtr PinnedLeafmostNode = LeafmostActiveNode.Pin();

	if (!ensure(PinnedLeafmostNode))
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot focus leaf most node - invalid PinnedLeafmostNode");
		return;
	}

	UCommonActivatableWidget* LeafWidget = PinnedLeafmostNode->GetWidget();
	if (!ensure(LeafWidget))
	{
		UE_LOGF(LogUIActionRouter, Error, "Cannot focus leaf most node - invalid LeafWidget");
		return;
	}

	const int32 OwnerSlateId = GetOwnerUserIndex();
	ULocalPlayer& LocalPlayer = *GetActionRouter().GetLocalPlayerChecked();

#if WITH_EDITOR
	// In PIE, we only want to modify focus if our owner's game viewport is in the focus path (otherwise we may be stealing it from another PIE client)
	
	if (LocalPlayer.ViewportClient)
	{
		TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(OwnerSlateId);
		if (!ensure(SlateUser) || !SlateUser->IsWidgetInFocusPath(LocalPlayer.ViewportClient->GetGameViewportWidget()))
		{
			return;
		}
	}
#endif

	TSharedPtr<SWidget> AutoRestoreTarget = LeafWidget->AutoRestoresFocus() ? PinnedLeafmostNode->GetFocusFallbackTarget() : nullptr;
	UWidget* DesiredTarget = LeafWidget->GetDesiredFocusTarget();

	const bool bCheckForKeyboardAndParentage = CVarCheckKeyboardFocusAndParentage.GetValueOnAnyThread();
	if (bCheckForKeyboardAndParentage)
	{
		if (AutoRestoreTarget.IsValid())
		{
			const bool bSupportsKeyboard = AutoRestoreTarget->SupportsKeyboardFocus();
			const bool bParentOfLeaf = PinnedLeafmostNode->IsExclusiveParentOfWidget(AutoRestoreTarget);
			if (!bSupportsKeyboard || !bParentOfLeaf)
			{
				AutoRestoreTarget.Reset();
			}
		}

		if (DesiredTarget)
		{
			const bool bSupportsKeyboard = DesiredTarget->SupportsKeyboardFocus();
			const bool bParentOfLeaf = PinnedLeafmostNode->IsExclusiveParentOfWidget(DesiredTarget->GetCachedWidget());
			if (!bSupportsKeyboard || !bParentOfLeaf)
			{
				DesiredTarget = nullptr;
			}
		}
	}

	const bool bSuccessfullyAutoRestoreTarget = AutoRestoreTarget.IsValid() && FSlateApplication::Get().SetUserFocus(OwnerSlateId, AutoRestoreTarget);
	bool bShouldCancelDelayedFocusOperation = bSuccessfullyAutoRestoreTarget;
	if (bSuccessfullyAutoRestoreTarget)
	{
		UE_LOGF(LogUIActionRouter, Display, "[User %d] Focused AutoRestoreTarget %ls", OwnerSlateId, *FReflectionMetaData::GetWidgetDebugInfo(AutoRestoreTarget.Get()));	
	}
		
	const bool bFallbackToDesiredOnAutoRestoreFailure = CVarFallbackToDesiredOnAutoRestoreFailure.GetValueOnAnyThread();
	const bool bAttemptToFocusDesiredTarget = DesiredTarget && (
			(!bFallbackToDesiredOnAutoRestoreFailure && !AutoRestoreTarget.IsValid()) || // Legacy Behavior
			(bFallbackToDesiredOnAutoRestoreFailure && !bSuccessfullyAutoRestoreTarget) // New Behavior
			);

	if (bAttemptToFocusDesiredTarget)
	{
		UE_LOGF(LogUIActionRouter, Display, "[User %d] Focused desired target %ls", OwnerSlateId, *DesiredTarget->GetName());
		DesiredTarget->SetFocus();
	}
	
	if (!bSuccessfullyAutoRestoreTarget && !bAttemptToFocusDesiredTarget)
	{
		UE_LOGF(LogUIActionRouter, Display, "[User %d] Failed to Auto Restore or Focus Desired Target for %ls", GetActionRouter().GetLocalPlayerIndex(), *LeafWidget->GetName());

		// Temporary support for existing stuff that manually sets focus in Activate. If they just requested it this frame, 
		// we won't actually be updating the focus until the LocalPlayer's FReply is processed. So if we've got a new focus target already set,
		//	evaluate that. Otherwise evaluate the currently focused widget. Going forward, all widgets should be built to rely fully on GetDesiredFocusTarget().
		TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetUserFocusedWidget(OwnerSlateId);
		TSharedPtr<SWidget> PendingFocusRecipient = GetActionRouter().GetLocalPlayerChecked()->GetSlateOperations().GetUserFocusRecepient();

		const bool bIgnoreFocusedWidget = CVarDisallowUserFocusedWidgetForPendingFocusRecipient.GetValueOnAnyThread();
		const bool bCheckParentOfFocusedWidget = !PendingFocusRecipient.IsValid() || !bIgnoreFocusedWidget;

		const bool bIsParentOfPendingFocusRecipient = PinnedLeafmostNode->IsExclusiveParentOfWidget(PendingFocusRecipient);
		const bool bIsParentOfFocusedWidget = PinnedLeafmostNode->IsExclusiveParentOfWidget(FocusedWidget);

		if (bIsParentOfPendingFocusRecipient || (bCheckParentOfFocusedWidget && bIsParentOfFocusedWidget))
		{
			if (bIgnoreFocusedWidget)
			{
				UE_LOGF(LogUIActionRouter, Display, "[User %d] Leaf-most node [%ls] did not set focus through `GetDesiredFocusTarget`. Doing nothing. this widget should be updated.", GetActionRouter().GetLocalPlayerIndex(),  *LeafWidget->GetName());
			}
			else
		{
			UE_LOGF(LogUIActionRouter, Display, "[User %d] Leaf-most node [%ls] did not set focus through desired methods, but the currently focused widget is acceptable. Doing nothing, but this widget should be updated.", GetActionRouter().GetLocalPlayerIndex(),  *LeafWidget->GetName());
		}
		}
		else
		{
			if (LeafWidget->IsFocusable())
			{
				UE_LOGF(LogUIActionRouter, Display, "[User %d] No focus target for leaf-most node [%ls] - setting focus directly to the widget as a last resort.", OwnerSlateId, *LeafWidget->GetName());
				LeafWidget->SetFocus();
			}
			else if (UGameViewportClient* ViewportClient = GetActionRouter().GetLocalPlayerChecked()->ViewportClient)
			{
				UE_LOGF(LogUIActionRouter, Display, "[User %d] No focus target for leaf-most node [%ls], and the widget isn't focusable - focusing the game viewport.", OwnerSlateId, *LeafWidget->GetName());
				
				bShouldCancelDelayedFocusOperation = FSlateApplication::Get().SetUserFocus(OwnerSlateId, ViewportClient->GetGameViewportWidget());
			}
		}
	}

	if (bShouldCancelDelayedFocusOperation)
	{
		LocalPlayer.GetSlateOperations().CancelFocusRequest();
	}

}

void FActivatableTreeRoot::HandleRequestRefreshLeafmostFocus()
{
	// We only listen to requests from our leafmost node's widget,
	// and, additionally, we should only honor them while we're the active root.
	if (CanReceiveInput())
	{
		FocusLeafmostNode();
	}
}

bool FActivatableTreeRoot::GamepadFocusHoveredWidget()
{

	FSlateApplication& SlateApplication = FSlateApplication::Get();
	TSharedPtr<FActivatableTreeNode> PinnedLeafmostActiveNode = LeafmostActiveNode.Pin();

	// First attempt to regain focus by checking if any activable widgets are under the last mouse cursor position
	FWidgetPath WidgetPath = SlateApplication.LocateWindowUnderMouse(SlateApplication.GetLastCursorPos(), SlateApplication.GetTopLevelWindows());
	for (int Index = WidgetPath.Widgets.Num() - 1; Index >= 0; --Index)
	{
		TSharedRef<SWidget> WidgetUnderCursor = WidgetPath.Widgets[Index].Widget;
		UCommonActivatableWidget* ActivatableWidgetUnderCursor = UCommonUIActionRouterBase::FindActivatable(WidgetUnderCursor , GetActionRouter().GetLocalPlayerChecked());
		if (ActivatableWidgetUnderCursor != nullptr && IsWidgetInNodeHierarchy(ActivatableWidgetUnderCursor, *PinnedLeafmostActiveNode))
		{
			if (FSlateApplication::Get().SetUserFocus(GetOwnerUserIndex(), WidgetUnderCursor))
			{
				ULocalPlayer& LocalPlayer = *GetActionRouter().GetLocalPlayerChecked();
				LocalPlayer.GetSlateOperations().CancelFocusRequest();
			}
			return true;
		}
	}

	return false;
}

void FActivatableTreeRoot::DebugDump(FString& OutputStr, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const
{
	DebugDumpRecursive(OutputStr, 0, bIncludeActions, bIncludeChildren, bIncludeInactive);
}
