// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/SlateApplication.h"
#include "Input/UIActionBindingHandle.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API COMMONUI_API

enum EInputEvent : int;
enum class EProcessHoldActionResult;
struct FKey;
struct FUIActionBinding;
template <typename OptionalType> struct TOptional;

// Note: Everything in here should be considered completely private to each other and CommonUIActionRouter.
//		They were all originally defined directly in CommonUIActionRouter.cpp, but it was annoying having to scroll around so much.

class SWidget;
class UWidget;
class UCommonUIActionRouterBase;
class UCommonActivatableWidget;

class IInputProcessor;
struct FBindUIActionArgs;
enum class ECommonInputType : uint8;

class FActivatableTreeNode;
struct FCommonInputActionDataBase;
class UCommonInputSubsystem;
using FActivatableTreeNodePtr = TSharedPtr<FActivatableTreeNode>;
using FActivatableTreeNodeRef = TSharedRef<FActivatableTreeNode>;

class FActivatableTreeRoot;
using FActivatableTreeRootPtr = TSharedPtr<FActivatableTreeRoot>;
using FActivatableTreeRootRef = TSharedRef<FActivatableTreeRoot>;

class FActionRouterBindingCollection;

DECLARE_LOG_CATEGORY_EXTERN(LogUIActionRouter, Log, All);

//////////////////////////////////////////////////////////////////////////
// FActionRouterBindingCollection
//////////////////////////////////////////////////////////////////////////

class FActionRouterBindingCollection : public TSharedFromThis<FActionRouterBindingCollection>
{
public:
	virtual ~FActionRouterBindingCollection() = default;

	UE_API virtual EProcessHoldActionResult ProcessHoldInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const;
	UE_API virtual bool ProcessNormalInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const;
	virtual bool IsReceivingInput() const { return true; }

	UE_API void AddBinding(FUIActionBinding& Binding);
	
	UE_API void RemoveBindings(const TArray<FUIActionBindingHandle>& WidgetBindings);
	UE_API void RemoveBinding(FUIActionBindingHandle ActionHandle);

	bool HasHoldBindings() const { return HoldBindingsCount > 0; }

	const TArray<FUIActionBindingHandle>& GetActionBindings() const { return ActionBindings; }
	
protected:
	UE_API FActionRouterBindingCollection(UCommonUIActionRouterBase& OwningRouter);
	UE_API virtual bool IsWidgetReachableForInput(const UWidget* Widget) const;

	UE_API int32 GetOwnerUserIndex() const;
	UE_API int32 GetOwnerControllerId() const;
	UCommonUIActionRouterBase& GetActionRouter() const { check(ActionRouterPtr.IsValid()); return *ActionRouterPtr; }
	
	UE_API void DebugDumpActionBindings(FString& OutputStr, int32 IndentSpaces) const;

	/** The set of action bindings contained within this collection */
	TArray<FUIActionBindingHandle> ActionBindings;
	
	/**
	 * Treat this as guaranteed to be valid and access via GetActionRouter()
	 * Only kept as a WeakObjectPtr so we can reliably assert in the case it somehow becomes invalid.
	 */
	TWeakObjectPtr<UCommonUIActionRouterBase> ActionRouterPtr;

private:
	//Slate application sends repeat actions only for the last pressed key, so we have to keep track of this last held binding and clear it when we get a new key to hold
	mutable FUIActionBindingHandle CurrentlyHeldBinding;

	int32 HoldBindingsCount = 0;
};

//////////////////////////////////////////////////////////////////////////
// FActivatableTreeNode
//////////////////////////////////////////////////////////////////////////

class FActivatableTreeNode : public FActionRouterBindingCollection
{
public:
	UE_API virtual ~FActivatableTreeNode();
	
	UE_API virtual EProcessHoldActionResult ProcessHoldInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const override;
	UE_API virtual bool ProcessNormalInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const override;
	virtual bool IsReceivingInput() const override { return bCanReceiveInput && IsWidgetActivated(); }

	UE_API bool IsWidgetValid() const;
	UE_API bool IsWidgetActivated() const;
	UE_API bool DoesWidgetSupportActivationFocus() const;
	UE_API void AppendAllActiveActions(TArray<FUIActionBindingHandle>& BoundActions) const;

	UCommonActivatableWidget* GetWidget() { return RepresentedWidget.Get(); }
	const UCommonActivatableWidget* GetWidget() const { return RepresentedWidget.Get(); }

	TArray<FActivatableTreeNodeRef>& GetChildren() { return Children; }
	const TArray<FActivatableTreeNodeRef>& GetChildren() const { return Children; }
	
	FActivatableTreeNodePtr GetParentNode() const { return Parent.Pin(); }
	
	UE_API FActivatableTreeNodeRef AddChildNode(UCommonActivatableWidget& InActivatableWidget);
	UE_API void CacheFocusRestorationTarget();
	void ClearFocusRestorationTarget() { FocusRestorationTarget.Reset(); }
	UE_API TSharedPtr<SWidget> GetFocusFallbackTarget() const;

	UE_API bool IsExclusiveParentOfWidget(const TSharedPtr<SWidget>& SlateWidget) const;
	enum EIsParentSearchType
	{
		ExcludeSelf,
		IncludeSelf
	};
	UE_API bool IsParentOfWidget(const TSharedPtr<SWidget>& SlateWidget, EIsParentSearchType ParentSearchType) const;

	UE_API int32 GetLastPaintLayer() const;
	UE_API TOptional<FUIInputConfig> FindDesiredInputConfig() const;
	UE_API TOptional<FUIInputConfig> FindDesiredActionDomainInputConfig() const;
	UE_API FActivationMetadata FindActivationMetadata() const;
	
	UE_API void SetCanReceiveInput(bool bInCanReceiveInput);
	
	UE_API void AddScrollRecipient(const UWidget& ScrollRecipient);
	UE_API void RemoveScrollRecipient(const UWidget& ScrollRecipient);
	UE_API void AddInputPreprocessor(const TSharedRef<IInputProcessor>& InputPreprocessor, const FInputPreprocessorRegistrationKey& RegistrationInfo);

	FSimpleDelegate OnActivated;
	FSimpleDelegate OnDeactivated;

protected:
	UE_API FActivatableTreeNode(UCommonUIActionRouterBase& OwningRouter, UCommonActivatableWidget& ActivatableWidget);
	UE_API FActivatableTreeNode(UCommonUIActionRouterBase& OwningRouter, UCommonActivatableWidget& ActivatableWidget, const FActivatableTreeNodeRef& InParent);

	UE_API virtual bool IsWidgetReachableForInput(const UWidget* Widget) const override;
	
	bool CanReceiveInput() const { return bCanReceiveInput; }
	UE_API virtual void Init();	
	UE_API FActivatableTreeRootRef GetRoot() const;

	UE_API void AppendValidScrollRecipients(TArray<const UWidget*>& AllScrollRecipients) const;
	UE_API void DebugDumpRecursive(FString& OutputStr, int32 Depth, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const;
	
private:
	UE_API void HandleWidgetActivated();
	UE_API void HandleWidgetDeactivated();
	UE_API void HandleChildSlateReleased(UCommonActivatableWidget* ChildWidget);
	
	UE_API void RegisterPreprocessors();
	UE_API void UnregisterPreprocessors();

	UE_API bool DoesPathSupportActivationFocus() const;
	
#if !UE_BUILD_SHIPPING
	FString DebugWidgetName;
#endif

	TWeakObjectPtr<UCommonActivatableWidget> RepresentedWidget;
	TWeakPtr<FActivatableTreeNode> Parent;
	TArray<FActivatableTreeNodeRef> Children;
	TWeakPtr<SWidget> FocusRestorationTarget;

	bool bCanReceiveInput = false;

	TArray<FInputPreprocessorRegistration> RegisteredPreprocessors;

	// Mutable so we can keep it clean during normal use
	mutable TArray<TWeakObjectPtr<const UWidget>> ScrollRecipients;
};

//////////////////////////////////////////////////////////////////////////
// FActivatableTreeRoot
//////////////////////////////////////////////////////////////////////////

class FActivatableTreeRoot : public FActivatableTreeNode
{
public:
	static UE_API FActivatableTreeRootRef Create(UCommonUIActionRouterBase& OwningRouter, UCommonActivatableWidget& ActivatableWidget);
	
	UE_API void UpdateLeafNode();

	UE_API TArray<const UWidget*> GatherScrollRecipients() const;

	UE_API bool UpdateLeafmostActiveNode(FActivatableTreeNodePtr BaseCandidateNode, bool bInApplyConfig = true);

	UE_API void DebugDump(FString& OutputStr, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const;

	FSimpleDelegate OnLeafmostActiveNodeChanged;

	UE_API void FocusLeafmostNode();

	UE_API void RefreshCachedRestorationTarget();

	UE_API void ApplyLeafmostNodeConfig(bool bAttemptRetainFocus = false);

	UE_API bool IsAnActionDomainRoot() const;
	UE_API bool IsActiveActionDomainRoot() const;
	UE_API bool CanSetInputConfigAndFocus() const;
	UE_API bool IsWidgetInLeafmostNodeHierarchy(const UWidget& Widget) const;

	UE_API UCommonActivatableWidget* GetLeafmostActivatableWidget() const;

protected:
	UE_API virtual void Init() override;

private:
	FActivatableTreeRoot(UCommonUIActionRouterBase& OwningRouter, UCommonActivatableWidget& ActivatableWidget)
		: FActivatableTreeNode(OwningRouter, ActivatableWidget)
	{}

	void HandleInputMethodChanged(ECommonInputType InputMethod);

	void HandleRequestRefreshLeafmostFocus();

	bool GamepadFocusHoveredWidget();

	// WeakPtr because the root itself can be the primary active node - results in a circular ref leak using a full SharedPtr here
	TWeakPtr<FActivatableTreeNode> LeafmostActiveNode;
};

#undef UE_API
