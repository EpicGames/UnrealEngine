// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnalogSlider.h"
#include "Input/NavigationReply.h"
#include "Misc/App.h"
#include "Input/Reply.h"

void SAnalogSlider::Construct(const SAnalogSlider::FArguments& InDeclaration)
{
	SetCanTick(false);

	SSlider::Construct(SSlider::FArguments()
		.Style(InDeclaration._Style)
		.IsFocusable(InDeclaration._IsFocusable)
		.OnMouseCaptureBegin(InDeclaration._OnMouseCaptureBegin)
		.OnMouseCaptureEnd(InDeclaration._OnMouseCaptureEnd)
		.OnControllerCaptureBegin(InDeclaration._OnControllerCaptureBegin)
		.OnControllerCaptureEnd(InDeclaration._OnControllerCaptureEnd)
		.OnValueChanged(InDeclaration._OnValueChanged));

	OnAnalogCapture = InDeclaration._OnAnalogCapture;
}

FReply SAnalogSlider::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsUsingGamepad)
	{
		return FReply::Handled();
	}

	return SSlider::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SAnalogSlider::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!bIsUsingGamepad)
	{
		return SSlider::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	return FReply::Unhandled();
}

FReply SAnalogSlider::OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent)
{
	FReply Reply = FReply::Unhandled();
	const FKey KeyPressed = InAnalogInputEvent.GetKey();
	const float AnalogValue = InAnalogInputEvent.GetAnalogValue();
	const float StepSizeValue = StepSize.Get();
	const float AnalogStepThreshold = 0.5f;
	const float SlowestStepTime = 1.0f;
	const float TimeToSlideFromZeroToOneAtFullSpeed = 3;
	const float FastestStepTime = TimeToSlideFromZeroToOneAtFullSpeed * StepSizeValue;

	float NewValue = GetValue();
	bool bCommitNewValue = false;

	if (Orientation == EOrientation::Orient_Horizontal && KeyPressed == EKeys::Gamepad_LeftX)
	{
		Reply = FReply::Handled();
	}
	else if (Orientation == EOrientation::Orient_Vertical && KeyPressed == EKeys::Gamepad_LeftY)
	{
		Reply = FReply::Handled();
	}
	else
	{
		Reply = SSlider::OnAnalogValueChanged(MyGeometry, InAnalogInputEvent);
	}

	if (bIsUsingGamepad)
	{
		const float AbsAnalogValue = FMath::Abs(AnalogValue);
		if (AbsAnalogValue > AnalogStepThreshold)
		{
			const float NormalizedStepSpeed = FMath::GetMappedRangeValueClamped(FVector2f(AnalogStepThreshold, 1), FVector2f(0, 1), AbsAnalogValue);
			const float NormalizedEase = FMath::CubicInterp<float>(0.0f, 1.0f, 1.0f, .0f, NormalizedStepSpeed);
			const float EasedStepSpeed = FMath::Lerp(SlowestStepTime, FastestStepTime, NormalizedEase);
			const float RepeatTime = EasedStepSpeed;

			const double TimeSinceLastStep = FPlatformTime::Seconds() - LastAnalogStepTime;
			if (TimeSinceLastStep > RepeatTime)
			{
				int32 StepMultiplier = 1;
				if (RepeatTime < FApp::GetDeltaTime())
				{
					StepMultiplier = FMath::Max(1, (int32)(FApp::GetDeltaTime() / RepeatTime));
				}

				if (Orientation == EOrientation::Orient_Horizontal && KeyPressed == EKeys::Gamepad_LeftX)
				{
					if (AnalogValue < -AnalogStepThreshold)
					{
						NewValue -= StepSizeValue * StepMultiplier;
						bCommitNewValue = true;
					}
					else if (AnalogValue > AnalogStepThreshold)
					{
						NewValue += StepSizeValue * StepMultiplier;
						bCommitNewValue = true;
					}
				}
				else if (Orientation == EOrientation::Orient_Vertical && KeyPressed == EKeys::Gamepad_LeftY)
				{
					if (AnalogValue < -AnalogStepThreshold)
					{
						NewValue -= StepSizeValue * StepMultiplier;
						bCommitNewValue = true;
					}
					else if (AnalogValue > AnalogStepThreshold)
					{
						NewValue += StepSizeValue * StepMultiplier;
						bCommitNewValue = true;
					}
				}
			}
		}
	}

	if (bCommitNewValue)
	{
		CommitValue(FMath::Clamp(NewValue, MinValue, MaxValue));
		OnAnalogCapture.ExecuteIfBound(GetValue());
		LastAnalogStepTime = FPlatformTime::Seconds();
	}

	return Reply;
}

FNavigationReply SAnalogSlider::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	// @TODO: Handle up and down.
	if (Orientation == EOrientation::Orient_Horizontal)
	{
		if (InNavigationEvent.GetNavigationType() == EUINavigation::Left)
		{
			CommitValue(FMath::Clamp(GetValue() - GetStepSize(), MinValue, MaxValue));
			OnAnalogCapture.ExecuteIfBound(GetValue());

			return FNavigationReply::Explicit(nullptr);
		}
		else if (InNavigationEvent.GetNavigationType() == EUINavigation::Right)
		{
			CommitValue(FMath::Clamp(GetValue() + GetStepSize(), MinValue, MaxValue));
			OnAnalogCapture.ExecuteIfBound(GetValue());

			return FNavigationReply::Explicit(nullptr);
		}
	}
	
	return SSlider::OnNavigation(MyGeometry, InNavigationEvent);
}

void SAnalogSlider::SetUsingGamepad(bool InUsingGamepad)
{
	bIsUsingGamepad = InUsingGamepad;
}
