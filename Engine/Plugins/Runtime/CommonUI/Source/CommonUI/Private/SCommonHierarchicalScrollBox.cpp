// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCommonHierarchicalScrollBox.h"
#include "Types/NavigationMetaData.h"

void SCommonHierarchicalScrollBox::AppendFocusableChildren(TArray<TSharedRef<SWidget>>& OutChildren, TSharedRef<SWidget> Owner)
{
	FChildren* Children = Owner->GetChildren();
	for (int32 SlotIndex = 0; SlotIndex < Children->Num(); ++SlotIndex)
	{
		TSharedRef<SWidget> Child = Children->GetChildAt(SlotIndex);
		if (Child->GetVisibility().IsVisible())
		{
			if (Child->SupportsKeyboardFocus())
			{
				OutChildren.Add(Child);
			}
			else
			{
				AppendFocusableChildren(OutChildren, Child);
			}
		}
	}
}

// This is a near copy of SScrollBox::OnNavigation. Ideally AppendFocusableChildren would be a virtual function in SScrollbox, but I
//	don't have enough time to test functional changes to a core slate widget at the time of this writing. It was safer to isolate it.
//	to this widget where I can test every case where it is used.
FNavigationReply SCommonHierarchicalScrollBox::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	TSharedPtr<SWidget> FocusedChild;
	int32 FocusedChildIndex = -1;
	
	TArray<TSharedRef<SWidget>> Children;
	AppendFocusableChildren(Children, ScrollPanel.ToSharedRef());

	for (int32 SlotIndex = 0; SlotIndex < Children.Num(); ++SlotIndex)
	{
		if (Children[SlotIndex]->HasUserFocus(InNavigationEvent.GetUserIndex()).IsSet() ||
			Children[SlotIndex]->HasUserFocusedDescendants(InNavigationEvent.GetUserIndex()))
		{
			FocusedChild = Children[SlotIndex];
			FocusedChildIndex = SlotIndex;
			break;
		}
	}

	if (FocusedChild.IsValid())
	{
		int32 FocusedChildDirection = 0;
		if (Orientation == Orient_Vertical)
		{
			switch (InNavigationEvent.GetNavigationType())
			{
			case EUINavigation::Up:
				FocusedChildDirection = -1;
				break;
			case EUINavigation::Down:
				FocusedChildDirection = 1;
				break;
			}
		}
		else // Orient_Horizontal
		{
			switch (InNavigationEvent.GetNavigationType())
			{
			case EUINavigation::Left:
				FocusedChildDirection = -1;
				break;
			case EUINavigation::Right:
				FocusedChildDirection = 1;
				break;
			}
		}

		// If the focused child index is in a valid range we know we can successfully focus
		// the new child we're moving focus to.
		if (FocusedChildDirection != 0)
		{
			TSharedPtr<SWidget> NextFocusableChild;

			// If we are set to wrap, and we've reached the end of our children, manually wrap to avoid
			//	falling back to the hittest grid, which may not be what we want.
			bool bWrapped = false;
			int32 ChildIndex = FocusedChildIndex + FocusedChildDirection;
			EUINavigation Type = InNavigationEvent.GetNavigationType();
			if (ChildIndex >= Children.Num() || ChildIndex < 0)
			{
				TSharedPtr<FNavigationMetaData> NavigationMetaData = GetMetaData<FNavigationMetaData>();
				if (NavigationMetaData.IsValid() && NavigationMetaData->GetBoundaryRule(Type) == EUINavigationRule::Wrap)
				{
					ChildIndex = (ChildIndex + Children.Num()) % Children.Num();
					bWrapped = true;
				}
				
			}

			// Search in the direction we need to move for the next focusable child of the scrollbox.
			for (; ChildIndex >= 0 && ChildIndex < Children.Num(); ChildIndex += FocusedChildDirection)
			{
				TSharedRef<SWidget> PossiblyFocusableChild = Children[ChildIndex];
				if (PossiblyFocusableChild->SupportsKeyboardFocus())
				{
					bool bIsValidScrollDirection = bWrapped;

					if (!bIsValidScrollDirection)
					{
						// Make sure the child is actually in the direction of navigation relative to the currently focused child (ex. a child wrap box could have the next child in the opposite direction of navigation)
						const FVector2D FocusedChildPos = FocusedChild->GetCachedGeometry().GetAbsolutePosition();
						const FVector2D NextChildPos = PossiblyFocusableChild->GetCachedGeometry().GetAbsolutePosition();
						if (Orientation == Orient_Vertical)
						{
							switch (InNavigationEvent.GetNavigationType())
							{
							case EUINavigation::Up:
								bIsValidScrollDirection = NextChildPos.Y < FocusedChildPos.Y;
								break;
							case EUINavigation::Down:
								bIsValidScrollDirection = NextChildPos.Y > FocusedChildPos.Y;;
								break;
							}
						}
						else // Orient_Horizontal
						{
							switch (InNavigationEvent.GetNavigationType())
							{
							case EUINavigation::Left:
								bIsValidScrollDirection = NextChildPos.X < FocusedChildPos.X;
								break;
							case EUINavigation::Right:
								bIsValidScrollDirection = NextChildPos.X > FocusedChildPos.X;
								break;
							}
						}
					}

					if (bIsValidScrollDirection)
					{
						NextFocusableChild = PossiblyFocusableChild;
						break;
					}
				}
			}

			// If we found a focusable child, scroll to it, and shift focus.
			if (NextFocusableChild.IsValid())
			{
				ScrollDescendantIntoView(NextFocusableChild, true, NavigationDestination, NavigationScrollPadding);
				return FNavigationReply::Explicit(NextFocusableChild);
			}
		}
	}

	return SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);
}
