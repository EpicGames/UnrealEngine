// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVisualAttachmentBox.h"

void SVisualAttachmentBox::Construct(const FArguments& InArgs)
{
	SBox::Construct(InArgs);
}

void SVisualAttachmentBox::SetContentAnchor(FVector2D InContentAnchor)
{
	if (InContentAnchor != ContentAnchor)
	{
		ContentAnchor = InContentAnchor;
		Invalidate(EInvalidateWidget::Layout);
	}
}

void SVisualAttachmentBox::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const FGeometry NewGeometry = AllottedGeometry.MakeChild(InnerDesiredSize, FSlateLayoutTransform(-(ContentAnchor * InnerDesiredSize)));
	SBox::OnArrangeChildren(NewGeometry, ArrangedChildren);
}

FVector2D SVisualAttachmentBox::ComputeDesiredSize(float InScale) const
{
	// We don't take up any space.
	InnerDesiredSize = SBox::ComputeDesiredSize(InScale);
	return FVector2D(0, 0);
}

int32 SVisualAttachmentBox::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SBox::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return LayerId;
}
