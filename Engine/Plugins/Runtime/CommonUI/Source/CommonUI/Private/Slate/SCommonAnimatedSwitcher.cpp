// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SCommonAnimatedSwitcher.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCommonAnimatedSwitcher)

void SCommonAnimatedSwitcher::Construct(const FArguments& InArgs)
{
	SWidgetSwitcher::Construct(SWidgetSwitcher::FArguments().WidgetIndex(InArgs._InitialIndex));
		
	SetCanTick(false);
	bTransitioningOut = false;
	PendingActiveWidget = nullptr;
	PendingActiveWidgetIndex = INDEX_NONE;

	TransitionType = InArgs._TransitionType;
	TransitionFallbackStrategy = InArgs._TransitionFallbackStrategy;

	SetTransition(InArgs._TransitionDuration, InArgs._TransitionCurveType);

	OnActiveIndexChanged = InArgs._OnActiveIndexChanged;
	OnIsTransitioningChanged = InArgs._OnIsTransitioningChanged;
}

int32 SCommonAnimatedSwitcher::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FWidgetStyle CompoundedWidgetStyle = FWidgetStyle(InWidgetStyle);
		
	// Set the alpha during transitions
	if (TransitionSequence.IsPlaying())
	{
		float Alpha = TransitionSequence.GetLerp();

		if ((bTransitioningOut && !TransitionSequence.IsInReverse()) ||
			(!bTransitioningOut && TransitionSequence.IsInReverse()))
		{
			Alpha = 1 - Alpha;
		}
			
		CompoundedWidgetStyle.BlendColorAndOpacityTint(FLinearColor(1.f, 1.f, 1.f, Alpha));
	}
		
	return SWidgetSwitcher::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, CompoundedWidgetStyle, bParentEnabled);
}

void SCommonAnimatedSwitcher::TransitionToIndex(int32 NewWidgetIndex, bool bInstantTransition)
{
	// Cache the widget we want to reach
	PendingActiveWidget = GetWidget(NewWidgetIndex);
	if (PendingActiveWidget.IsValid())
	{
		PendingActiveWidgetIndex = NewWidgetIndex;
	}
	else
	{
		PendingActiveWidgetIndex = INDEX_NONE;
		UE_LOGF(LogSlate, Verbose, "Called SCommonAnimatedSwitcher::TransitionToIndex('%d') to an invalid Index", NewWidgetIndex);
		return;
	}

	const int32 CurrentIndex = GetActiveWidgetIndex();
	const bool bNewGoalHigher = NewWidgetIndex > CurrentIndex;
	const bool bNewGoalLower = NewWidgetIndex < CurrentIndex;

	if (bInstantTransition || TransitionSequence.GetCurve(0).DurationSeconds <= 0.f)
	{
		// Don't bother doing anything if we're already at the desired index and aren't mid-transition
		if (TransitionSequence.IsPlaying() || bNewGoalHigher || bNewGoalLower)
		{
			// Snap instantly to the target index
			TransitionSequence.JumpToEnd();
			SetActiveWidgetIndex(NewWidgetIndex);

			// Technically this may not be true here, but worst-case we were transitioning away from the current index and snapped back - worth announcing
			// If you're debugging and find that's an issue for you, get in touch with me (DanH) with your scenario. We can tweak this if a need presents itself.
			OnActiveIndexChanged.ExecuteIfBound(NewWidgetIndex);
		}
	}
	else if (TransitionSequence.IsPlaying())
	{
		// Already a transition in progress - see if we need to reverse it
		const bool bNeedsReverse = (TransitionSequence.IsInReverse() && bNewGoalHigher) ||		// Currently headed to a lower index, now need to go to a higher one
									(!TransitionSequence.IsInReverse() && bNewGoalLower) ||		// Currently headed to a higher index, now need to go to a lower one
									(bTransitioningOut && NewWidgetIndex == CurrentIndex);			// Return to the index we're just now leaving
		if (bNeedsReverse)
		{
			bTransitioningOut = !bTransitioningOut;
			TransitionSequence.Reverse();

			if (NewWidgetIndex == CurrentIndex)
			{
				// Similar to above, this isn't technically true in that the true underlying active index hasn't changed, but
				//	but hitting this does mean that the requested transition target has changed. From a user perspective, this is
				//	still a scenario where one would expect to hear that the active index has changed.
				OnActiveIndexChanged.ExecuteIfBound(NewWidgetIndex);
			}
		}
	}
	else if (bNewGoalHigher || bNewGoalLower)
	{
		SetVisibility(EVisibility::HitTestInvisible);
		OnIsTransitioningChanged.ExecuteIfBound(true);

		if (bNewGoalHigher)
		{
			TransitionSequence.Play(AsShared());
		}
		else
		{
			TransitionSequence.PlayReverse(AsShared());
		}

		// If ever we're transitioning away from an empty/placeholder child index, we don't want to bother playing the outro on the current index.
		// After all, what's the point if there's nothing there? It just looks like an erroneous delay.
		TSharedPtr<SWidget> CurrentWidget = GetActiveWidget();
		bTransitioningOut = CurrentWidget && CurrentWidget != SNullWidget::NullWidget;

		if (!bTransitioningOut)
		{
			// Nothing worth animating out, so skip straight to the next index
			SetActiveWidgetIndex(NewWidgetIndex);
			OnActiveIndexChanged.ExecuteIfBound(NewWidgetIndex);
		}

		// We always want to register our timer to update the transition, even if we've skipped the outro - we still have an intro to play!
		if (!bIsTransitionTimerRegistered)
		{
			bIsTransitionTimerRegistered = true;
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SCommonAnimatedSwitcher::UpdateTransition));
		}
	}
}

SWidgetSwitcher::FSlot* SCommonAnimatedSwitcher::GetChildSlot(int32 SlotIndex)
{
	return GetTypedChildren().IsValidIndex(SlotIndex) ? &GetTypedChildren()[SlotIndex] : nullptr;
}

void SCommonAnimatedSwitcher::SetTransition(float Duration, ETransitionCurve Curve)
{
	TransitionSequence = FCurveSequence(0.f, Duration * 0.5f, TransitionCurveToCurveEaseFunction(Curve));
}

void SCommonAnimatedSwitcher::SetTransition(float Duration, ETransitionCurve Curve, ECommonSwitcherTransition NewTransitionType)
{
	SetTransition(Duration, Curve);
	SetTransitionType(NewTransitionType);
}

void SCommonAnimatedSwitcher::SetTransitionType(ECommonSwitcherTransition NewTransitionType)
{
	TransitionType = NewTransitionType;
}

void SCommonAnimatedSwitcher::OnSlotAdded(int32 AddedIndex)
{
	SWidgetSwitcher::OnSlotAdded(AddedIndex);
	
	if (PendingActiveWidgetIndex >= 0 && PendingActiveWidgetIndex >= AddedIndex)
	{
		// A slot has been added before our PendingActiveWidgetIndex
		++PendingActiveWidgetIndex;
	}
}

void SCommonAnimatedSwitcher::OnSlotRemoved(int32 RemovedIndex, TSharedRef<SWidget> RemovedWidget, bool bWasActiveSlot)
{
	SWidgetSwitcher::OnSlotRemoved(RemovedIndex, RemovedWidget, bWasActiveSlot);

	if (PendingActiveWidgetIndex >= 0)
	{
		if (PendingActiveWidgetIndex == RemovedIndex)
		{
			// Our PendingActiveWidget has been removed, attempt a fallback if enabled
			TryTransitionFallbackOfPendingWidget();
		}
		else if (PendingActiveWidgetIndex < RemovedIndex)
		{
			// A slot has been removed before our PendingActiveWidget
			--PendingActiveWidgetIndex;
		}
	}
	
	if (bWasActiveSlot && TransitionSequence.IsPlaying())
	{
		// The widget being animated has been removed. Instantly finish the current transition animation and allow UpdateTransition() to finish the process.
		if (TransitionSequence.IsForward())
		{
			TransitionSequence.JumpToEnd();
		}
		else
		{
			TransitionSequence.JumpToStart();
		}

		if (!bTransitioningOut)
		{
			// Since the removed widget was active and the target of the current transition, set the fallback widget active.
			TryTransitionFallbackOfActiveWidget(RemovedIndex);
		}
	}
}

EActiveTimerReturnType SCommonAnimatedSwitcher::UpdateTransition(double InCurrentTime, float InDeltaTime)
{
	if (TransitionType == ECommonSwitcherTransition::Zoom)
	{
		static const float MaxScaleModifier = 0.25f;
			
		SetRenderTransform(FSlateRenderTransform(1 + (MaxScaleModifier * GetTransitionProgress())));
	}
	else if (TransitionType != ECommonSwitcherTransition::FadeOnly)
	{
		static const float MaxTranslation = 200.f;

		const float Offset = MaxTranslation * GetTransitionProgress();
			
		FVector2D Translation = FVector2D::ZeroVector;
		if (TransitionType == ECommonSwitcherTransition::Horizontal)
		{
			Translation.X = -Offset;
		}
		else
		{
			Translation.Y = Offset;
		}

		SetRenderTransform(Translation);
	}

	if (!TransitionSequence.IsPlaying())
	{
		// The transition is complete in one direction
		TSharedPtr<SWidget> PinnedPendingActiveWidget = PendingActiveWidget.Pin();
		if (bTransitioningOut && (!PinnedPendingActiveWidget.IsValid() || GetWidgetIndex(PinnedPendingActiveWidget.ToSharedRef()) == INDEX_NONE))
		{
			// The PendingActiveWidget is no longer contained in this switcher
			UE_LOGF(LogSlate, Verbose, "SCommonAnimatedSwitcher Pending Widget was removed or became invalid while a transition was happening");
			if (TryTransitionFallbackOfPendingWidget())
			{
				// We have fallen back to an alternative PendingActiveWidget
				PinnedPendingActiveWidget = PendingActiveWidget.Pin();
			}
		}
		
		if (PinnedPendingActiveWidget.IsValid() && GetActiveWidget().Get() != PinnedPendingActiveWidget.Get())
		{
			const bool bWasTransitioningOut = bTransitioningOut;
			bTransitioningOut = !bTransitioningOut;

			if (bWasTransitioningOut)
			{
				// Finished transitioning out - update the active widget and play again from the start
				SetActiveWidget(PinnedPendingActiveWidget.ToSharedRef());
				PendingActiveWidget = nullptr;
				PendingActiveWidgetIndex = INDEX_NONE;

				//Note that any listener could decide to trigger ANOTHER transition, 
				//changing the values of PendingActiveWidget and bTransitioningOut to something new.
				OnActiveIndexChanged.ExecuteIfBound(GetActiveWidgetIndex());
			}

			//If we are introing into or outroing from an invalid or null widget then there is no need to animate that transition.
			TSharedPtr<SWidget> CurrentWidget = GetActiveWidget();
			if (CurrentWidget && CurrentWidget != SNullWidget::NullWidget)
			{
				if (TransitionSequence.IsInReverse())
				{
					TransitionSequence.PlayReverse(AsShared());
				}
				else
				{
					TransitionSequence.Play(AsShared());
				}
			}
		}
		else
		{
			PendingActiveWidget = nullptr;
			PendingActiveWidgetIndex = INDEX_NONE;
		}
	}

	// If the sequence still isn't playing, the transition is complete
	if (!TransitionSequence.IsPlaying())
	{
		SetVisibility(EVisibility::SelfHitTestInvisible);
		OnIsTransitioningChanged.ExecuteIfBound(false);
		bIsTransitionTimerRegistered = false;

		// TODO - UE-256488 - There's a bug with SlateGI where setting the visibility of this widget in the same frame as becoming non-volatile (by stopping our timer) will cause this widget to disappear for that frame, so we stay volatile for 1 more frame
		ForceVolatile(true);
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float DeltaTime)
		{
			ForceVolatile(false);
			return false;
		}));

		return EActiveTimerReturnType::Stop;
	}
	else
	{
		// if the sequence is playing update we need to invalidate painting so our alpha is recomputed
		Invalidate(EInvalidateWidget::Paint);
	}
		
	return EActiveTimerReturnType::Continue;
}

float SCommonAnimatedSwitcher::GetTransitionProgress() const
{ 
	float Progress = TransitionSequence.GetLerp();

	if ((bTransitioningOut && TransitionSequence.IsInReverse()) ||
		(!bTransitioningOut && TransitionSequence.IsForward()))
	{
		Progress += -1.f;
	}
		
	return Progress;
}

int32 SCommonAnimatedSwitcher::GetTransitionFallbackForIndex(int32 RemovedWidgetIndex) const
{
	int32 DesiredIndex;
	switch (TransitionFallbackStrategy)
	{
	case ECommonSwitcherTransitionFallbackStrategy::First:
		// Falling back to the index of the first item
		DesiredIndex = 0;
		break;
	case ECommonSwitcherTransitionFallbackStrategy::Last:
		// Falling back to the index of the last item
		DesiredIndex = GetNumWidgets() - 1;
		break;
	case ECommonSwitcherTransitionFallbackStrategy::Next:
		// Falling back to the index of the next item, which is now the same as RemovedWidgetIndex because it shifted after the removal.
		DesiredIndex = RemovedWidgetIndex;
		break;
	case ECommonSwitcherTransitionFallbackStrategy::Previous:
	default:
		// Falling back to the index of the previous item
		DesiredIndex = RemovedWidgetIndex - 1;
		break;
	}

	// Constraining the index to the valid range
	return FMath::Clamp(DesiredIndex, 0, GetNumWidgets() - 1);
}

bool SCommonAnimatedSwitcher::TryTransitionFallbackOfPendingWidget()
{
	if (IsTransitionFallbackEnabled() && GetNumWidgets() > 0)
	{
		// Falling back to another transition target
		const int32 NewPendingWidgetIndex = GetTransitionFallbackForIndex(PendingActiveWidgetIndex);
		PendingActiveWidget = GetWidget(NewPendingWidgetIndex);
		if (PendingActiveWidget.IsValid())
		{
			PendingActiveWidgetIndex = NewPendingWidgetIndex;
			return true;
		}
		else
		{
			PendingActiveWidgetIndex = INDEX_NONE;
			UE_LOGF(LogSlate, Verbose, "SCommonAnimatedSwitcher failed to fall back to another Pending Widget for the transition");
		}
	}
	
	return false;
}

bool SCommonAnimatedSwitcher::TryTransitionFallbackOfActiveWidget(int32 RemovedWidgetIndex)
{
	if (IsTransitionFallbackEnabled() && GetNumWidgets() > 0)
	{
		// Falling back to another active widget
		const int32 NewActiveWidgetIndex = GetTransitionFallbackForIndex(RemovedWidgetIndex);
		SetActiveWidgetIndex(NewActiveWidgetIndex);
		
		//Note that any listener could decide to trigger ANOTHER transition, 
		//changing the values of PendingActiveWidget and bTransitioningOut to something new.
		OnActiveIndexChanged.ExecuteIfBound(GetActiveWidgetIndex());
		return true;
	}
	
	return false;
}

bool SCommonAnimatedSwitcher::IsTransitionPlaying() const
{
	return TransitionSequence.IsPlaying();
}
