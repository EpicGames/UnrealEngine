// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/CommonActivatableWidgetContainer.h"
#include "CommonActivatableWidget.h"
#include "CommonUIPrivate.h"
#include "CommonWidgetPaletteCategories.h"
#include "Slate/SCommonAnimatedSwitcher.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SSpacer.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonActivatableWidgetContainer)

UCommonActivatableWidget* ActivatableWidgetFromSlate(const TSharedPtr<SWidget>& SlateWidget)
{
	if (SlateWidget && SlateWidget != SNullWidget::NullWidget && SlateWidget->GetMetaData<FCommonActivatableSlateMetaData>().IsValid())
	{
		UCommonActivatableWidget* ActivatableWidget = Cast<UCommonActivatableWidget>(StaticCastSharedPtr<SObjectWidget>(SlateWidget)->GetWidgetObject());
		if (ensure(ActivatableWidget))
		{
			return ActivatableWidget;
		}
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////
// UCommonActivatableWidgetContainerBase
//////////////////////////////////////////////////////////////////////////

UCommonActivatableWidgetContainerBase::UCommonActivatableWidgetContainerBase(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, GeneratedWidgetsPool(*this)
{
	SetVisibilityInternal(ESlateVisibility::Collapsed);
}

void UCommonActivatableWidgetContainerBase::AddWidgetInstance(UCommonActivatableWidget& ActivatableWidget)
{
	RegisterInstanceInternal(ActivatableWidget);
}

UCommonActivatableWidget* UCommonActivatableWidgetContainerBase::GetActiveWidget() const
{
	return MySwitcher ? ActivatableWidgetFromSlate(MySwitcher->GetActiveWidget()) : nullptr;
}

int32 UCommonActivatableWidgetContainerBase::GetNumWidgets() const
{
	return WidgetList.Num();
}

void UCommonActivatableWidgetContainerBase::RemoveWidget(UCommonActivatableWidget* WidgetToRemove)
{
	if (WidgetToRemove)
	{
		RemoveWidget(*WidgetToRemove);
	}
}

void UCommonActivatableWidgetContainerBase::RemoveWidget(UCommonActivatableWidget& WidgetToRemove)
{
	UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase::RemoveWidget() WidgetToRemove: %ls", *WidgetToRemove.GetName());
	
	if (&WidgetToRemove == GetActiveWidget())
	{
		// To remove the active widget, just deactivate it (if it's already deactivated, then we're already in the process of ditching it)
		if (WidgetToRemove.IsActivated())
		{
			WidgetToRemove.DeactivateWidget();
		}
		else
		{
			bRemoveDisplayedWidgetPostTransition = true;
		}
	}
	else
	{
		// Otherwise if the widget isn't actually being shown right now, yank it right on out
		TSharedPtr<SWidget> CachedWidget = WidgetToRemove.GetCachedWidget();
		if (CachedWidget && MySwitcher)
		{
			ReleaseWidget(CachedWidget.ToSharedRef());
		}
		else
		{
			GeneratedWidgetsPool.Release(&WidgetToRemove, true);
			WidgetList.Remove(&WidgetToRemove);
		}
	}
}

void UCommonActivatableWidgetContainerBase::ClearWidgets()
{
	SetSwitcherIndex(0);
}

TSharedRef<SWidget> UCommonActivatableWidgetContainerBase::RebuildWidget()
{
	MyOverlay = SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SAssignNew(MySwitcher, SCommonAnimatedSwitcher)
			.TransitionCurveType(TransitionCurveType)
			.TransitionDuration(TransitionDuration)
			.TransitionType(TransitionType)
			.TransitionFallbackStrategy(TransitionFallbackStrategy)
			.OnActiveIndexChanged_UObject(this, &UCommonActivatableWidgetContainerBase::HandleActiveIndexChanged)
			.OnIsTransitioningChanged_UObject(this, &UCommonActivatableWidgetContainerBase::HandleSwitcherIsTransitioningChanged)
		]
		+ SOverlay::Slot()
		[
			SAssignNew(MyInputGuard, SSpacer)
			.Visibility(EVisibility::Collapsed)
		];

	// We always want a 0th slot to be able to animate the first real entry in and out
	MySwitcher->AddSlot() [SNullWidget::NullWidget];

	return MyOverlay.ToSharedRef();
}

void UCommonActivatableWidgetContainerBase::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyOverlay.Reset();
	MyInputGuard.Reset();
	MySwitcher.Reset();
	ReleasedWidgets.Empty();
	WidgetList.Reset();

	if (bResetPoolWhenReleasingSlateResources)
	{
		GeneratedWidgetsPool.ResetPool();
	}
	else
	{
		GeneratedWidgetsPool.ReleaseAll(true);
	}
}

void UCommonActivatableWidgetContainerBase::OnWidgetRebuilt()
{
	Super::OnWidgetRebuilt();
	
	if (!IsDesignTime())
	{
		// When initially created, fake that we just did an initial transition to index 0
		HandleActiveIndexChanged(0);
	}
}

void UCommonActivatableWidgetContainerBase::SetSwitcherIndex(int32 TargetIndex, bool bInstantTransition /*= false*/)
{
	UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase::SetSwitcherIndex() TargetIndex: %d, bInstantTransition: %d", TargetIndex, bInstantTransition);
	
	if (MySwitcher && MySwitcher->GetActiveWidgetIndex() != TargetIndex)
	{
		if (DisplayedWidget)
		{
			DisplayedWidget->OnDeactivated().RemoveAll(this);
			if (DisplayedWidget->IsActivated())
			{
				DisplayedWidget->DeactivateWidget();
			}
			else if (MySwitcher->GetActiveWidgetIndex() != 0)
			{
				// The displayed widget has already been deactivated by something other than us, so it should be removed from the container
				// We still need it to remain briefly though until we transition to the new index - then we can remove this entry's slot
				bRemoveDisplayedWidgetPostTransition = true;
			}
		}

		MySwitcher->TransitionToIndex(TargetIndex, bInstantTransition);
	}
}

#if WITH_EDITOR
const FText UCommonActivatableWidgetContainerBase::GetPaletteCategory()
{
	return CommonWidgetPaletteCategories::Default;
}
#endif // WITH_EDITOR

UCommonActivatableWidget* UCommonActivatableWidgetContainerBase::BP_AddWidget(TSubclassOf<UCommonActivatableWidget> ActivatableWidgetClass)
{
	return AddWidgetInternal(ActivatableWidgetClass, [](UCommonActivatableWidget&) {});
}

UCommonActivatableWidget* UCommonActivatableWidgetContainerBase::AddWidgetInternal(TSubclassOf<UCommonActivatableWidget> ActivatableWidgetClass, TFunctionRef<void(UCommonActivatableWidget&)> InitFunc)
{
	if (UCommonActivatableWidget* WidgetInstance = GeneratedWidgetsPool.GetOrCreateInstance(ActivatableWidgetClass))
	{
		InitFunc(*WidgetInstance);
		RegisterInstanceInternal(*WidgetInstance);
		return WidgetInstance;
	}
	return nullptr;
}

void UCommonActivatableWidgetContainerBase::RegisterInstanceInternal(UCommonActivatableWidget& NewWidget)
{
	UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase::RegisterInstanceInternal() NewWidget: %ls", *NewWidget.GetName());
	
	// @TODO: Log if bAutoActivate is true on the provided widget, since it quite simply makes no sense.
	if (ensure(!WidgetList.Contains(&NewWidget)))
	{
		WidgetList.Add(&NewWidget);
		OnWidgetAddedToList(NewWidget);
	}
}

void UCommonActivatableWidgetContainerBase::HandleSwitcherIsTransitioningChanged(bool bIsTransitioning)
{
	UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase::HandleSwitcherIsTransitioningChanged() bIsTransitioning: %d", bIsTransitioning);
	
	// While the switcher is transitioning, put up the guard to intercept all input
	MyInputGuard->SetVisibility(bIsTransitioning ? EVisibility::Visible : EVisibility::Collapsed);
	OnTransitioningChanged.Broadcast(this, bIsTransitioning);
}

void UCommonActivatableWidgetContainerBase::HandleActiveWidgetDeactivated(UCommonActivatableWidget* DeactivatedWidget)
{
	UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase::HandleActiveWidgetDeactivated() DeactivatedWidget: %ls", *DeactivatedWidget->GetName());
	
	// When the currently displayed widget deactivates, transition the switcher to the preceding slot (if it exists)
	// We'll clean up this slot once the switcher index actually changes
	if (ensure(DeactivatedWidget == DisplayedWidget) && MySwitcher && MySwitcher->GetActiveWidgetIndex() > 0)
	{
		DisplayedWidget->OnDeactivated().RemoveAll(this);
		MySwitcher->TransitionToIndex(MySwitcher->GetActiveWidgetIndex() - 1);
	}
}

void UCommonActivatableWidgetContainerBase::ReleaseWidget(const TSharedRef<SWidget>& WidgetToRelease)
{
	UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase::ReleaseWidget()");
	
	if (UCommonActivatableWidget* ActivatableWidget = ActivatableWidgetFromSlate(WidgetToRelease))
	{
		UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase WidgetToRelease: %ls", *ActivatableWidget->GetName());
		
		GeneratedWidgetsPool.Release(ActivatableWidget, true);
		WidgetList.Remove(ActivatableWidget);
	}
	else
	{
		UE_LOGF(LogCommonUI, Warning, "UCommonActivatableWidgetContainerBase::ReleaseWidget() No matching Activatable Widget found.");
	}

	const int32 RemovedIndex = MySwitcher->RemoveSlot(WidgetToRelease);
	if (RemovedIndex != INDEX_NONE)
	{
		UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase Widget removed from slot %d", RemovedIndex);
		
		ReleasedWidgets.Add(WidgetToRelease);
		if (ReleasedWidgets.Num() == 1)
		{
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this,
				[this](float)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_UCommonActivatableWidgetContainerBase_ReleaseWidget);
					ReleasedWidgets.Reset();
					return false;
				}));
		}
	}
}

void UCommonActivatableWidgetContainerBase::HandleActiveIndexChanged(int32 ActiveWidgetIndex)
{
	UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetContainerBase::HandleActiveIndexChanged() ActiveWidgetIndex: %d", ActiveWidgetIndex);
	
	// Remove all slots above the currently active one and release the widgets back to the pool
	while (MySwitcher->GetNumWidgets() - 1 > ActiveWidgetIndex)
	{
		TSharedPtr<SWidget> WidgetToRelease = MySwitcher->GetWidget(MySwitcher->GetNumWidgets() - 1);
		if (ensure(WidgetToRelease))
		{
			ReleaseWidget(WidgetToRelease.ToSharedRef());
		}
	}

	// Also remove the widget that we just transitioned away from if desired
	if (DisplayedWidget && bRemoveDisplayedWidgetPostTransition)
	{
		if (TSharedPtr<SWidget> DisplayedSlateWidget = DisplayedWidget->GetCachedWidget())
		{
			ReleaseWidget(DisplayedSlateWidget.ToSharedRef());
		}
	}
	bRemoveDisplayedWidgetPostTransition = false;

	// Activate the widget that's now being displayed
	DisplayedWidget = ActivatableWidgetFromSlate(MySwitcher->GetActiveWidget());
	if (DisplayedWidget)
	{
		SetVisibility(ESlateVisibility::SelfHitTestInvisible);

		DisplayedWidget->OnDeactivated().AddUObject(this, &UCommonActivatableWidgetContainerBase::HandleActiveWidgetDeactivated, ToRawPtr(DisplayedWidget));
		DisplayedWidget->ActivateWidget();

		if (UWorld* MyWorld = GetWorld())
		{
			FTimerManager& TimerManager = MyWorld->GetTimerManager();
			TimerManager.SetTimerForNextTick(FTimerDelegate::CreateWeakLambda(this, [this]() { InvalidateLayoutAndVolatility(); }));
		}
	}
	else
	{
		SetVisibility(ESlateVisibility::Collapsed);
	}

	OnDisplayedWidgetChanged().Broadcast(DisplayedWidget);
}

void UCommonActivatableWidgetContainerBase::SetTransitionDuration(float Duration)
{
	TransitionDuration = Duration;
	if (MySwitcher.IsValid())
	{
		MySwitcher->SetTransition(TransitionDuration, TransitionCurveType);
	}
}

float UCommonActivatableWidgetContainerBase::GetTransitionDuration() const
{
	return TransitionDuration;
}

//////////////////////////////////////////////////////////////////////////
// UCommonActivatableWidgetStack
//////////////////////////////////////////////////////////////////////////

UCommonActivatableWidget* UCommonActivatableWidgetStack::GetRootContent() const
{
	return RootContentWidget;
}

void UCommonActivatableWidgetStack::SynchronizeProperties()
{
	Super::SynchronizeProperties();
	
#if WITH_EDITOR
	if (IsDesignTime() && RootContentWidget && RootContentWidget->GetClass() != RootContentWidgetClass)
	{
		// At design time, account for the possibility of the preview class changing
		if (RootContentWidget->GetCachedWidget())
		{
			MySwitcher->GetChildSlot(0)->DetachWidget();
		}

		RootContentWidget = nullptr;
	}
#endif

	if (!RootContentWidget && RootContentWidgetClass)
	{
		// Establish the root content as the blank 0th slot content
		RootContentWidget = CreateWidget<UCommonActivatableWidget>(this, RootContentWidgetClass);
		MySwitcher->GetChildSlot(0)->AttachWidget(RootContentWidget->TakeWidget());
		SetVisibility(ESlateVisibility::SelfHitTestInvisible);
	}
}

void UCommonActivatableWidgetStack::OnWidgetAddedToList(UCommonActivatableWidget& AddedWidget)
{
	if (MySwitcher)
	{
		// Toss the widget onto the end of the switcher's children and transition to it immediately
		MySwitcher->AddSlot() [AddedWidget.TakeWidget()];

		SetSwitcherIndex(MySwitcher->GetNumWidgets() - 1);
	}
}

//////////////////////////////////////////////////////////////////////////
// UCommonActivatableWidgetQueue
//////////////////////////////////////////////////////////////////////////

void UCommonActivatableWidgetQueue::OnWidgetAddedToList(UCommonActivatableWidget& AddedWidget)
{
	if (MySwitcher)
	{
		UE_LOGF(LogCommonUI, VeryVerbose, "UCommonActivatableWidgetQueue::OnWidgetAddedToList() PriorNumWidgets: %d", MySwitcher->GetNumWidgets() - 1);
		
		// Insert after the empty slot 0 and before the already queued widgets
		MySwitcher->AddSlot(1) [AddedWidget.TakeWidget()];

		if (MySwitcher->GetNumWidgets() == 2)
		{
			// The queue was empty, so we'll show this one immediately
			SetSwitcherIndex(1);
		}
		else if (DisplayedWidget && !DisplayedWidget->IsActivated() && MySwitcher->GetNumWidgets() == 3)
		{
			//The displayed widget is on its way out and we should not be going to 0 anymore, we should go to the new one
			SetSwitcherIndex(1, true); //and get there fast, we need to finish and evict the old widget before anything else happens
		}

	}
}
