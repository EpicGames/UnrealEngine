// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Slider.h"


#include "AnalogSlider.generated.h"

#define UE_API COMMONUI_API

enum class ECommonInputType : uint8;

class SAnalogSlider;

/**
 * A simple widget that shows a sliding bar with a handle that allows you to control the value in a user define range (between 0..1 by default).
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class UAnalogSlider : public USlider
{
	GENERATED_UCLASS_BODY()

public:
	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnFloatValueChangedEvent OnAnalogCapture;

	// UWidget interface
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface
	
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	
	UE_API void HandleOnAnalogCapture(float InValue);

	UE_API void HandleInputMethodChanged(ECommonInputType CurrentInputType);

protected:
	TSharedPtr<SAnalogSlider> MyAnalogSlider;
};

#undef UE_API
