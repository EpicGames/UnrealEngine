// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonUITypes.h"

#include "Components/Widget.h"
#include "UObject/Interface.h"
#include "CommonActionHandlerInterface.generated.h"

#define UE_API COMMONUI_API

// @FIXME: This entire file is related to legacy CommonUI and should be removed in 5.3 - UE-164871

enum EInputEvent : int;
namespace ETouchType { enum Type : int; }
struct FKey;

/** Action Delegates */

/** 
 * Action committed delegate tells the handler that an action is ready to handle. Return value
 * is used to determine if the action was handled or ignored.
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FCommonActionCommited, bool&, bPassThrough);
DECLARE_DELEGATE_OneParam(FCommonActionCommittedNative, bool&);

/**
 * Action complete delegate will tell a listener if a held action completed. The single delegate
 * will be used for binding with a listener that the multicast delegate calls.
 */
DECLARE_DYNAMIC_DELEGATE(FCommonActionCompleteSingle);
DECLARE_DELEGATE(FCommonActionCompleteSingleNative);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FCommonActionComplete);
DECLARE_MULTICAST_DELEGATE(FCommonActionCompleteNative);

/**
 * Action progress delegate will tell a listener about the progress of an action being held. The 
 * single delegate will be used for binding with a listener that the multicast delegate calls.
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FCommonActionProgressSingle, float, HeldPercent);
DECLARE_DELEGATE_OneParam(FCommonActionProgressSingleNative, float);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FCommonActionProgress, float, HeldPercent);
DECLARE_MULTICAST_DELEGATE_OneParam(FCommonActionProgressNative, float);

// @FIXME: This struct is related to legacy CommonUI and should be removed in 5.3 - UE-164871
USTRUCT(BlueprintType)
struct FCommonInputActionHandlerData
{
	GENERATED_BODY()

	FCommonInputActionHandlerData()
		: State(EInputActionState::Enabled)
	{
	}

	FCommonInputActionHandlerData(const FDataTableRowHandle& InInputActionRow)
		: InputActionRow(InInputActionRow)
		, State(EInputActionState::Enabled)
	{
	}

	FCommonInputActionHandlerData(const FDataTableRowHandle& InInputActionRow, UWidget* InPopupMenu)
		: InputActionRow(InInputActionRow)
		, State(EInputActionState::Enabled)
		, PopupMenu(InPopupMenu)
	{
	}

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Default)
	FDataTableRowHandle InputActionRow;

	UE_API EInputActionState GetState(ECommonInputType InputType, const FName& GamepadName) const;
	void SetState(EInputActionState InState)
	{
		State = InState;
	}

	TWeakObjectPtr<UWidget> GetPopupMenu() const { return PopupMenu; }

private:
	UPROPERTY(EditAnywhere, Category = Default)
	EInputActionState State;

	TWeakObjectPtr<UWidget> PopupMenu;
};

/**
 * Action Handle Data used to trigger any callbacks by an object implementing the 
 * ICommonActionHandlerInterface. For use with native delegates
 */
struct FCommonInputActionHandlerDelegateData
{
	FCommonActionProgressNative OnActionProgress;
	FCommonActionCommittedNative OnActionCommitted;
	FCommonActionCompleteNative OnActionComplete;

	FCommonActionCommittedNative OnDisabledActionCommited;

	FCommonInputActionHandlerData Data;

	FCommonInputActionHandlerDelegateData()
	{
	}

	FCommonInputActionHandlerDelegateData(const FDataTableRowHandle& InInputActionRow, const FCommonActionProgressSingleNative& ActivationProgressDelegate, const FCommonActionCommittedNative& ActivationCommittedDelegate, UWidget* PopupMenu)
		: OnActionCommitted(ActivationCommittedDelegate)
		, Data(InInputActionRow, PopupMenu)
	{
		OnActionProgress.Add(ActivationProgressDelegate);
	}

};

/**
 *  Action Handler Interface is primarily used to take key input and do something with it
 *	in the implementation of the interface or another user widget.
 * 
 *  @FIXME: This interface is related to legacy CommonUI and should be removed in 5.3 - UE-164871
 */

UINTERFACE(MinimalAPI)
class UCommonActionHandlerInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};


class ICommonActionHandlerInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	/** The common input manager calls this for any object implementing this interface */
	virtual bool HandleHoldInput(int32 ControllerId, FKey KeyPressed, EInputEvent EventType, bool& bPassThrough) = 0;
	virtual bool HandlePressInput(FKey KeyPressed, EInputEvent EventType, bool& bPassThrough) = 0;
	virtual bool HandleTouchInput(uint32 TouchId, ETouchType::Type TouchType, const FVector2D& TouchLocation, bool& bPassThrough) = 0;

	/** When an action progress happens for a held key, this will be called */
	virtual void UpdateCurrentlyHeldAction(float InDeltaTime) = 0;
	
	/** 
	 *	This will trigger the first input action based on input action data and the hold action flag
	 *  @param Key the key to use for triggering the first matching input action
	 *  @param bHoldAction if true, we're looking for a hold action specifically.
	 */
	virtual void TriggerFirstMatchingInputAction(int32 ControllerId, const FCommonInputActionDataBase& InInputActionData, bool bHoldAction) = 0;
};

#undef UE_API
