// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "Input/UIActionBindingHandle.h"

#include "Engine/DataTable.h"
#include "CommonActionWidget.generated.h"

#define UE_API COMMONUI_API

class UCommonInputSubsystem;
enum class ECommonInputType : uint8;
struct FCommonInputActionDataBase;

class SBox;
class SImage;
class UMaterialInstanceDynamic;

/**
 * A widget that shows a platform-specific icon for the given input action.
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UCommonActionWidget: public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	//UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface
	
	/** Begin UWidget */
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void SynchronizeProperties() override;
	/** End UWidget */
	
	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API virtual FSlateBrush GetIcon() const;

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API FText GetDisplayText() const;

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API UMaterialInstanceDynamic* GetIconDynamicMaterial();

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API virtual void SetEnhancedInputAction(UInputAction* InInputAction);

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API const UInputAction* GetEnhancedInputAction() const;

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API virtual void SetInputAction(FDataTableRowHandle InputActionRow);

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API virtual void SetInputActionBinding(FUIActionBindingHandle BindingHandle);

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API virtual void SetInputActions(TArray<FDataTableRowHandle> NewInputActions);

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API void SetIconRimBrush(FSlateBrush InIconRimBrush);

	UFUNCTION(BlueprintCallable, Category = CommonActionWidget)
	UE_API virtual bool IsHeldAction() const;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnInputMethodChanged, bool, bUsingGamepad);
	UPROPERTY(BlueprintAssignable, Category = CommonActionWidget)
	FOnInputMethodChanged OnInputMethodChanged;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnInputIconUpdated);
	UPROPERTY(BlueprintAssignable, Category = CommonActionWidget)
	FOnInputIconUpdated OnInputIconUpdated;

	/**
	 * The material to use when showing held progress, the progress will be sent using the material parameter
	 * defined by ProgressMaterialParam and the value will range from 0..1.
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CommonActionWidget)
	FSlateBrush ProgressMaterialBrush;

	/** The material parameter on ProgressMaterialBrush to update the held percentage.  This value will be 0..1. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonActionWidget)
	FName ProgressMaterialParam;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonActionWidget)
	FSlateBrush IconRimBrush;

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	UE_API virtual void OnActionProgress(float HeldPercent);
	UE_API virtual void OnActionComplete();
	UE_API void SetProgressMaterial(const FSlateBrush& InProgressMaterialBrush, const FName& InProgressMaterialParam);
	UE_API void SetHidden(bool bAlwaysHidden);

protected:
	/**
	 * List all the input actions that this common action widget is intended to represent.  In some cases you might have multiple actions
	 * that you need to represent as a single entry in the UI.  For example - zoom, might be mouse wheel up or down, but you just need to
	 * show a single icon for Up & Down on the mouse, this solves that problem.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonActionWidget, meta = (RowType = "/Script/CommonUI.CommonInputActionDataBase", TitleProperty = "RowName"))
	TArray<FDataTableRowHandle> InputActions;

	/**
	 * Input Action this common action widget is intended to represent. Optional if using EnhancedInputs
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category = CommonActionWidget, meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	TObjectPtr<class UInputAction> EnhancedInputAction;

	FUIActionBindingHandle DisplayedBindingHandle;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FDataTableRowHandle InputActionDataRow_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = "CommonActionWidget")
	FKey DesignTimeKey;
#endif

protected:
	UE_API UCommonInputSubsystem* GetInputSubsystem() const;
	UE_API const FCommonInputActionDataBase* GetInputActionData() const;

	UE_API virtual void UpdateActionWidget();

	UE_API virtual bool ShouldUpdateActionWidgetIcon() const;

	UE_API virtual void OnWidgetRebuilt() override;
	
	UE_API void UpdateBindingHandleInternal(FUIActionBindingHandle BindingHandle);

	UE_API void ListenToInputMethodChanged(bool bListen = true);
	UE_API void HandleInputMethodChanged(ECommonInputType InInputType);

	UFUNCTION()
	UE_API void OnEnhancedInputMappingsRebuilt();

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> ProgressDynamicMaterial;
	
	TSharedPtr<SBox> MyKeyBox;

	TSharedPtr<SImage> MyIcon;

	TSharedPtr<SImage> MyProgressImage;

	TSharedPtr<SImage> MyIconRim;

	UPROPERTY()
	FSlateBrush Icon;

	bool bAlwaysHideOverride = false;
};

#undef UE_API
