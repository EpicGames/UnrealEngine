// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonUserWidget.h"
#include "CommonInputActionDomain.h"
#include "CommonActivatableWidget.generated.h"

#define UE_API COMMONUI_API

class FActivatableTreeNode;
class UCommonInputActionDomain;
class UInputMappingContext;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnWidgetActivationChanged);

/**
 * Metadata set on UCommonActivatableWidget's underlying slate widget to allow Slate -> UMG lookup
 */
class FCommonActivatableSlateMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FCommonActivatableSlateMetaData, ISlateMetaData);
};

/** 
 * The base for widgets that are capable of being "activated" and "deactivated" during their lifetime without being otherwise modified or destroyed. 
 *
 * This is generally desired for one or more of the following purposes:
 *	- This widget can turn on/off without being removed from the hierarchy (or otherwise reconstructing the underlying SWidgets), so Construct/Destruct are insufficient
 *	- You'd like to be able to "go back" from this widget, whether that means back a breadcrumb, closing a modal, or something else. This is built-in here.
 *	- This widget's place in the hierarchy is such that it defines a meaningful node-point in the tree of activatable widgets through which input is routed to all widgets.
 *
 * By default, an activatable widget:
 *	- Is not automatically activated upon construction
 *	- Does not register to receive back actions (or any other actions, for that matter)
 *	- If classified as a back handler, is automatically deactivated (but not destroyed) when it receives a back action
 * 
 * Note that removing an activatable widget from the UI (i.e. triggering Destruct()) will always deactivate it, even if the UWidget is not destroyed.
 * Re-constructing the underlying SWidget will only result in re-activation if auto-activate is enabled.
 */
UCLASS(MinimalAPI, meta = (DisableNativeTick))
class UCommonActivatableWidget : public UCommonUserWidget
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = ActivatableWidget)
	bool IsActivated() const { return bIsActive; }

	UFUNCTION(BlueprintCallable, Category = ActivatableWidget)
	UE_API void ActivateWidget();

	UFUNCTION(BlueprintCallable, Category = ActivatableWidget)
	UE_API void DeactivateWidget();
	
	/**
	 * Visibilities to use for when bound widgets in BindVisibilityToActivation are activated.
	 *
	 * @param	OnActivatedVisibility	- Visibility for when bound widgets are active
	 * @param	OnDeactivatedVisibility - Visibility for when bound widgets are not active, not used if this widget has activation / deactivation visibilities
	 * @param	bInAllActive			- True if we should switch to activated visibility only when all bound widgets are active
	 */
	UFUNCTION(BlueprintCallable, Category = ActivatableWidget, meta=(BlueprintProtected = "true"))
	UE_API void SetBindVisibilities(ESlateVisibility OnActivatedVisibility, ESlateVisibility OnDeactivatedVisibility, UPARAM(DisplayName = "All Active") bool bInAllActive);

	/**
	 * Bind our visibility to the activation of another widget, useful for making mouse collisions behave similiar to console navigation w.r.t activation
	 * Will immediately update visibility based on the bound widget activation & visibilites set by SetBindVisibilities.
	 *
	 * @param	ActivatableWidget		- The widget whose activation / deactivation will modify our visibility 
	 */
	UFUNCTION(BlueprintCallable, Category = ActivatableWidget)
	UE_API void BindVisibilityToActivation(UCommonActivatableWidget* ActivatableWidget);

	/** Returns the desired widget to focus when this Widget Activates. */
	UFUNCTION(BlueprintCallable, Category = ActivatableWidget)
	UE_API UWidget* GetDesiredFocusTarget() const;

	/** Clears the cached focus target that's set when bAutoRestoreFocus is true */
	UFUNCTION(BlueprintCallable, Category = ActivatableWidget)
	UE_API void ClearFocusRestorationTarget();

	/**
	 * Ask for focus to be re-set to our current DesiredFocusTarget,
	 * but only if our node is currently the leaf-most active node (no stealing!).
	 * This is useful for complex cases like: the buttons animate in from off-screen,
	 * or the buttons are deeply nested in a multi-switcher hierarchy and it would be burdensome
	 * to wrap each element in a CommonActivatableWidget.
	 */
	UFUNCTION(BlueprintCallable, Category = ActivatableWidget)
	UE_API void RequestRefreshFocus();

	FSimpleMulticastDelegate& OnActivated() const { return OnActivatedEvent; }
	FSimpleMulticastDelegate& OnDeactivated() const { return OnDeactivatedEvent; }

public:

	/**
	 * Gets custom game-specific activation metadata for this widget. By default does nothing & used for nothing.
	 */
	UE_API virtual TOptional<FActivationMetadata> GetActivationMetadata() const;

	/**
	 * Gets the desired input configuration to establish when this widget activates and can receive input (i.e. all parents are also active).
	 * This configuration will override the existing one established by any previous activatable widget and restore it (if valid) upon deactivation.
	 */
	UE_API virtual TOptional<FUIInputConfig> GetDesiredInputConfig() const;

	bool IsModal() const { return bIsModal; }
	bool SupportsActivationFocus() const { return bSupportsActivationFocus; }
	bool AutoRestoresFocus() const { return bSupportsActivationFocus && bAutoRestoreFocus; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FActivatableWidgetRebuildEvent, UCommonActivatableWidget&);
	static UE_API FActivatableWidgetRebuildEvent OnRebuilding;
	
	FSimpleMulticastDelegate& OnSlateReleased() const { return OnSlateReleasedEvent; }

	FSimpleMulticastDelegate& OnRequestRefreshFocus() const { return OnRequestRefreshFocusEvent; }

	bool SetsVisibilityOnActivated() const { return bSetVisibilityOnActivated; }
	bool SetsVisibilityOnDeactivated() const { return bSetVisibilityOnDeactivated; }

	UE_API TWeakPtr<FActivatableTreeNode> GetInputTreeNode() const;
	UE_API void RegisterInputTreeNode(const TSharedPtr<FActivatableTreeNode>& OwnerNode);
	UE_API void ClearActiveHoldInputs();

	/**
	 * Returns the widget's ActionDomain, respecting any inheritance requirements.
	 */
	UE_API TObjectPtr<UCommonInputActionDomain> GetCalculatedActionDomain();

	/**
	 * Reset the Action Domain cached by GetCalculatedActionDomain()
	 * Call this when updating this widget or a parent widget's action domain
	 */
	UE_API void ResetCalculatedActionDomainCache();

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void OnWidgetRebuilt() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void NativeConstruct() override;
	UE_API virtual void NativeDestruct() override;

	/**
	 * Override to provide the desired widget that should receive focus when this becomes the primary active widget.
	 * If bAutoRestoreFocus is true, is only called when there is no valid cached restoration target (to provide the default/fallback)
	 */
	UE_API virtual UWidget* NativeGetDesiredFocusTarget() const;
	
	/** 
	 * Implement to provide the desired widget to focus if/when this activatable becomes the primary active widget.
	 * Note: This is a fallback used only if the native class parentage does not provide a target.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = ActivatableWidget, meta = (DisplayName = "Get Desired Focus Target"))
	UE_API UWidget* BP_GetDesiredFocusTarget() const;
	
	/** 
	 * Implement to provide the input config to use when this widget is activated. Keep in mind when all widgets
	 * are deactivated, CommonUI will not attempt to automatically restore the input config to before any widget was active.
	 * Note: This is a fallback used only if the native class parentage does not provide an input config.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = ActivatableWidget, meta = (DisplayName = "Get Desired Input Config"))
	UE_API FUIInputConfig BP_GetDesiredInputConfig() const;

	UFUNCTION(BlueprintImplementableEvent, Category = ActivatableWidget, meta = (DisplayName = "On Activated"))
	UE_API void BP_OnActivated();
	UE_API virtual void NativeOnActivated();
	UE_API virtual void ActivateMappingContext();

	UFUNCTION(BlueprintImplementableEvent, Category = ActivatableWidget, meta = (DisplayName = "On Deactivated"))
	UE_API void BP_OnDeactivated();
	UE_API virtual void NativeOnDeactivated();
	UE_API virtual void DeactivateMappingContext();

	/** 
	 * Override in BP implementations to provide custom behavior when receiving a back action 
	 * Note: Only called if native code in the base class hasn't handled it in NativeOnHandleBackAction 
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = ActivatableWidget, meta = (DisplayName = "On Handle Back Action"))
	UE_API bool BP_OnHandleBackAction();
	UE_API virtual bool NativeOnHandleBackAction();

	UE_API void HandleBackAction();

	/** True to receive "Back" actions automatically. Custom back handler behavior can be provided, default is to deactivate. */
	UPROPERTY(EditAnywhere, Category = Back)
	bool bIsBackHandler = false;

	/** True to receive "Back" actions automatically. Custom back handler behavior can be provided, default is to deactivate. */
	UPROPERTY(EditAnywhere, Category = Back)
	bool bIsBackActionDisplayedInActionBar = false;

	/** You can specify an override for display name for the back action. If left empty, the default will be used. */
	UPROPERTY(EditAnywhere, Category = Back)
	FText OverrideBackActionDisplayName;

	/** True to automatically activate upon construction */
	UPROPERTY(EditAnywhere, Category = Activation)
	bool bAutoActivate = false;

	/**
	 * True if this widget is a candidate to receive/route focus or specify a desired UIInputConfig when active.
	 * Primary reason for disabling is for utility sub-widgets within a larger screen that possess actions, but are never
	 * intended to be involved in navigation or dictate changes to the active UI input config.
	 */
	UPROPERTY(EditAnywhere, Category = Activation)
	bool bSupportsActivationFocus = true;

	/** 
	 * True to have this widget be treated as a root node for input routing, regardless of its actual parentage.
	 * Should seldom be needed, but useful in cases where a child widget should prevent all action processing by parents, even though they remain active (ex: modal popup menu).
	 */
	UPROPERTY(EditAnywhere, Category = Activation, meta = (EditCondition = bSupportsActivationFocus))
	bool bIsModal = false;

	/** 
	 * True to prefer automatically restoring focus to the widget that was focused when this widget last became the non-leafmost-active-widget.
	 * If true and a valid restoration candidate exists, we'll use that. If it doesn't, we rely on GetDesiredFocusTarget()
	 * If false, we simply always rely on GetDesiredFocusTarget()
	 */
	UPROPERTY(EditAnywhere, Category = Activation, meta = (EditCondition = bSupportsActivationFocus))
	bool bAutoRestoreFocus = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input", meta = (InlineEditConditionToggle))
	bool bOverrideActionDomain = false;
	
	/** Optional mapping context to be applied & removed on activation & deactivation, respectively. */
	UPROPERTY(EditAnywhere, Category="Input", meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	TObjectPtr<UInputMappingContext> InputMapping;

	/** Enhanced Input priority. Higher priority input mappings will be prioritized over mappings with a lower priority. */
	UPROPERTY(EditAnywhere, Category="Input", meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	int32 InputMappingPriority = 0;

	/**
	 * Enable to override the inherited ActionDomain from owning CommonActivatableWidget.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input", meta = (EditCondition = "bOverrideActionDomain"))
	TSoftObjectPtr<UCommonInputActionDomain> ActionDomainOverride;

private:
	/** See BindVisibilityToMultipleActivations */
	UE_API void HandleVisibilityBoundWidgetActivations();

	/** Fires when the widget is activated. */
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (AllowPrivateAccess = true, DisplayName = "On Widget Activated"))
	FOnWidgetActivationChanged BP_OnWidgetActivated;
	
	/** Fires when the widget is deactivated. */
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (AllowPrivateAccess = true, DisplayName = "On Widget Deactivated"))
	FOnWidgetActivationChanged BP_OnWidgetDeactivated;
	
	UPROPERTY(BlueprintReadOnly, Category = ActivatableWidget, meta = (AllowPrivateAccess = true))
	bool bIsActive = false;

	/** List of widgets whose collective activation controls our visibility. */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<UCommonActivatableWidget>> VisibilityBoundWidgets;

	/** Visibility to use when widgets we are bound to are activated */
	ESlateVisibility ActivatedBindVisibility = ESlateVisibility::SelfHitTestInvisible;

	/** Visibility to use when widgets we are bound to are deactivated, not used if widget has activation / deactivation visibilities */
	ESlateVisibility DeactivatedBindVisibility = ESlateVisibility::SelfHitTestInvisible;

	/** True if we should switch to activated visibility only when all bound widgets are active */
	bool bAllActive = true;

	/** Handle to default back action, if bound */
	FUIActionBindingHandle DefaultBackActionHandle;

	/** Input tree node referencing this widget */
	TWeakPtr<FActivatableTreeNode> InputTreeNode;

	mutable FSimpleMulticastDelegate OnActivatedEvent;
	mutable FSimpleMulticastDelegate OnDeactivatedEvent;
	mutable FSimpleMulticastDelegate OnSlateReleasedEvent;
	mutable FSimpleMulticastDelegate OnRequestRefreshFocusEvent;

	TOptional<TSoftObjectPtr<UCommonInputActionDomain>> CalculatedActionDomainCache;

protected:
	
	UPROPERTY(EditAnywhere, Category = Activation, meta = (InlineEditConditionToggle = "ActivatedVisibility"))
	bool bSetVisibilityOnActivated = false;

	UPROPERTY(EditAnywhere, Category = Activation, meta = (EditCondition = "bSetVisibilityOnActivated"))
	ESlateVisibility ActivatedVisibility = ESlateVisibility::SelfHitTestInvisible;

	UPROPERTY(EditAnywhere, Category = Activation, meta = (InlineEditConditionToggle = "DeactivatedVisibility"))
	bool bSetVisibilityOnDeactivated = false;

	UPROPERTY(EditAnywhere, Category = Activation, meta = (EditCondition = "bSetVisibilityOnDeactivated"))
	ESlateVisibility DeactivatedVisibility = ESlateVisibility::Collapsed;
	
	UE_API virtual void InternalProcessActivation();
	UE_API virtual void InternalProcessDeactivation();
	UE_API void Reset();
};

#undef UE_API
