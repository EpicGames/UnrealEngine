// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonAnimatedSwitcher.h"


#include "CommonActivatableWidgetSwitcher.generated.h"

#define UE_API COMMONUI_API

class UCommonActivatableWidget;

/**
 * An animated switcher that knows about CommonActivatableWidgets. It can also hold other Widgets.
 */
UCLASS(MinimalAPI)
class UCommonActivatableWidgetSwitcher : public UCommonAnimatedSwitcher
{
	GENERATED_BODY()

public:
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

protected:
	UE_API virtual void OnWidgetRebuilt() override;

	UE_API virtual void HandleOutgoingWidget() override;
	UE_API virtual void HandleSlateActiveIndexChanged(int32 ActiveIndex) override;

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Switcher")
	bool bClearFocusRestorationTargetOfDeactivatedWidgets = false;

private:
	void HandleOwningWidgetActivationChanged(const bool bIsActivated);

	void AttemptToActivateActiveWidget();
	void DeactivateActiveWidget();

	void BindOwningActivatableWidget();
	void UnbindOwningActivatableWidget();

	UCommonActivatableWidget* GetOwningActivatableWidget() const;

	TOptional<TWeakObjectPtr<UCommonActivatableWidget>> WeakOwningActivatableWidget;
};

#undef UE_API
