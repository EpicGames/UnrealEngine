// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/WidgetSwitcher.h"
#include "Slate/SCommonAnimatedSwitcher.h"

#include "CommonAnimatedSwitcher.generated.h"

#define UE_API COMMONUI_API

class SOverlay;
class SSpacer;


DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnActiveIndexChangedDelegate, UWidget*, ActiveWidget, int32, ActiveIndex);

/**
 * A widget switcher that activates / deactivates CommonActivatableWidgets, allowing for associated animations to trigger.
 */
UCLASS(MinimalAPI)
class UCommonAnimatedSwitcher : public UWidgetSwitcher
{
	GENERATED_BODY()

public:
	UE_API UCommonAnimatedSwitcher(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	UE_API virtual void SetActiveWidgetIndex(int32 Index) override;
	UE_API virtual void SetActiveWidget(UWidget* Widget) override;

	UFUNCTION(BlueprintCallable, Category = "Common Widget Switcher")
    UE_API void ActivateNextWidget(bool bCanWrap);

    UFUNCTION(BlueprintCallable, Category = "Common Widget Switcher")
    UE_API void ActivatePreviousWidget(bool bCanWrap);

    UFUNCTION(BlueprintCallable, Category = "Common Widget Switcher")
    UE_API bool HasWidgets() const;

	UFUNCTION(BlueprintCallable, Category = "Common Widget Switcher")
	UE_API void SetDisableTransitionAnimation(bool bDisableAnimation);

	UFUNCTION(BlueprintCallable, Category = "Common Widget Switcher")
	UE_API bool IsCurrentlySwitching() const;

	/** Is the switcher playing a transition animation? */
	UFUNCTION(BlueprintCallable, Category = "Common Widget Switcher")
	UE_API bool IsTransitionPlaying() const;

	UE_API UWidget* GetPendingActiveWidget() const;
	UE_API int32 GetPendingActiveWidgetIndex() const;

protected:
	UE_API virtual void HandleSlateActiveIndexChanged(int32 ActiveIndex);

	UE_API virtual void HandleSlateIsTransitioningChanged(bool bIsTransitioning);

	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;

	virtual void HandleOutgoingWidget() {};

public:
	/** Fires when the active widget displayed by the switcher changes */
	DECLARE_EVENT_TwoParams(UCommonAnimatedSwitcher, FOnActiveIndexChanged, UWidget*, int32)
	FOnActiveIndexChanged OnActiveWidgetIndexChanged;

	/** Fires when the switcher changes its transition animation state */
	DECLARE_EVENT_OneParam(UCommonAnimatedSwitcher, FOnTransitioningChanged, bool)
	FOnTransitioningChanged OnTransitioningChanged;
	
	/** Fires when the active widget displayed by the switcher changes */
	UPROPERTY(BlueprintAssignable, Category = "Common Widget Switcher")
	FOnActiveIndexChangedDelegate OnActiveWidgetIndexChangedBP;

protected:
	/** The type of transition to play between widgets */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Transition")
	ECommonSwitcherTransition TransitionType;

	/** The curve function type to apply to the transition animation */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Transition")
	ETransitionCurve TransitionCurveType;

	/** The total duration of a single transition between widgets */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Transition")
	float TransitionDuration;

	/** Controls how we will choose another widget if a transitioning widget is removed during the transition. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Transition")
	ECommonSwitcherTransitionFallbackStrategy TransitionFallbackStrategy = ECommonSwitcherTransitionFallbackStrategy::None;

	TSharedPtr<SOverlay> MyOverlay;
	TSharedPtr<SSpacer> MyInputGuard;
	TSharedPtr<SCommonAnimatedSwitcher> MyAnimatedSwitcher;

	/* If set, transition animations will not play */
	bool bInstantTransition = false;

	bool bSetOnce = false;
	bool bCurrentlySwitching = false;

private:
	UE_API void SetActiveWidgetIndex_Internal(int32 Index);
};

#undef UE_API
