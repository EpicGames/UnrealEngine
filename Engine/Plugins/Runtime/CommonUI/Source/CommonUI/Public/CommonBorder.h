// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Border.h"
#include "CommonBorder.generated.h"

#define UE_API COMMONUI_API

struct FDesignerChangedEventArgs;

/* 
 * ---- All properties must be EditDefaultsOnly, BlueprintReadOnly !!! -----
 * We return the CDO to blueprints, so we cannot allow any changes (blueprint doesn't support const variables)
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = UI, meta = (Category = "Common UI"))
class UCommonBorderStyle : public UObject
{
	GENERATED_BODY()

public:
	UE_API UCommonBorderStyle();
	
	/** The brush for the background of the border */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	FSlateBrush Background;

	UFUNCTION(BlueprintCallable, Category = "Common Border Style|Getters")
	UE_API void GetBackgroundBrush(FSlateBrush& Brush) const;
};

/**
 * Uses the border style template defined in CommonUI project settings by default
 */
UCLASS(MinimalAPI, Config = CommonUI, DefaultConfig, ClassGroup = UI, meta = (Category = "Common UI", DisplayName = "Common Border"))
class UCommonBorder : public UBorder
{
	GENERATED_UCLASS_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "Common Border")
	UE_API void SetStyle(TSubclassOf<UCommonBorderStyle> InStyle);

	/** References the border style to use */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Common Border", meta = (ExposeOnSpawn = true))
	TSubclassOf<UCommonBorderStyle> Style;

	/** Turning this on will cause the safe zone size to be removed from this borders content padding down to the minimum specified */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Common Border")
	bool bReducePaddingBySafezone;

	/** The minimum padding we will reduce to when the safezone grows */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Common Border", meta = (EditCondition = "bReducePaddingBySafezone"))
	FMargin MinimumPadding;

#if WITH_EDITORONLY_DATA
	/** Used to track widgets that were created before changing the default style pointer to null */
	UPROPERTY()
	bool bStyleNoLongerNeedsConversion;
#endif

protected:
	UE_API virtual void PostLoad() override;

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	UE_API void SafeAreaUpdated();
	void DebugSafeAreaUpdated(const FMargin& NewSafeZone, bool bShouldRecacheMetrics) { SafeAreaUpdated(); };
#if WITH_EDITOR
	UE_API virtual void OnCreationFromPalette() override;
	UE_API const FText GetPaletteCategory() override;
	UE_API virtual void OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs) override;
#endif // WITH_EDITOR

protected:
#if WITH_EDITOR
	/** The editor-only size constraint passed in by UMG Designer*/
	TOptional<FVector2D> DesignerSize;
#endif

private:
	UE_API const UCommonBorderStyle* GetStyleCDO() const;

};

#undef UE_API
