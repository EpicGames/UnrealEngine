// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Binding/States/WidgetStateRegistration.h"
#include "CommonUserWidget.h"
#include "Components/Button.h"
#include "Engine/DataTable.h"
#include "FieldNotification/WidgetEventField.h"
#include "Types/ISlateMetaData.h"
#include "CommonInputModeTypes.h"
#include "CommonInputBaseTypes.h"
#include "Containers/Ticker.h"
#include "Framework/SlateDelegates.h"
#include "CommonButtonBase.generated.h"

class UCommonButtonBase;
class UMaterialInstanceDynamic;
enum class ECommonInputType : uint8;

class UCommonTextStyle;
class SBox;
class UCommonActionWidget;
class UInputAction;

enum class EHoverEventSource : uint8
{
	Unknown,
	MouseEvent,
	InteractabilityChanged,
	SelectionChanged,
	SimulationForTouch,
};

class FCommonButtonMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FCommonButtonMetaData, ISlateMetaData)

	FCommonButtonMetaData(const UCommonButtonBase& InOwningCommonButtonInternal):OwningCommonButton(&InOwningCommonButtonInternal)
	{}

	const TWeakObjectPtr<const UCommonButtonBase> OwningCommonButton;
};

USTRUCT(BlueprintType)
struct FCommonButtonStyleOptionalSlateSound
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Properties")
	bool bHasSound = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Properties", meta = (EditCondition = "bHasSound"))
	FSlateSound Sound;

	explicit operator bool() const
	{
		return bHasSound;
	}
};


/* ---- All properties must be EditDefaultsOnly, BlueprintReadOnly !!! -----
 *       we return the CDO to blueprints, so we cannot allow any changes (blueprint doesn't support const variables)
 */
UCLASS(Abstract, Blueprintable, MinimalAPI, ClassGroup = UI, meta = (Category = "Common UI"))
class UCommonButtonStyle : public UObject
{
	GENERATED_BODY()

	COMMONUI_API virtual bool NeedsLoadForServer() const override;

public:
	
	/** Whether or not the style uses a drop shadow */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	bool bSingleMaterial;

	/** The normal (un-selected) brush to apply to each size of this button */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (EditCondition = "bSingleMaterial"))
	FSlateBrush SingleMaterialBrush;

	/** The normal (un-selected) brush to apply to each size of this button */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Normal", meta = (EditCondition = "!bSingleMaterial"))
	FSlateBrush NormalBase;

	/** The normal (un-selected) brush to apply to each size of this button when hovered */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Normal", meta = (EditCondition = "!bSingleMaterial"))
	FSlateBrush NormalHovered;

	/** The normal (un-selected) brush to apply to each size of this button when pressed */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Normal", meta = (EditCondition = "!bSingleMaterial"))
	FSlateBrush NormalPressed;

	/** The selected brush to apply to each size of this button */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Selected", meta = (EditCondition = "!bSingleMaterial"))
	FSlateBrush SelectedBase;

	/** The selected brush to apply to each size of this button when hovered */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Selected", meta = (EditCondition = "!bSingleMaterial"))
	FSlateBrush SelectedHovered;

	/** The selected brush to apply to each size of this button when pressed */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Selected", meta = (EditCondition = "!bSingleMaterial"))
	FSlateBrush SelectedPressed;

	/** The disabled brush to apply to each size of this button */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Disabled", meta = (EditCondition = "!bSingleMaterial"))
	FSlateBrush Disabled;

	/** The button content padding to apply for each size */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	FMargin ButtonPadding;
	
	/** The custom padding of the button to use for each size */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	FMargin CustomPadding;

	/** The minimum width of buttons using this style */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	int32 MinWidth;

	/** The minimum height of buttons using this style */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	int32 MinHeight;
	
	/** The maximum width of buttons using this style */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	int32 MaxWidth;
	
	/** The maximum height of buttons using this style */
    UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
    int32 MaxHeight;

	/** The text style to use when un-selected */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	TSubclassOf<UCommonTextStyle> NormalTextStyle;

	/** The text style to use when un-selected */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	TSubclassOf<UCommonTextStyle> NormalHoveredTextStyle;

	/** The text style to use when selected */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	TSubclassOf<UCommonTextStyle> SelectedTextStyle;

	/** The text style to use when un-selected */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	TSubclassOf<UCommonTextStyle> SelectedHoveredTextStyle;

	/** The text style to use when disabled */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	TSubclassOf<UCommonTextStyle> DisabledTextStyle;

	/** The sound to play when the button is pressed */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Pressed Sound"))
	FSlateSound PressedSlateSound;

	/** The sound to play when the button is clicked */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Clicked Sound"))
	FSlateSound ClickedSlateSound;

	/** The sound to play when the button is pressed while selected */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Selected Pressed Sound"))
	FCommonButtonStyleOptionalSlateSound SelectedPressedSlateSound;

	/** The sound to play when the button is clicked while selected */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Selected Clicked Sound"))
	FCommonButtonStyleOptionalSlateSound SelectedClickedSlateSound;

	/** The sound to play when the button is pressed while locked */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Locked Pressed Sound"))
	FCommonButtonStyleOptionalSlateSound LockedPressedSlateSound;

	/** The sound to play when the button is clicked while locked */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Locked Clicked Sound"))
	FCommonButtonStyleOptionalSlateSound LockedClickedSlateSound;
	
	/** The sound to play when the button is hovered */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Hovered Sound"))
	FSlateSound HoveredSlateSound;

	/** The sound to play when the button is hovered while selected */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Selected Hovered Sound"))
	FCommonButtonStyleOptionalSlateSound SelectedHoveredSlateSound;
	
	/** The sound to play when the button is hovered while locked */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties", meta = (DisplayName = "Locked Hovered Sound"))
	FCommonButtonStyleOptionalSlateSound LockedHoveredSlateSound;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetMaterialBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetNormalBaseBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetNormalHoveredBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetNormalPressedBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetSelectedBaseBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetSelectedHoveredBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetSelectedPressedBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetDisabledBrush(FSlateBrush& Brush) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetButtonPadding(FMargin& OutButtonPadding) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API void GetCustomPadding(FMargin& OutCustomPadding) const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API UCommonTextStyle* GetNormalTextStyle() const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API UCommonTextStyle* GetNormalHoveredTextStyle() const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API UCommonTextStyle* GetSelectedTextStyle() const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API UCommonTextStyle* GetSelectedHoveredTextStyle() const;

	UFUNCTION(BlueprintCallable, Category = "Common ButtonStyle|Getters")
	COMMONUI_API UCommonTextStyle* GetDisabledTextStyle() const;
};

DECLARE_DELEGATE_RetVal(FReply, FOnButtonDoubleClickedEvent);

/** Custom UButton override that allows us to disable clicking without disabling the widget entirely */
UCLASS(Experimental, MinimalAPI)	// "Experimental" to hide it in the designer
class UCommonButtonInternalBase : public UButton
{
	GENERATED_UCLASS_BODY()

public:
	COMMONUI_API void SetButtonEnabled(bool bInIsButtonEnabled);
	COMMONUI_API void SetInteractionEnabled(bool bInIsInteractionEnabled);

	/** Updates the IsFocusable flag and updates the bIsFocusable flag of the underlying slate button widget */
	COMMONUI_API void SetButtonFocusable(bool bInIsButtonFocusable);
	COMMONUI_API bool IsHovered() const;
	COMMONUI_API bool IsPressed() const;

	COMMONUI_API void SetMinDesiredHeight(int32 InMinHeight);
	int32 GetMinHeight() const { return MinHeight; }
	COMMONUI_API void SetMinDesiredWidth(int32 InMinWidth);
	int32 GetMinWidth() const { return MinWidth; }
	COMMONUI_API void SetMaxDesiredHeight(int32 InMaxHeight);
	int32 GetMaxHeight() const { return MaxHeight; }
	COMMONUI_API void SetMaxDesiredWidth(int32 InMaxWidth);
	int32 GetMaxWidth() const { return MaxWidth; }

	inline TSharedPtr<class SCommonButton> GetCommonButton() const { return MyCommonButton; }

	/** Called when the button is clicked */
	FOnButtonDoubleClickedEvent HandleDoubleClicked;

	/** Called when the button is clicked */
	UPROPERTY(BlueprintAssignable, Category = "Common Button Internal|Event")
	FOnButtonClickedEvent OnDoubleClicked;

protected:
	// UWidget interface
	COMMONUI_API virtual TSharedRef<SWidget> RebuildWidget() override;
	COMMONUI_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UWidget interface

	COMMONUI_API virtual FReply SlateHandleClickedOverride();
	COMMONUI_API virtual void SlateHandlePressedOverride();
	COMMONUI_API virtual void SlateHandleReleasedOverride();
	COMMONUI_API virtual FReply SlateHandleDoubleClicked();
	COMMONUI_API virtual FReply SlateHandleDragDetectedOverride(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);
	COMMONUI_API virtual void SlateHandleDragEnterOverride(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
	COMMONUI_API virtual void SlateHandleDragLeaveOverride(const FDragDropEvent& DragDropEvent);
	COMMONUI_API virtual FReply SlateHandleDragOverOverride(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
	COMMONUI_API virtual FReply SlateHandleDropOverride(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	/** The minimum width of the button */
	UPROPERTY()
	int32 MinWidth;

	/** The minimum height of the button */
	UPROPERTY()
	int32 MinHeight;
	
	/** The maximum width of the button */
	UPROPERTY()
	int32 MaxWidth;
	
	/** The maximum height of the button */
	UPROPERTY()
	int32 MaxHeight;

	/** If true, this button is enabled. */
	UPROPERTY()
	bool bButtonEnabled;

	/** If true, this button can be interacted with it normally. Otherwise, it will not react to being hovered or clicked. */
	UPROPERTY()
	bool bInteractionEnabled;

	/** Cached pointer to the underlying slate button owned by this UWidget */
	TSharedPtr<SBox> MyBox;

	/** Cached pointer to the underlying slate button owned by this UWidget */
	TSharedPtr<class SCommonButton> MyCommonButton;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FCommonSelectedStateChangedBase, class UCommonButtonBase*, Button, bool, Selected);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FCommonButtonBaseClicked, class UCommonButtonBase*, Button);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnButtonBaseOperationDynamic, class UCommonButtonBase*, Button, UDragDropOperation*, Operation);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnButtonBaseGeoOperationDynamic, class UCommonButtonBase*, Button, const FGeometry&, MyGeometry, UDragDropOperation*, Operation);

/**
 * Button that disables itself when not active. Also updates actions for CommonActionWidget if bound to display platform-specific icons.
 */
UCLASS(Abstract, Blueprintable, MinimalAPI, ClassGroup = UI, meta = (Category = "Common UI", DisableNativeTick))
class UCommonButtonBase : public UCommonUserWidget
{
	GENERATED_UCLASS_BODY()

public:
	// UWidget interface
	COMMONUI_API virtual bool IsHovered() const override;
	// End of UWidget interface

	// UUserWidget interface
	COMMONUI_API virtual void NativeConstruct() override;
	COMMONUI_API virtual void NativeDestruct() override;
	COMMONUI_API virtual bool Initialize() override;
	COMMONUI_API virtual void SetIsEnabled(bool bInIsEnabled) override;
	COMMONUI_API virtual void SetVisibility(ESlateVisibility InVisibility) override;
	COMMONUI_API virtual bool NativeIsInteractable() const override;
	// End of UUserWidget interface
	
	/** Disables this button with a reason (use instead of SetIsEnabled) */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void DisableButtonWithReason(const FText& DisabledReason);

	/** Change whether this widget is selectable at all. If false and currently selected, will deselect. */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetIsInteractionEnabled(bool bInIsInteractionEnabled);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetHideInputAction(bool bInHideInputAction);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetHideInputActionWithKeyboard(bool bInHideInputActionWithKeyboard);

	/** Is this button currently interactable? (use instead of GetIsEnabled) */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API bool IsInteractionEnabled() const;

	/** Is this button currently pressed? */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API bool IsPressed() const;

	/** Set the click method for mouse interaction */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetClickMethod(EButtonClickMethod::Type InClickMethod);

	/** Set the click method for touch interaction */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetTouchMethod(EButtonTouchMethod::Type InTouchMethod);

	/** Set the click method for keyboard/gamepad button press interaction */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetPressMethod(EButtonPressMethod::Type InPressMethod);

	/** Change whether this widget is selectable at all. If false and currently selected, will deselect. */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetIsSelectable(bool bInIsSelectable);

	/** Change whether this widget is selectable at all. If false and currently selected, will deselect. */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetIsInteractableWhenSelected(bool bInInteractableWhenSelected);

	/** Change whether this widget is toggleable. If toggleable, clicking when selected will deselect. */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetIsToggleable(bool bInIsToggleable);

	/** Change whether this widget should use the fallback default input action. */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetShouldUseFallbackDefaultInputAction(bool bInShouldUseFallbackDefaultInputAction);

	/** 
	 * Change the selected state manually.
	 * @param bGiveClickFeedback	If true, the button may give user feedback as if it were clicked. IE: Play a click sound, trigger animations as if it were clicked.
	 */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetIsSelected(bool InSelected, bool bGiveClickFeedback = true);

	/** Change whether this widget is locked. If locked, the button can be focusable and responsive to mouse input but will not broadcast OnClicked events. */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetIsLocked(bool bInIsLocked);

	/** @returns True if the button is currently in a selected state, False otherwise */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API bool GetSelected() const;

	/** @returns True if the button is currently locked, False otherwise */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API bool GetLocked() const;

	UFUNCTION(BlueprintCallable, Category = "Common Button" )
	COMMONUI_API void ClearSelection();

	/** Set whether the button should become selected upon receiving focus or not; Only settable for buttons that are selectable */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetShouldSelectUponReceivingFocus(bool bInShouldSelectUponReceivingFocus);

	/** Get whether the button should become selected upon receiving focus or not */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API bool GetShouldSelectUponReceivingFocus() const;

	/** Sets the style of this button, rebuilds the internal styling */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetStyle(TSubclassOf<UCommonButtonStyle> InStyle = {});

	/** @Returns Current button style*/
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API UCommonButtonStyle* GetStyle() const;

	COMMONUI_API const UCommonButtonStyle* GetStyleCDO() const;

	/** @return The current button padding that corresponds to the current size and selection state */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API void GetCurrentButtonPadding(FMargin& OutButtonPadding) const;

	/** @return The custom padding that corresponds to the current size and selection state */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API void GetCurrentCustomPadding(FMargin& OutCustomPadding) const;
	
	/** @return The text style that corresponds to the current size and selection state */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API UCommonTextStyle* GetCurrentTextStyle() const;

	/** @return The class of the text style that corresponds to the current size and selection state */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API TSubclassOf<UCommonTextStyle> GetCurrentTextStyleClass() const;

	/** Sets the minimum dimensions of this button */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetMinDimensions(int32 InMinWidth, int32 InMinHeight);

	/** Sets the maximum dimensions of this button */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetMaxDimensions(int32 InMaxWidth, int32 InMaxHeight);

	/** Updates the display name via its bound UIAction **/
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetActionDisplayName(const FText& InDisplayName);

	/** Updates the current triggered action */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetTriggeredInputAction(const FDataTableRowHandle &InputActionRow);

	/** Updates the current triggering action */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetTriggeringInputAction(const FDataTableRowHandle & InputActionRow);

	/** Updates the current triggering enhanced input action, requires enhanced input enabled in CommonUI settings */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetTriggeringEnhancedInputAction(UInputAction* InInputAction);

	/** Gets the appropriate input action that is set */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API bool GetInputAction(FDataTableRowHandle &InputActionRow) const;

	/** Gets the appropriate enhanced input action that is set */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API UInputAction* GetEnhancedInputAction() const;

	/** Returns true if this button has a hold behavior, even if the triggering action is not holdable. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Common Button|Getters")
	bool GetRequiresHold() const { return bRequiresHold; }

	/** Change whether this button should have a hold behavior even if the triggering action is not holdable. */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetRequiresHold(bool bInRequiresHold);

	/** Returns required hold time for performing a triggering action. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Common Button|Getters")
	float GetRequiredHoldTime() const { return HoldTime; };

	/** Updates the bAllowDragDrop flag  on the RootButton */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetAllowDragDrop(bool bInAllowDragDrop);

	/** Updates the bIsFocusable flag */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Setters")
	COMMONUI_API void SetIsFocusable(bool bInIsFocusable);

	/** Gets the bIsFocusable flag */
	UFUNCTION(BlueprintCallable, Category = "Common Button|Getters")
	COMMONUI_API bool GetIsFocusable() const;

	/** Returns the dynamic instance of the material being used for this button, if it is using a single material style. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Common Button|Getters")
	COMMONUI_API UMaterialInstanceDynamic* GetSingleMaterialStyleMID() const;

	UFUNCTION(BlueprintCallable, Category = "Common Button|Input")
	COMMONUI_API void SetInputActionProgressMaterial(const FSlateBrush& InProgressMaterialBrush, const FName& InProgressMaterialParam);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetPressedSoundOverride(USoundBase* Sound);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetClickedSoundOverride(USoundBase* Sound);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetHoveredSoundOverride(USoundBase* Sound);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetSelectedPressedSoundOverride(USoundBase* Sound);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetSelectedClickedSoundOverride(USoundBase* Sound);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetSelectedHoveredSoundOverride(USoundBase* Sound);
	
	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetLockedPressedSoundOverride(USoundBase* Sound);
	
	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetLockedClickedSoundOverride(USoundBase* Sound);

	UFUNCTION(BlueprintCallable, Category = "Common Button|Sound")
	COMMONUI_API void SetLockedHoveredSoundOverride(USoundBase* Sound);

	DECLARE_EVENT(UCommonButtonBase, FCommonButtonEvent);
	FCommonButtonEvent& OnClicked() const { return OnClickedEvent; }
	FCommonButtonEvent& OnDoubleClicked() const { return OnDoubleClickedEvent; }
	FCommonButtonEvent& OnPressed() const { return OnPressedEvent; }
	FCommonButtonEvent& OnReleased() const { return OnReleasedEvent; }
	FCommonButtonEvent& OnHovered() const { return OnHoveredEvent; }
	FCommonButtonEvent& OnUnhovered() const { return OnUnhoveredEvent; }
	FCommonButtonEvent& OnFocusReceived() const { return OnFocusReceivedEvent; }
	FCommonButtonEvent& OnFocusLost() const { return OnFocusLostEvent; }
	FCommonButtonEvent& OnLockClicked() const { return OnLockClickedEvent; }
	FCommonButtonEvent& OnLockDoubleClicked() const { return OnLockDoubleClickedEvent; }

	FOnDragDetected& OnCommonButtonDragDetected() const { return OnDragDetectedEvent; }
	FOnDragEnter& OnCommonButtonDragEnter() const { return OnDragEnterEvent; }
	FOnDragLeave& OnCommonButtonDragLeave() const { return OnDragLeaveEvent; }
	FOnDragOver& OnCommonButtonDragOver() const { return OnDragOverEvent; }
	FOnDrop& OnCommonButtonDrop() const { return OnDropEvent; }

	UPROPERTY(BlueprintReadOnly, FieldNotify, Category= "Common Button")
	FWidgetEventField ClickEvent;

	DECLARE_EVENT_OneParam(UCommonButtonBase, FOnIsSelectedChanged, bool);
	FOnIsSelectedChanged& OnIsSelectedChanged() const { return OnIsSelectedChangedEvent; }

protected:
	bool IsPersistentBinding() const { return bIsPersistentBinding; }
	ECommonInputMode GetInputModeOverride() const { return InputModeOverride; }
	
	COMMONUI_API virtual UCommonButtonInternalBase* ConstructInternalButton();

	COMMONUI_API virtual void OnWidgetRebuilt();
	COMMONUI_API virtual void PostLoad() override;
	COMMONUI_API virtual void SynchronizeProperties() override;
	COMMONUI_API virtual FReply NativeOnFocusReceived(const FGeometry& InGeometry, const FFocusEvent& InFocusEvent) override;

#if WITH_EDITOR
	COMMONUI_API virtual void OnCreationFromPalette() override;
	COMMONUI_API const FText GetPaletteCategory() override;
#endif // WITH_EDITOR

	// In some scenarios, USoundBase* setters are bypassed, so standard setters must also be provided for sound override properties
	COMMONUI_API void SetPressedSlateSoundOverride(const FSlateSound& InPressedSlateSoundOverride);
	COMMONUI_API void SetClickedSlateSoundOverride(const FSlateSound& InClickedSlateSoundOverride);
	COMMONUI_API void SetHoveredSlateSoundOverride(const FSlateSound& InHoveredSlateSoundOverride);
	COMMONUI_API void SetSelectedPressedSlateSoundOverride(const FSlateSound& InSelectedPressedSlateSoundOverride);
	COMMONUI_API void SetSelectedClickedSlateSoundOverride(const FSlateSound& InSelectedClickedSlateSoundOverride);
	COMMONUI_API void SetSelectedHoveredSlateSoundOverride(const FSlateSound& InSelectedHoveredSlateSoundOverride);
	COMMONUI_API void SetLockedPressedSlateSoundOverride(const FSlateSound& InLockedPressedSlateSoundOverride);
	COMMONUI_API void SetLockedClickedSlateSoundOverride(const FSlateSound& InLockedClickedSlateSoundOverride);
	COMMONUI_API void SetLockedHoveredSlateSoundOverride(const FSlateSound& InLockedHoveredSlateSoundOverride);

	/** Helper function to bind to input method change events */
	COMMONUI_API virtual void BindInputMethodChangedDelegate();

	/** Helper function to unbind from input method change events */
	COMMONUI_API virtual void UnbindInputMethodChangedDelegate();

	/** Called via delegate when the input method changes */
	UFUNCTION()
	COMMONUI_API virtual void OnInputMethodChanged(ECommonInputType CurrentInputType);

	/** This should be used as a check before calling NativeOnHovered and NativeOnUnhovered.
	returns false if this button is set to explicitly block hover events on touch.*/
	COMMONUI_API bool ShouldProcessHoverEvent(EHoverEventSource HoverReason);
	
	/** If HoldData is valid, assigns its values to Keyboard and Mouse, Gamepad and Touch, based off the Current Input Type. */
    UFUNCTION()
	COMMONUI_API virtual void UpdateHoldData(ECommonInputType CurrentInputType);

	/** Associates this button at its priority with the given key */
	COMMONUI_API virtual void BindTriggeringInputActionToClick();

	/** Associates this button at its priority with the given key */
	COMMONUI_API virtual void UnbindTriggeringInputActionToClick();

	UFUNCTION()
	COMMONUI_API virtual void HandleTriggeringActionCommited(bool& bPassthrough);
	COMMONUI_API virtual void HandleTriggeringActionCommited();

	// @TODO: DarenC - API decision, consider removing
	COMMONUI_API virtual void ExecuteTriggeredInput();

	/** Helper function to update the associated input action widget, if any, based upon the state of the button */
	COMMONUI_API virtual void UpdateInputActionWidget();

	/** Handler function registered to the underlying button's click. */
	UFUNCTION()
	COMMONUI_API virtual void HandleButtonClicked();

	/** Handler function registered to the underlying button's double click. */
	COMMONUI_API virtual FReply HandleButtonDoubleClicked();

	/** Helper function registered to the underlying button receiving focus */
	UFUNCTION()
	COMMONUI_API virtual void HandleFocusReceived();
	
	/** Helper function registered to the underlying button losing focus */
	UFUNCTION()
	COMMONUI_API virtual void HandleFocusLost();

	/** Helper function registered to the underlying button when pressed */
	UFUNCTION()
	COMMONUI_API virtual void HandleButtonPressed();

	/** Helper function registered to the underlying button when released */
	UFUNCTION()
	COMMONUI_API virtual void HandleButtonReleased();

	/** Helper function registered to the underlying button when drag is detected */
	COMMONUI_API virtual FReply HandleButtonDragDetected(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);

	/** Helper function registered to the underlying button when dragged widget enters */
	COMMONUI_API virtual void HandleButtonDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	/** Helper function registered to the underlying button when dragged widget leaves */
	COMMONUI_API virtual void HandleButtonDragLeave(const FDragDropEvent& DragDropEvent);

	/** Helper function registered to the underlying button when dragged over */
	COMMONUI_API virtual FReply HandleButtonDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	/** Helper function registered to the underlying button when dropped during a drag drop operation */
	COMMONUI_API virtual FReply HandleButtonDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Selected"))
	COMMONUI_API void BP_OnSelected();
	COMMONUI_API virtual void NativeOnSelected(bool bBroadcast);

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Deselected"))
	COMMONUI_API void BP_OnDeselected();
	COMMONUI_API virtual void NativeOnDeselected(bool bBroadcast);

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Hovered"))
	COMMONUI_API void BP_OnHovered();
	COMMONUI_API virtual void NativeOnHovered();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Unhovered"))
	COMMONUI_API void BP_OnUnhovered();
	COMMONUI_API virtual void NativeOnUnhovered();
	
	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Focused"))
	COMMONUI_API void BP_OnFocusReceived();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Unfocused"))
	COMMONUI_API void BP_OnFocusLost();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Locked Changed"))
	COMMONUI_API void BP_OnLockedChanged(bool bIsLocked);
	
	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Lock Clicked"))
	COMMONUI_API void BP_OnLockClicked();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Lock Double Clicked"))
	COMMONUI_API void BP_OnLockDoubleClicked();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Clicked"))
	COMMONUI_API void BP_OnClicked();
	COMMONUI_API virtual void NativeOnClicked();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Double Clicked"))
	COMMONUI_API void BP_OnDoubleClicked();
	COMMONUI_API virtual void NativeOnDoubleClicked();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Input Action Triggered"))
	COMMONUI_API void BP_OnInputActionTriggered();

	/** Unless this is called, we will assume the double click should be converted into a normal click. */
	UFUNCTION(BlueprintCallable, Category = CommonButton)
	COMMONUI_API void StopDoubleClickPropagation();
	
	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Pressed"))
	COMMONUI_API void BP_OnPressed();
	COMMONUI_API virtual void NativeOnPressed();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Released"))
	COMMONUI_API void BP_OnReleased();
	COMMONUI_API virtual void NativeOnReleased();

	COMMONUI_API virtual void NativeOnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, UDragDropOperation*& OutOperation) override;

	COMMONUI_API virtual void NativeOnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation) override;
	
	COMMONUI_API virtual void NativeOnDragLeave(const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation) override;

	COMMONUI_API virtual bool NativeOnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation) override;

	COMMONUI_API virtual bool NativeOnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation) override;

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Enabled"))
	COMMONUI_API void BP_OnEnabled();
	COMMONUI_API virtual void NativeOnEnabled();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Disabled"))
	COMMONUI_API void BP_OnDisabled();
	COMMONUI_API virtual void NativeOnDisabled();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Input Method Changed"))
	COMMONUI_API void BP_OnInputMethodChanged(ECommonInputType CurrentInputType);

	/** Allows derived classes to take action when the current text style has changed */
	UFUNCTION(BlueprintImplementableEvent, meta=(BlueprintProtected="true"), Category = "Common Button")
	COMMONUI_API void OnCurrentTextStyleChanged();
	COMMONUI_API virtual void NativeOnCurrentTextStyleChanged();

	UFUNCTION(BlueprintImplementableEvent, Category = CommonButton, meta = (DisplayName = "On Requires Hold Changed"))
	COMMONUI_API void BP_OnRequiresHoldChanged();

	/** Internal method to allow the selected state to be set regardless of selectability or toggleability */
	UFUNCTION(BlueprintCallable, meta=(BlueprintProtected="true"), Category = "Common Button")
	COMMONUI_API void SetSelectedInternal(bool bInSelected, bool bAllowSound = true, bool bBroadcast = true);

	/** Callback fired when input action datatable row changes */
	UFUNCTION(BlueprintImplementableEvent, Category = "Common Button")
	COMMONUI_API void OnTriggeredInputActionChanged(const FDataTableRowHandle& NewTriggeredAction);

	/** Callback fired when triggered input action datatable row changes */
	UFUNCTION(BlueprintImplementableEvent, Category = "Common Button")
	COMMONUI_API void OnTriggeringInputActionChanged(const FDataTableRowHandle& NewTriggeredAction);

	/** Callback fired when enhanced input action changes */
	UFUNCTION(BlueprintImplementableEvent, Category = "Common Button")
	COMMONUI_API void OnTriggeringEnhancedInputActionChanged(const UInputAction* InInputAction);

	/** Callback fired continously during hold interactions */
	UFUNCTION(BlueprintImplementableEvent, Category = "Common Button")
	COMMONUI_API void OnActionProgress(float HeldPercent);
	
	/**
    * By default, if bRequiresHold is true, the bound Input Action will be forced to require hold as well.
    * The Bound Input action will use the row's hold values regardless of the row's bRequiresHold state.
    */
	UFUNCTION()
	COMMONUI_API virtual bool GetConvertInputActionToHold();
    
    /** Bound to the hold progress of the bound key from the input action */
	UFUNCTION()
	COMMONUI_API virtual void NativeOnActionProgress(float HeldPercent);

	/** Bound to the hold progress not related to the bound key */
	UFUNCTION()
	COMMONUI_API virtual bool NativeOnHoldProgress(float DeltaTime);
	
	/** Bound to the hold progress rollback not related to the bound key */
	UFUNCTION()
	COMMONUI_API virtual bool NativeOnHoldProgressRollback(float DeltaTime);
	
	/** Callback fired when hold events complete */
	UFUNCTION(BlueprintImplementableEvent, Category = "Common Button")
	COMMONUI_API void OnActionComplete();

	UFUNCTION()
	COMMONUI_API virtual void NativeOnActionComplete();
	
	UFUNCTION()
    COMMONUI_API virtual void HoldReset();

	COMMONUI_API virtual bool GetButtonAnalyticInfo(FString& ButtonName, FString& ABTestName, FString& ExtraData) const;

	COMMONUI_API void RefreshDimensions();
	COMMONUI_API void RefreshDimensions(const int32 InMinWidth,const int32 InMinHeight,const int32 InMaxWidth,const int32 InMaxHeight);
	COMMONUI_API virtual void NativeOnMouseEnter( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent ) override;
	COMMONUI_API virtual void NativeOnMouseLeave( const FPointerEvent& InMouseEvent ) override;

	COMMONUI_API virtual void UpdateInputActionWidgetVisibility();

	/** The minimum width of the button (only used if greater than the style's minimum) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Layout, meta = (ClampMin = "0"))
	int32 MinWidth;

	/** The minimum height of the button (only used if greater than the style's minimum) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Layout, meta = (ClampMin = "0"))
	int32 MinHeight;
	
	/** The maximum width of the button (only used if greater than the style's maximum) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Layout, meta = (ClampMin = "0"))
	int32 MaxWidth;
	
	/** The maximum height of the button (only used if greater than the style's maximum) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Layout, meta = (ClampMin = "0"))
	int32 MaxHeight;

	/** References the button style asset that defines a style in multiple sizes */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Style, meta = (ExposeOnSpawn = true))
	TSubclassOf<UCommonButtonStyle> Style;

	/** Whether to hide the input action widget at all times (useful for textless small buttons) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Style)
	bool bHideInputAction;

	/**
	 * Optional override for the sound to play when this button is pressed.
	 * Also used for the Selected and Locked Pressed state if their respective overrides are empty.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Pressed Sound Override"))
	FSlateSound PressedSlateSoundOverride;

	/**
	 * Optional override for the sound to play when this button is clicked (based on Click/Touch/Press methods).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Clicked Sound Override"))
	FSlateSound ClickedSlateSoundOverride;

	/**
	 * Optional override for the sound to play when this button is hovered.
	 * Also used for the Selected and Locked Hovered state if their respective overrides are empty.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Hovered Sound Override"))
	FSlateSound HoveredSlateSoundOverride;

	/** Optional override for the sound to play when this button is pressed while Selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Selected Pressed Sound Override"))
	FSlateSound SelectedPressedSlateSoundOverride;

	/** Optional override for the sound to play when this button is clicked while Selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Selected Clicked Sound Override"))
	FSlateSound SelectedClickedSlateSoundOverride;

	/** Optional override for the sound to play when this button is hovered while Selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Selected Hovered Sound Override"))
	FSlateSound SelectedHoveredSlateSoundOverride;

	/** Optional override for the sound to play when this button is pressed while Locked */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Locked Pressed Sound Override"))
	FSlateSound LockedPressedSlateSoundOverride;

	/** Optional override for the sound to play when this button is clicked while Locked */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Locked Clicked Sound Override"))
	FSlateSound LockedClickedSlateSoundOverride;

	/** Optional override for the sound to play when this button is hovered while Locked */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category = Sound, meta = (DisplayName = "Locked Hovered Sound Override"))
	FSlateSound LockedHoveredSlateSoundOverride;

	/** The type of mouse action required by the user to trigger the button's 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Style, meta = (ExposeOnSpawn = true))
	uint8 bApplyAlphaOnDisable:1;

	/**
	 * True if this button is currently locked.
	 * Locked button can be hovered, focused, and pressed, but the Click event will not go through.
	 * Business logic behind it will not be executed. Designed for progressive disclosure
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Locked, meta = (ExposeOnSpawn = true))
	uint8 bLocked:1;
	
	/** True if the button supports being in a "selected" state, which will update the style accordingly */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Selection, meta = (ExposeOnSpawn = true))
	uint8 bSelectable:1;
	
	/** If true, the button will be selected when it receives focus. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Selection, meta = (ExposeOnSpawn = true, EditCondition = "bSelectable"))
	uint8 bShouldSelectUponReceivingFocus:1;

	/** If true, the button may be clicked while selected. Otherwise, interaction is disabled in the selected state. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Selection, meta = (ExposeOnSpawn = true, EditCondition = "bSelectable"))
	uint8 bInteractableWhenSelected:1;

	/** True if the button can be deselected by clicking it when selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Selection, meta = (ExposeOnSpawn = true, EditCondition = "bSelectable"))
	uint8 bToggleable:1;
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Selection, meta = (ExposeOnSpawn = true, EditCondition = "bSelectable"))
	uint8 bTriggerClickedAfterSelection:1;

	/** True if the input action should be displayed when the button is not interactable */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (ExposeOnSpawn = true))
	uint8 bDisplayInputActionWhenNotInteractable:1;

	/** True if the input action should be hidden while the user is using a keyboard */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (ExposeOnSpawn = true))
	uint8 bHideInputActionWithKeyboard:1;

	/** True if this button should use the default fallback input action (bool is useful for buttons that shouldn't because they are never directly hit via controller) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (ExposeOnSpawn = true))
	uint8 bShouldUseFallbackDefaultInputAction:1;
	
	/** True if this button should have a press and hold behavior, triggering the click when the specified hold time is met */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|Hold", meta = (ExposeOnSpawn = true))
	uint8 bRequiresHold:1;

	/** Press and Hold values used for Keyboard and Mouse, Gamepad and Touch, depending on the current input type */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|Hold", meta = (EditCondition="bRequiresHold", ExposeOnSpawn = true))
	TSubclassOf<UCommonUIHoldData> HoldData;
	
	/** True if this button should play the hover effect when pressed by a touch input */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, AdvancedDisplay, meta = (EditCondition="IsHoverSimulationOnTouchAvailable()", EditConditionHides))
	bool bSimulateHoverOnTouchInput = true;

private:

	/** True if this button is currently selected */
	uint8 bSelected:1;

	/** True if this button is currently enabled */
	uint8 bButtonEnabled:1;

	/** True if interaction with this button is currently enabled */
	uint8 bInteractionEnabled:1;

public:
	/** The type of mouse action required by the user to trigger the button's 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (ExposeOnSpawn = true))
	TEnumAsByte<EButtonClickMethod::Type> ClickMethod;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input)
	TEnumAsByte<EButtonTouchMethod::Type> TouchMethod;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input)
	TEnumAsByte<EButtonPressMethod::Type> PressMethod;

	/** 
	 * This is the priority for the TriggeringInputAction.  The first, HIGHEST PRIORITY widget will handle the input action, and no other widgets will be considered.  
	 * Additionally, no inputs with a priority below the current ActivatablePanel's Input Priority value will even be considered! 
	 * 
	 * @TODO: This is part of legacy CommonUI and should be removed
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (ExposeOnSpawn = true))
	int32 InputPriority;

	/** 
	 *	The input action that is bound to this button. The common input manager will trigger this button to 
	 *	click if the action was pressed 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (ExposeOnSpawn = true, RowType = "/Script/CommonUI.CommonInputActionDataBase"))
	FDataTableRowHandle TriggeringInputAction;

	/** 
	 *	The enhanced input action that is bound to this button. The common input manager will trigger this button to 
	 *	click if the action was pressed 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	TObjectPtr<UInputAction> TriggeringEnhancedInputAction;

	/**
	 * The input action that can be visualized as well as triggered when the user
	 * clicks the button.
	 * 
	 * @TODO: This is part of legacy CommonUI and should be removed
	 */
	FDataTableRowHandle TriggeredInputAction;

#if WITH_EDITORONLY_DATA
	/** Used to track widgets that were created before changing the default style pointer to null */
	UPROPERTY()
	bool bStyleNoLongerNeedsConversion;
#endif

	/** If this button is currently in focus, and is disabled, hidden, or collapsed then focus will be routed to the next available widget */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input)
	uint8 bNavigateToNextWidgetOnDisable : 1;

protected:
	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Selected Changed"))
	FCommonSelectedStateChangedBase OnSelectedChangedBase;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Clicked"))
	FCommonButtonBaseClicked OnButtonBaseClicked;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Double Clicked"))
	FCommonButtonBaseClicked OnButtonBaseDoubleClicked;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Hovered"))
	FCommonButtonBaseClicked OnButtonBaseHovered;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Unhovered"))
	FCommonButtonBaseClicked OnButtonBaseUnhovered;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Focused"))
	FCommonButtonBaseClicked OnButtonBaseFocused;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Unfocused"))
	FCommonButtonBaseClicked OnButtonBaseUnfocused;
	
	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Lock Clicked"))
	FCommonButtonBaseClicked OnButtonBaseLockClicked;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Lock Double Clicked"))
	FCommonButtonBaseClicked OnButtonBaseLockDoubleClicked;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Selected"))
	FCommonButtonBaseClicked OnButtonBaseSelected;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Unselected"))
	FCommonButtonBaseClicked OnButtonBaseUnselected;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Drag Detected"))
	FOnButtonBaseGeoOperationDynamic OnButtonBaseDragDetected;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Drag Enter"))
	FOnButtonBaseGeoOperationDynamic OnButtonBaseDragEnter;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Drag Leave"))
	FOnButtonBaseOperationDynamic OnButtonBaseDragLeave;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Drag Over"))
	FOnButtonBaseGeoOperationDynamic OnButtonBaseDragOver;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true, DisplayName = "On Drag Drop"))
	FOnButtonBaseGeoOperationDynamic OnButtonBaseDrop;

	FUIActionBindingHandle TriggeringBindingHandle;
	
	/** Press and hold time in seconds */
	float HoldTime;
	
    /**
    * Time (in seconds) for hold progress to go from 1.0 (completed) to 0.0.
    * Used when the press and hold is interrupted.
    * If set to 0, there will be no rollback and the hold progress will reset immediately.
    */
    float HoldRollbackTime;
	
    /** Current hold time for this button */
    float CurrentHoldTime;

	/** Current hold progress % for this button */
	float CurrentHoldProgress;
	
    /** Handle for ticker spawned for press and hold */
    FTSTicker::FDelegateHandle HoldTickerHandle;
	
    /** Handle for ticker spawned for button hold rollback */
    FTSTicker::FDelegateHandle HoldProgressRollbackTickerHandle;

private:
	friend class UCommonButtonGroupBase;
	template <typename> friend class SCommonButtonTableRow;

	/**
	 * DANGER! Be very, very careful with this. Unless you absolutely know what you're doing, this is not the property you're looking for.
	 *
	 * True to register the action bound to this button as a "persistent" binding. False (default) will register a standard activation-based binding.
	 * A persistent binding ignores the standard ruleset for UI input routing - the binding will be live immediately upon construction of the button.
	 */
	UPROPERTY(EditAnywhere, Category = Input, AdvancedDisplay)
	bool bIsPersistentBinding = false;

	//Set this to Game for special cases where an input action needs to be set for an in-game button.
	UPROPERTY(EditAnywhere, Category = Input, AdvancedDisplay)
	ECommonInputMode InputModeOverride = ECommonInputMode::Menu;

	UFUNCTION()
	bool IsHoverSimulationOnTouchAvailable() const;
	
	void BuildStyles();
	void SetButtonStyle();

	/** Enables this button (called in SetIsEnabled override) */
	void EnableButton();

	/** Disables this button (called in SetIsEnabled override) */
	void DisableButton();

	void HandleImplicitFocusLost();

	FText EnabledTooltipText;
	FText DisabledTooltipText;

	/** The dynamic material instance of the material set by the single material style, if specified. */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> SingleMaterialStyleMID;

	/** Internally managed and applied style to use when not selected */
	UPROPERTY()
	FButtonStyle NormalStyle;
	
	/** Internally managed and applied style to use when selected */
	UPROPERTY()
	FButtonStyle SelectedStyle;

	/** Internally managed and applied style to use when disabled */
	UPROPERTY()
	FButtonStyle DisabledStyle;

	/** Internally managed and applied style to use when locked */
	UPROPERTY()
	FButtonStyle LockedStyle;

	UPROPERTY(Transient)
	uint32 bStopDoubleClickPropagation : 1;

	/**
	 * The actual UButton that we wrap this user widget into. 
	 * Allows us to get user widget customization and built-in button functionality. 
	 */
	TWeakObjectPtr<class UCommonButtonInternalBase> RootButton;

	mutable FCommonButtonEvent OnClickedEvent;
	mutable FCommonButtonEvent OnDoubleClickedEvent;
	mutable FCommonButtonEvent OnPressedEvent;
	mutable FCommonButtonEvent OnReleasedEvent;
	mutable FCommonButtonEvent OnHoveredEvent;
	mutable FCommonButtonEvent OnUnhoveredEvent;
	mutable FCommonButtonEvent OnFocusReceivedEvent;
	mutable FCommonButtonEvent OnFocusLostEvent;
	mutable FCommonButtonEvent OnLockClickedEvent;
	mutable FCommonButtonEvent OnLockDoubleClickedEvent;
	mutable FOnDragDetected OnDragDetectedEvent;
	mutable FOnDragEnter OnDragEnterEvent;
	mutable FOnDragLeave OnDragLeaveEvent;
	mutable FOnDragOver OnDragOverEvent;
	mutable FOnDrop OnDropEvent;

	mutable FOnIsSelectedChanged OnIsSelectedChangedEvent;

protected:
	/**
	 * Optionally bound widget for visualization behavior of an input action;
	 * NOTE: If specified, will visualize according to the following algorithm:
	 * If TriggeringEnhancedInputAction is specified, visualize it else:
	 * If TriggeringInputAction is specified, visualize it else:
	 * If TriggeredInputAction is specified, visualize it else:
	 * Visualize the default click action while hovered
	 */
	UPROPERTY(BlueprintReadOnly, Category = Input, meta = (BindWidget, OptionalWidget = true, AllowPrivateAccess = true))
	TObjectPtr<UCommonActionWidget> InputActionWidget;
};

UCLASS(Transient)
class UWidgetLockedStateRegistration : public UWidgetBinaryStateRegistration
{
	GENERATED_BODY()

public:

	/** Post-load initialized bit corresponding to this binary state */
	COMMONUI_API static inline FWidgetStateBitfield Bit;

	static const inline FName StateName = FName("Locked");

	//~ Begin UWidgetBinaryStateRegistration Interface.
	COMMONUI_API virtual FName GetStateName() const override;
	COMMONUI_API virtual bool GetRegisteredWidgetState(const UWidget* InWidget) const override;
	//~ End UWidgetBinaryStateRegistration Interface

protected:
	friend UWidgetStateSettings;

	//~ Begin UWidgetBinaryStateRegistration Interface.
	COMMONUI_API virtual void InitializeStaticBitfields() const override;
	//~ End UWidgetBinaryStateRegistration Interface
};
