// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Border.h"
#include "CommonCustomNavigation.generated.h"

#define UE_API COMMONUI_API

/**
 * Exposes a bindable event that can be used to stomp default border navigation with custom behaviors.
 */
UCLASS(MinimalAPI, Config = CommonUI, DefaultConfig, ClassGroup = UI, meta = (Category = "Common UI", DisplayName = "Common Custom Navigation"))
class UCommonCustomNavigation : public UBorder
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FOnCustomNavigationEvent, EUINavigation, NavigationType);

	/** Return true if the Navigation has been handled */
	UPROPERTY(EditAnywhere, Category = Events, meta = (IsBindableEvent = "True"))
	FOnCustomNavigationEvent OnNavigationEvent;

public:

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	//~ Begin UWidget Interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

protected:
	
	UE_API bool OnNavigation(EUINavigation NavigationType);

};

#undef UE_API
