// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonTextBlock.h"
#include "CommonDateTimeTextBlock.generated.h"

#define UE_API COMMONUI_API

UCLASS(MinimalAPI, BlueprintType)
class UCommonDateTimeTextBlock : public UCommonTextBlock
{
	GENERATED_BODY()

public:
	UE_API UCommonDateTimeTextBlock(const FObjectInitializer& ObjectInitializer);

	DECLARE_EVENT(UCommonDateTimeTextBlock, FOnTimeCountDownCompletion);
	FOnTimeCountDownCompletion& OnTimeCountDownCompletion() const { return OnTimeCountDownCompletionEvent; }

#if WITH_EDITOR
	UE_API const FText GetPaletteCategory() override;
#endif // WITH_EDITOR

	UE_API virtual void SynchronizeProperties() override;

	UFUNCTION(BlueprintCallable, Category = "DateTime Text Block")
	UE_API void SetDateTimeValue(const FDateTime InDateTime, bool bShowAsCountdown, float InRefreshDelay = 1.0f);

	UFUNCTION(BlueprintCallable, Category = "DateTime Text Block")
	UE_API void SetTimespanValue(const FTimespan InTimespan);

	UFUNCTION(BlueprintCallable, Category = "DateTime Text Block")
	UE_API void SetCountDownCompletionText(const FText InCompletionText);

	UFUNCTION(BlueprintCallable, Category = "DateTime Text Block")
	UE_API FDateTime GetDateTime() const;

protected:
	
	UE_API void UpdateUnderlyingText();

	UE_API virtual FDateTime GetTimespanStartingTime() const;
	UE_API virtual bool ShouldClearTimer(const FTimespan& TimeRemaining) const;

	UE_API virtual TOptional<FText> FormatTimespan(const FTimespan& InTimespan) const;
	UE_API virtual TOptional<FText> FormatDateTime(const FDateTime& InDateTime) const;

	int32 GetLastDaysCount() const { return LastDaysCount; }
	int32 GetLastHoursCount() const { return LastHoursCount; }

	/*
	* Supplies a custom timespan format to use if desired
	* Supported arguments include {Days}, {Hours}, {Minutes}, and {Seconds}
	*/
	UPROPERTY(EditAnywhere, Category = "Custom Timespan")
	FText CustomTimespanFormat;

	/*
	* If the custom timespan should use a leading zero for values, ie "02"
	*/
	UPROPERTY(EditAnywhere, Category = "Custom Timespan")
	bool bCustomTimespanLeadingZeros = false;

private:

	// Timer handle for timer-based ticking based on InterpolationUpdateInterval.
	FTimerHandle TimerTickHandle;
	float LastTimerTickTime;

	FDateTime DateTime;
	bool bShowAsCountdown;
	int32 LastDaysCount;
	int32 LastHoursCount;

	bool bUseCountdownCompletionText;
	FText CountdownCompletionText;

	mutable FOnTimeCountDownCompletion OnTimeCountDownCompletionEvent;

	UE_API void TimerTick();
};

#undef UE_API
