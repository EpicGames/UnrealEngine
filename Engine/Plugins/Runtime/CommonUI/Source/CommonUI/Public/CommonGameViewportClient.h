// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/GameViewportClient.h"
#include "CommonGameViewportClient.generated.h"

#define UE_API COMMONUI_API

class FReply;

DECLARE_DELEGATE_FourParams(FOnRerouteInputDelegate, FInputDeviceId /* InputDeviceId */, FKey /* Key */, EInputEvent /* EventType */, FReply& /* Reply */);
DECLARE_DELEGATE_FourParams(FOnRerouteAxisDelegate, FInputDeviceId /* InputDeviceId */, FKey /* Key */, float /* Delta */, FReply& /* Reply */);

UE_DEPRECATED(5.8, "Use the version which takes an FTouchId instead (FTouchRerouteDelegate).")
DECLARE_DELEGATE_FiveParams(FOnRerouteTouchInputDelegate, FInputDeviceId /* Deviceid */, uint32 /* TouchId */, ETouchType::Type /* TouchType */, const FVector2D& /* TouchLocation */, FReply& /* Reply */);

DECLARE_DELEGATE_FourParams(FTouchRerouteDelegate, FTouchId /* TouchId */, ETouchType::Type /* TouchType */, const FVector2D& /* TouchLocation */, FReply& /* Reply */);

/**
* CommonUI Viewport to reroute input to UI first. Needed to allow CommonUI to route / handle inputs.
*/
UCLASS(MinimalAPI, Within = Engine, transient, config = Engine)
class UCommonGameViewportClient : public UGameViewportClient
{
	GENERATED_BODY()

public:

	UE_API UCommonGameViewportClient(FVTableHelper& Helper);

	// UGameViewportClient interface begin
	UE_API virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	UE_API virtual bool InputAxis(const FInputKeyEventArgs& Args) override;
	UE_API virtual bool InputTouch(FViewport* const InViewport, const FTouchId TouchId, const ETouchType::Type Type, const FVector2D& TouchLocation, const float Force, const uint64 Timestamp) override;
	UE_API virtual void MouseMove(FViewport* InViewport, int32 X, int32 Y) override;
	UE_API virtual void CapturedMouseMove(FViewport* InViewport, int32 X, int32 Y) override;
	UE_API virtual TOptional<TSharedRef<SWidget>> MapCursor(FViewport* Viewport, const FCursorReply& CursorReply) override;
	// UGameViewportClient interface end

	// When true, MapCursor redirects any non-None cursor type to Custom so the virtual pointer widget is always displayed.
	void SetUseVirtualPointerCursor(bool bEnabled) { bUseVirtualPointerCursor = bEnabled; }

	FOnRerouteInputDelegate& OnRerouteInput() { return RerouteInput; }
	FOnRerouteAxisDelegate& OnRerouteAxis() { return RerouteAxis; }

	UE_DEPRECATED(5.8, "Use GetRerouteTouchRegistration() instead.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnRerouteTouchInputDelegate& OnRerouteTouchInput()
	{
		return RerouteTouchInput;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FTouchRerouteDelegate::RegistrationType& GetRerouteTouchRegistration()
	{
		return RerouteTouchDelegate;
	}

	FOnRerouteInputDelegate& OnRerouteBlockedInput() { return RerouteBlockedInput; }

	/** Default Handler for Key input. */
	UE_API virtual void HandleRerouteInput(FInputDeviceId DeviceId, FKey Key, EInputEvent EventType, FReply& Reply);

	/** Default Handler for Axis input. */
	UE_API virtual void HandleRerouteAxis(FInputDeviceId DeviceId, FKey Key, float Delta, FReply& Reply);

	UE_DEPRECATED(5.8, "Use the version which takes a FTouchId instead.")
	UE_API virtual void HandleRerouteTouch(FInputDeviceId DeviceId, uint32 TouchId, ETouchType::Type TouchType, const FVector2D& TouchLocation, FReply& Reply) final;

	/** Default Handler for Touch input. */
	UE_API virtual void HandleRerouteTouch(FTouchId TouchId, ETouchType::Type TouchType, const FVector2D& TouchLocation, FReply& Reply);

protected:

	FTouchRerouteDelegate& GetRerouteTouchDelegate()
	{
		return RerouteTouchDelegate;
	}

	/** Console window & fullscreen shortcut have higher priority than UI */
	UE_API virtual bool IsKeyPriorityAboveUI(const FInputKeyEventArgs& EventArgs);

	FOnRerouteInputDelegate RerouteInput;
	FOnRerouteAxisDelegate RerouteAxis;
	
	UE_DEPRECATED(5.8, "Use GetRerouteTouchDelegate() instead.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FOnRerouteTouchInputDelegate RerouteTouchInput;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FOnRerouteInputDelegate RerouteBlockedInput;

private:

	bool bUseVirtualPointerCursor = false;

	FTouchRerouteDelegate RerouteTouchDelegate;
};

#undef UE_API
