// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonBorder.h"
#include "GameplayTagContainer.h"
#include "CommonHardwareVisibilityBorder.generated.h"

#define UE_API COMMONUI_API

enum class ESlateVisibility : uint8;

class UCommonUIVisibilitySubsystem;

/**
 * A container that controls visibility based on Platform, Input 
 */
UCLASS(MinimalAPI)
class UCommonHardwareVisibilityBorder : public UCommonBorder
{
	GENERATED_UCLASS_BODY()

public:
	

protected:

	UPROPERTY(EditAnywhere, Category = "Visibility", meta=(Categories="Input,Platform.Trait"))
	FGameplayTagQuery VisibilityQuery;

	UPROPERTY(EditAnywhere, Category = "Visibility")
	ESlateVisibility VisibleType;
	
	UPROPERTY(EditAnywhere, Category = "Visibility")
	ESlateVisibility HiddenType;

protected:
	// Begin UWidget
	UE_API virtual void OnWidgetRebuilt() override;
	// End UWidget

	UE_API void UpdateVisibility(UCommonUIVisibilitySubsystem* VisSystem = nullptr);

	UE_API void ListenToInputMethodChanged();

	UE_API void HandleInputMethodChanged(UCommonUIVisibilitySubsystem*);
};

#undef UE_API
