// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ScrollBox.h"
#include "CommonHierarchicalScrollBox.generated.h"

#define UE_API COMMONUI_API

/**
 * An arbitrary scrollable collection of widgets.  Great for presenting 10-100 widgets in a list.  Doesn't support virtualization.
 */
UCLASS(MinimalAPI)
class UCommonHierarchicalScrollBox : public UScrollBox
{
	GENERATED_UCLASS_BODY()

protected:
	//~ Begin UWidget Interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface
};

#undef UE_API
