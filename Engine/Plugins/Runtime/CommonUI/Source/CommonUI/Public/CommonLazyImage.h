// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonLoadGuard.h"
#include "Components/Image.h"

#include "CommonLazyImage.generated.h"

#define UE_API COMMONUI_API

class UTexture;

class UCommonMcpItemDefinition;

/**
 * A special Image widget that can show unloaded images and takes care of the loading for you!
 * 
 * UCommonLazyImage is another wrapper for SLoadGuard, but it only handles image loading and 
 * a throbber during loading.
 * 
 * If this class changes to show any text, by default it will have CoreStyle styling
 */
UCLASS(MinimalAPI)
class UCommonLazyImage : public UImage
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual void SetBrush(const FSlateBrush& InBrush) override;
	UE_API virtual void SetBrushFromAsset(USlateBrushAsset* Asset) override;
	UE_API virtual void SetBrushFromTexture(UTexture2D* Texture, bool bMatchSize = false) override;
	UE_API virtual void SetBrushFromTextureDynamic(UTexture2DDynamic* Texture, bool bMatchSize = false) override;
	UE_API virtual void SetBrushFromMaterial(UMaterialInterface* Material) override;

	/** Set the brush from a lazy texture asset pointer - will load the texture as needed. */
	UFUNCTION(BlueprintCallable, Category = LazyImage)
	UE_API void SetBrushFromLazyTexture(const TSoftObjectPtr<UTexture2D>& LazyTexture, bool bMatchSize = false);

	/** Set the brush from a lazy material asset pointer - will load the material as needed. */
	UFUNCTION(BlueprintCallable, Category = LazyImage)
	UE_API void SetBrushFromLazyMaterial(const TSoftObjectPtr<UMaterialInterface>& LazyMaterial);
	
	/** Set the brush from a string asset ref only - expects the referenced asset to be a texture or material. */
	UFUNCTION(BlueprintCallable, Category = LazyImage)
	UE_API void SetBrushFromLazyDisplayAsset(const TSoftObjectPtr<UObject>& LazyObject, bool bMatchTextureSize = false);

	UFUNCTION(BlueprintCallable, Category = LazyImage)
	UE_API bool IsLoading() const;

	/**
	 * Establishes the name of the texture parameter on the currently applied brush material to which textures should be applied.
	 * Does nothing if the current brush resource object is not a material.
	 *
	 * Note: that this is cleared out automatically if/when a new material is established on the brush.
	 * You must call this function again after doing so if the new material has a texture param.
	 */
	UFUNCTION(BlueprintCallable, Category = LazyImage)
	UE_API void SetMaterialTextureParamName(FName TextureParamName);

	FOnLoadGuardStateChangedEvent& OnLoadingStateChanged() { return OnLoadingStateChangedEvent; }

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override final;
	UE_API virtual void OnWidgetRebuilt() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void SynchronizeProperties() override;

	UE_API virtual void CancelImageStreaming() override;
	UE_API virtual void OnImageStreamingStarted(TSoftObjectPtr<UObject> SoftObject) override;
	UE_API virtual void OnImageStreamingComplete(TSoftObjectPtr<UObject> LoadedSoftObject) override;

	UE_API virtual TSharedRef<SWidget> RebuildImageWidget();

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif	

	UE_API void SetIsLoading(bool bIsLoading);
	UE_API void ShowDefaultImage();

private:
	UE_API void HandleLoadGuardStateChanged(bool bIsLoading);

	UE_API void SetBrushObjectInternal(UMaterialInterface* Material);
	UE_API void SetBrushObjectInternal(UTexture* Texture, bool bMatchSize = false);

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = LoadPreview)
	bool bShowLoading = false;
#endif

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush LoadingBackgroundBrush;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush LoadingThrobberBrush;

	/** 
	 * If this image uses a material that a texture should be applied to, this is the name of the material param to use.
	 * I.e. if this property is not blank, the resource object of our brush is a material, and we are given a lazy texture, that texture
	 * will be assigned to the texture param on the material instead of replacing the material outright on the brush.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Appearance)
	FName MaterialTextureParamName;

	UPROPERTY(BlueprintAssignable, Category = LazyImage, meta = (DisplayName = "On Loading State Changed", ScriptName = "OnLoadingStateChanged"))
	FOnLoadGuardStateChangedDynamic BP_OnLoadingStateChanged;

	TSharedPtr<SLoadGuard> MyLoadGuard;
	FOnLoadGuardStateChangedEvent OnLoadingStateChangedEvent;
};

#undef UE_API
