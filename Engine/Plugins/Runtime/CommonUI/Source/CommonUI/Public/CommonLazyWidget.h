// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "CommonLoadGuard.h"
#include "Engine/StreamableManager.h"

#include "CommonLazyWidget.generated.h"

#define UE_API COMMONUI_API

class UCommonMcpItemDefinition;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnLazyContentChangedEvent, UUserWidget*);

/**
 * A widget that can async load and create an instance of a UserWidget.
 */
UCLASS(MinimalAPI)
class UCommonLazyWidget : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnWidgetCreated, UUserWidget*, Widget);

	/** Loads and creates an instance of SoftWidget. */
	UFUNCTION(BlueprintCallable, Category = LazyContent)
	UE_API void SetLazyContent(const TSoftClassPtr<UUserWidget> SoftWidget);

	/** Loads and creates an instance of SoftWidget. */
	UFUNCTION(BlueprintCallable, Category = LazyContent)
	UE_API void SetLazyContentWithCallback(const TSoftClassPtr<UUserWidget> SoftWidget, const FOnWidgetCreated& OnCreatedCallback);

	/** Loads and creates an instance of the WidgetClass property. */
	UFUNCTION(BlueprintCallable, Category = LazyContent)
	UE_API void LoadLazyContent();

	template<typename TWidget = UUserWidget>
	void LoadLazyContent(TFunction<void(TWidget&)>&& InitInstanceFunc)
	{
		if (!WidgetClass.IsNull())
		{
			SetLazyContentInternal(WidgetClass, MoveTemp(InitInstanceFunc));
		}
	}

	/** Gets the attached Content which was instanced from an async loaded TSoftClassPtr. */
	UFUNCTION(BlueprintCallable, Category = LazyContent)
	UUserWidget* GetContent() const { return Content; }

	template <class TContent = UUserWidget>
	TContent* GetContent() const
	{
		return Cast<TContent>(GetContent());
	}

	UFUNCTION(BlueprintCallable, Category = LazyContent)
	UE_API bool IsLoading() const;

	FOnLazyContentChangedEvent& OnContentChanged() { return OnContentChangedEvent; }
	FOnLoadGuardStateChangedEvent& OnLoadingStateChanged() { return OnLoadingStateChangedEvent; }

protected:
	template <typename TWidget = UUserWidget>
	void SetLazyContentInternal(const TSoftClassPtr<UUserWidget> SoftWidget, TFunction<void(TWidget&)>&& InitInstanceFunc)
	{
		if (SoftWidget.IsNull())
		{
			CancelStreaming();
			SetLoadedContent(nullptr);
			return;
		}

		TWeakObjectPtr<UCommonLazyWidget> WeakThis(this);

		RequestAsyncLoad(SoftWidget,
						 [WeakThis, SoftWidget, InitInstanceFunc = MoveTemp(InitInstanceFunc)]() {
			if (ThisClass* StrongThis = WeakThis.Get())
			{
				if (ensureMsgf(SoftWidget.Get(), TEXT("Failed to load %s"), *SoftWidget.ToSoftObjectPath().ToString()))
				{
					// Don't reload the class if we're already this class.
					if (StrongThis->Content && StrongThis->Content->GetClass() == SoftWidget.Get())
					{
						return;
					}

					TWidget* UserWidget = CreateWidget<TWidget>(StrongThis, SoftWidget.Get());

					if (InitInstanceFunc && UserWidget && !StrongThis->IsDesignTime())
					{
						InitInstanceFunc(*UserWidget);
					}

					StrongThis->SetLoadedContent(UserWidget);
				}
			}
		});
	}

	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void OnWidgetRebuilt() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void SynchronizeProperties() override;

	UE_API void SetForceShowSpinner(bool bShowLoading);

	UE_API void CancelStreaming();
	UE_API void OnStreamingStarted(TSoftClassPtr<UObject> SoftObject);
	UE_API void OnStreamingComplete(TSoftClassPtr<UObject> LoadedSoftObject);

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif	

private:
	UE_API void SetLoadedContent(UUserWidget* InContent);
	UE_API void RequestAsyncLoad(TSoftClassPtr<UObject> SoftObject, TFunction<void()>&& Callback);
	UE_API void RequestAsyncLoad(TSoftClassPtr<UObject> SoftObject, FStreamableDelegate DelegateToCall);
	UE_API void HandleLoadGuardStateChanged(bool bIsLoading);

	UPROPERTY(EditAnywhere, Category = LazyWidget)
	TSoftClassPtr<UUserWidget> WidgetClass;

	/** The loading throbber brush */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush LoadingThrobberBrush;

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush LoadingBackgroundBrush;

	UPROPERTY(Transient)
	TObjectPtr<UUserWidget> Content;

	TSharedPtr<FStreamableHandle> StreamingHandle;
	FSoftObjectPath StreamingObjectPath;

	UPROPERTY(BlueprintAssignable, Category = LazyWidget, meta = (DisplayName = "On Loading State Changed", ScriptName = "OnLoadingStateChanged"))
	FOnLoadGuardStateChangedDynamic BP_OnLoadingStateChanged;

	TSharedPtr<SLoadGuard> MyLoadGuard;
	FOnLoadGuardStateChangedEvent OnLoadingStateChangedEvent;

	FOnLazyContentChangedEvent OnContentChangedEvent;
};

#undef UE_API
