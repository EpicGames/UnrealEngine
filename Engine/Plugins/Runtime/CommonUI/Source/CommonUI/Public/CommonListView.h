// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ListView.h"
#include "Widgets/Views/SListView.h"
#include "CommonButtonBase.h"
#include "Slate/SObjectTableRow.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateUser.h"
#include "CommonListView.generated.h"

#define UE_API COMMONUI_API

class STableViewBase;

//////////////////////////////////////////////////////////////////////////
// SCommonListView
//////////////////////////////////////////////////////////////////////////

template <typename ItemType>
class SCommonListView : public SListView<ItemType>
{
public:
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		if (bScrollToSelectedOnFocus && (InFocusEvent.GetCause() == EFocusCause::Navigation || InFocusEvent.GetCause() == EFocusCause::SetDirectly))
		{
			if (this->HasValidItemsSource() && this->GetItems().Num() > 0)
			{
				typename TListTypeTraits<ItemType>::NullableType ItemNavigatedTo = TListTypeTraits<ItemType>::MakeNullPtr();

				if (this->bEnableProximateEntryNavigation)
				{
					// Resolve spatially via proximate entry. If proximate fails, this falls back
					// to the previously selected item, then to item 0.
					this->TryResolveProximateFocusEntry(ItemNavigatedTo, InFocusEvent);
				}
				else
				{
					// Set selection to the first item in a list if no items are selected.
					// If bReturnFocusToSelection is true find the last selected object and focus on that.
					if (this->GetNumItemsSelected() == 0)
					{
						ItemNavigatedTo = this->GetItems()[0];
					}
					else if (this->bReturnFocusToSelection && TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem))
					{
						ItemNavigatedTo = this->SelectorItem;
					}
				}

				if (TListTypeTraits<ItemType>::IsPtrValid(ItemNavigatedTo))
				{
					ItemType SelectedItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(ItemNavigatedTo);

					TOptional<ItemType> FirstValidItem = this->Private_FindNextSelectableOrNavigable(SelectedItem);
					if (FirstValidItem.IsSet())
					{
						if (this->bSelectItemOnNavigation)
						{
							this->SetSelection(FirstValidItem.GetValue(), ESelectInfo::OnNavigation);
						}
						else
						{
							this->SelectorItem = FirstValidItem.GetValue();
						}

						// One-frame focus flash fix: when focus is set programmatically (SetDirectly)
						// and the target row is already fully visible, push focus immediately so the
						// analog cursor cannot hittest the centered tile during the deferred scroll path.
						bool bFocusedImmediately = false;
						if (this->bEnableProximateEntryNavigation && InFocusEvent.GetCause() == EFocusCause::SetDirectly)
						{
							if (TSharedPtr<ITableRow> Row = this->WidgetFromItem(FirstValidItem.GetValue()))
							{
								const FSlateRect ListRect = this->GetCachedGeometry().GetRenderBoundingRect();
								const FSlateRect ItemRect = Row->AsWidget()->GetCachedGeometry().GetRenderBoundingRect();
								if (ListRect.ContainsRect(ItemRect))
								{
									FSlateApplication::Get().SetUserFocus(InFocusEvent.GetUser(), Row->AsWidget(), EFocusCause::SetDirectly);
									bFocusedImmediately = true;
								}
							}
						}
						if (!bFocusedImmediately)
						{
							this->RequestNavigateToItem(FirstValidItem.GetValue(), InFocusEvent.GetUser());
						}
					}
				}
			}
		}
		bScrollToSelectedOnFocus = true;

		return SListView<ItemType>::OnFocusReceived(MyGeometry, InFocusEvent);
	}

	virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override
	{
		SListView<ItemType>::OnFocusChanging(PreviousFocusPath, NewWidgetPath, InFocusEvent);

		if (!NewWidgetPath.IsValid())
		{
			return;
		}

		const TSharedPtr<SWidget> NewWidget = NewWidgetPath.GetLastWidget();
		if (this->bEnableProximateEntryNavigation && NewWidget.IsValid() && NewWidget.Get() != this && NewWidgetPath.ContainsWidget(this))
		{
			if (!PreviousFocusPath.ContainsWidget(this) || !TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem))
			{
				this->SyncSelectorItemFromFocusPath(NewWidgetPath, InFocusEvent.GetUser());
			}

			// Keep IntraEntrySourceWidget current for intra-entry/cross-entry spatial routing.
			// When focus lands on the row, seed from GetDesiredFocusWidget().
			// When it lands on a sub-widget directly, cache it as-is.
			TSharedPtr<SWidget> RowWidget;
			TSharedPtr<ITableRow> SelectorRow;
			if (TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem))
			{
				const ItemType CurrentItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(this->SelectorItem);
				SelectorRow = this->WidgetFromItem(CurrentItem);
				if (SelectorRow.IsValid())
				{
					RowWidget = SelectorRow->AsWidget();
				}
			}

			if (RowWidget.IsValid() && NewWidget == RowWidget)
			{
				// Row received focus - seed from GetDesiredFocusWidget() for intra-entry spatial routing.
				const UUserWidget* EntryWidget = SelectorRow->GetUserWidget();
				if (EntryWidget && !EntryWidget->IsA<UCommonButtonBase>())
				{
					if (const UWidget* DesiredFocus = EntryWidget->GetDesiredFocusWidget())
					{
						IntraEntrySourceWidget = DesiredFocus->GetCachedWidget();
					}
				}
			}
			else
			{
				// Sub-widget received focus - cache it directly.
				IntraEntrySourceWidget = NewWidget;
			}

			bool bIsOutermostList = true;
			for (int32 i = 0; i < NewWidgetPath.Widgets.Num(); ++i)
			{
				if (&NewWidgetPath.Widgets[i].Widget.Get() == this) break;
				if (NewWidgetPath.Widgets[i].Widget->IsTableView())
				{
					bIsOutermostList = false;
					break;
				}
			}
			if (bIsOutermostList)
			{
				this->ScrollNestedClippedWidgetIntoView(*NewWidget);
			}
		}
		else if (this->bEnableProximateEntryNavigation && NewWidget.IsValid() && NewWidget.Get() == this && InFocusEvent.GetCause() == EFocusCause::Navigation && !PreviousFocusPath.ContainsWidget(this))
		{
			// Focus entered this list from an external widget. The escape block in SListView::OnNavigation
			// only fires for other lists, so no context has been written yet. Infer the navigation
			// direction geometrically and seed the context so OnFocusReceived can find the proximate item.
			if (!FSlateApplication::Get().GetPendingNavigationContext<UE::Slate::Private::FListViewNavigationContext>(InFocusEvent.GetUser()).IsValid())
			{
				if (const TSharedPtr<SWidget> PreviousWidget = PreviousFocusPath.GetLastWidget().Pin())
				{
					const FSlateRect SourceRect = PreviousWidget->GetCachedGeometry().GetRenderBoundingRect();
					const FSlateRect ListRect   = this->GetCachedGeometry().GetRenderBoundingRect();

					EUINavigation InferredType = EUINavigation::Invalid;
					if      (SourceRect.Bottom <= ListRect.Top)    InferredType = EUINavigation::Down;
					else if (SourceRect.Top    >= ListRect.Bottom) InferredType = EUINavigation::Up;
					else if (SourceRect.Right  <= ListRect.Left)   InferredType = EUINavigation::Right;
					else if (SourceRect.Left   >= ListRect.Right)  InferredType = EUINavigation::Left;

					if (InferredType != EUINavigation::Invalid)
					{
						TSharedRef<UE::Slate::Private::FListViewNavigationContext> NavContext =
							MakeShared<UE::Slate::Private::FListViewNavigationContext>();
						NavContext->SourceWidget   = PreviousWidget;
						NavContext->NavigationType = InferredType;
						FSlateApplication::Get().SetPendingNavigationContext(InFocusEvent.GetUser(), NavContext);
					}
				}
			}
		}
	}

	virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override
	{
		if (!this->bEnableProximateEntryNavigation)
		{
			return SListView<ItemType>::OnNavigation(MyGeometry, InNavigationEvent);
		}

		// Snapshot the precise source widget before calling super - SListView::OnNavigation's
		// escape block overwrites the pending context's SourceWidget with the full row; we upgrade it below.
		TSharedPtr<SWidget> SourceWidget = IntraEntrySourceWidget.Pin();
		if (!SourceWidget.IsValid())
		{
			if (const TSharedPtr<FSlateUser> User = FSlateApplication::Get().GetUser(InNavigationEvent.GetUserIndex()))
			{
				SourceWidget = User->GetFocusedWidget();
			}
		}

		// Intra-entry navigation: route focus between focusable widgets within the current entry.
		if (SourceWidget.IsValid() && TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem))
		{
			const ItemType CurrentItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(this->SelectorItem);
			if (const TSharedPtr<ITableRow> CurrentRow = this->WidgetFromItem(CurrentItem))
			{
				const TSharedRef<SWidget> RowWidget = CurrentRow->AsWidget();
				const UUserWidget* EntryWidget = CurrentRow->GetUserWidget();

				if (EntryWidget && !EntryWidget->IsA<UCommonButtonBase>())
				{
					if (const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(SourceWidget.ToSharedRef()))
					{
						const FArrangedWidget SourceArranged(SourceWidget.ToSharedRef(), SourceWidget->GetCachedGeometry());
						const FArrangedWidget BoundaryArranged(RowWidget, RowWidget->GetCachedGeometry());

						if (const TSharedPtr<SWidget> IntraEntryTarget = Window->GetHittestGrid().FindNextFocusableWidget(
							SourceArranged,
							InNavigationEvent.GetNavigationType(),
							FNavigationReply::Stop(),
							BoundaryArranged,
							InNavigationEvent.GetUserIndex()))
						{
							const FSlateRect ListRect = this->GetCachedGeometry().GetRenderBoundingRect();
							const FSlateRect ChildRect = IntraEntryTarget->GetCachedGeometry().GetRenderBoundingRect();
							const bool bFullyVisible = ChildRect.Top >= ListRect.Top && ChildRect.Bottom <= ListRect.Bottom && ChildRect.Left >= ListRect.Left && ChildRect.Right <= ListRect.Right;

							if (bFullyVisible)
							{
								// Route to the intra-entry target without advancing selection.
								IntraEntrySourceWidget.Reset();
								FSlateApplication::Get().SetUserFocus(
									InNavigationEvent.GetUserIndex(),
									IntraEntryTarget.ToSharedRef(),
									EFocusCause::Navigation);
								return FNavigationReply::Explicit(nullptr);
							}
						}
					}
				}
			}
		}

		// No intra-entry target found: let SListView handle navigation.
		const FNavigationReply Reply = SListView<ItemType>::OnNavigation(MyGeometry, InNavigationEvent);

		// Upgrade the pending context's SourceWidget to the precise sub-widget so the receiving
		// list's FindProximateBoundaryItem uses the button rect rather than the full row rect.
		if (const TSharedPtr<UE::Slate::Private::FListViewNavigationContext> NavContext =
			FSlateApplication::Get().GetPendingNavigationContext<UE::Slate::Private::FListViewNavigationContext>(InNavigationEvent.GetUserIndex()))
		{
			if (const TSharedPtr<SWidget> PreciseSource = IntraEntrySourceWidget.Pin())
			{
				NavContext->SourceWidget = PreciseSource;
			}
		}
		IntraEntrySourceWidget.Reset();

		// Cross-entry proximate focus: resolve focus to the nearest focusable widget in the newly selected entry.
		if (Reply.GetBoundaryRule() == EUINavigationRule::Explicit && SourceWidget.IsValid() && TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem))
		{
			// Reset any previously deferred cross-entry proximate focus state before deciding what to do this frame.
			this->PendingScrollContextWidget.Reset();
			this->PendingProximateNavType = EUINavigation::Invalid;

			const ItemType NewItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(this->SelectorItem);
			if (const TSharedPtr<ITableRow> NewRow = this->WidgetFromItem(NewItem))
			{
				const TSharedRef<SWidget> RowWidget = NewRow->AsWidget();
				const UUserWidget* EntryWidget = NewRow->GetUserWidget();

				if (EntryWidget && !EntryWidget->IsA<UCommonButtonBase>())
				{
					if (const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(SourceWidget.ToSharedRef()))
					{
						const FArrangedWidget SourceArranged(SourceWidget.ToSharedRef(), SourceWidget->GetCachedGeometry());
						const FArrangedWidget BoundaryArranged(RowWidget, RowWidget->GetCachedGeometry());

						if (const TSharedPtr<SWidget> ProximateChild = Window->GetHittestGrid().FindNextFocusableWidget(
							SourceArranged,
							InNavigationEvent.GetNavigationType(),
							FNavigationReply::Stop(),
							BoundaryArranged,
							InNavigationEvent.GetUserIndex()))
						{
							const FSlateRect ListRect = this->GetCachedGeometry().GetRenderBoundingRect();
							const FSlateRect ChildRect = ProximateChild->GetCachedGeometry().GetRenderBoundingRect();
							const bool bFullyVisible = ChildRect.Top >= ListRect.Top && ChildRect.Bottom <= ListRect.Bottom && ChildRect.Left >= ListRect.Left && ChildRect.Right <= ListRect.Right;

							if (bFullyVisible)
							{
								FSlateApplication::Get().SetUserFocus(
									InNavigationEvent.GetUserIndex(),
									ProximateChild.ToSharedRef(),
									EFocusCause::Navigation);

								// Suppress the pending RequestNavigateToItem focus override - it always
								// routes to DesiredFocusWidget regardless of navigation direction.
								this->bNavigateOnScrollIntoView = false;
							}
							else
							{
								const FGeometry& ListGeo = this->GetCachedGeometry();
								const FGeometry& ChildGeo = ProximateChild->GetCachedGeometry();
								const float ScrollDelta = this->ComputeScrollDeltaForAlignment(ListGeo, ChildGeo);

								if (ScrollDelta != 0.f)
								{
									this->ScrollBy(ListGeo, ScrollDelta, EAllowOverscroll::No);
								}

								// Focus before clearing the navigate flag so the OnFocusChanging clip-scroll early-returns.
								FSlateApplication::Get().SetUserFocus(
									InNavigationEvent.GetUserIndex(),
									ProximateChild.ToSharedRef(),
									EFocusCause::Navigation);

								this->CancelScrollIntoView();
							}
						}
						else
						{
							// ProximateChild not found (tiles may be culled/not yet generated).
							// Defer cross-entry proximate focus to NotifyItemScrolledIntoView.
							// Consumed by NotifyItemScrolledIntoView
							this->PendingScrollContextWidget = SourceWidget;
							this->PendingProximateNavType = InNavigationEvent.GetNavigationType();
						}
					}
				}
			}
			else
			{
				// Destination entry is off-screen: defer cross-entry proximate focus to NotifyItemScrolledIntoView.
				// Consumed by NotifyItemScrolledIntoView
				this->PendingScrollContextWidget = SourceWidget;
				this->PendingProximateNavType = InNavigationEvent.GetNavigationType();
			}
		}

		return Reply;
	}

	virtual void NotifyItemScrolledIntoView() override
	{
		if (!this->bEnableProximateEntryNavigation)
		{
			SListView<ItemType>::NotifyItemScrolledIntoView();
			return;
		}

		const TSharedPtr<SWidget> Source  = this->PendingScrollContextWidget.Pin();
		const EUINavigation       NavType = this->PendingProximateNavType;
		this->PendingScrollContextWidget.Reset();
		this->PendingProximateNavType = EUINavigation::Invalid;

		if (Source.IsValid() && NavType != EUINavigation::Invalid)
		{
			TSharedRef<UE::Slate::Private::FListViewNavigationContext> NavContext =
				MakeShared<UE::Slate::Private::FListViewNavigationContext>();
			NavContext->SourceWidget    = Source;
			NavContext->NavigationType  = NavType;

			FSlateApplication::Get().SetPendingNavigationContext(this->UserRequestingScrollIntoView, NavContext);
		}

		SListView<ItemType>::NotifyItemScrolledIntoView();
	}

	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override
	{
		SListView<ItemType>::OnFocusLost(InFocusEvent);
		this->bNavigateOnScrollIntoView = false;
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SListView<ItemType>::OnMouseLeave(MouseEvent);

		if (MouseEvent.IsTouchEvent() && this->HasMouseCapture())
		{
			// Regular list views will clear this flag when the pointer leaves the list. To
			// continue scrolling outside the list, we need this to remain on.
			this->bStartedTouchInteraction = true;
		}
	}

	virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		FReply Reply = SListView<ItemType>::OnTouchMoved(MyGeometry, InTouchEvent);

		if (Reply.IsEventHandled() && this->HasMouseCapture())
		{
			bScrollToSelectedOnFocus = false;
			Reply.SetUserFocus(this->AsShared());
		}

		return Reply;
	}

	virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		return SListView<ItemType>::OnTouchEnded(MyGeometry, InTouchEvent);
	}


protected:
	/** Last sub-widget that held focus within an entry. Written by OnFocusChanging;
	 * consumed by OnNavigation intra-entry navigation and the cross-list source precision correction.
	 */
	TWeakPtr<SWidget> IntraEntrySourceWidget;

	bool bScrollToSelectedOnFocus = true;

private:
};

//////////////////////////////////////////////////////////////////////////
// UCommonListView
//////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI)
class UCommonListView : public UListView
{
	GENERATED_BODY()

public:
	UE_API UCommonListView(const FObjectInitializer& ObjectInitializer);
	
	UFUNCTION(BlueprintCallable, Category = ListView)
	UE_API void SetEntrySpacing(float InEntrySpacing);

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	UE_API virtual TSharedRef<STableViewBase> RebuildListWidget() override;
	UE_API virtual UUserWidget& OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable) override;
};

#undef UE_API
