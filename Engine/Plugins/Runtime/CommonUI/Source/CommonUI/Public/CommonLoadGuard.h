// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ContentWidget.h"
#include "Widgets/Accessibility/SlateWidgetAccessibleTypes.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#include "CommonLoadGuard.generated.h"

#define UE_API COMMONUI_API

struct FStreamableHandle;
struct FTextBlockStyle;

class UCommonTextStyle;
class STextBlock;
class SBorder;

//////////////////////////////////////////////////////////////////////////
// SLoadGuard
//////////////////////////////////////////////////////////////////////////

DECLARE_DELEGATE_OneParam(FOnLoadGuardStateChanged, bool);
DECLARE_DELEGATE_OneParam(FOnLoadGuardAssetLoaded, UObject*);

class SLoadGuard : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLoadGuard)
		: _ThrobberHAlign(HAlign_Center)
		, _Throbber(nullptr)
		, _GuardTextStyle(nullptr)
		, _GuardBackgroundBrush(nullptr)
	{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	
		SLATE_ARGUMENT(EHorizontalAlignment, ThrobberHAlign)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, Throbber)
		SLATE_ARGUMENT(FText, GuardText)
		SLATE_ARGUMENT(TSubclassOf<UCommonTextStyle>, GuardTextStyle)
		SLATE_ARGUMENT(const FSlateBrush*, GuardBackgroundBrush)

		SLATE_EVENT(FOnLoadGuardStateChanged, OnLoadingStateChanged)
	SLATE_END_ARGS()

public:
	UE_API SLoadGuard();
	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	UE_API void SetForceShowSpinner(bool bInForceShowSpinner);
	bool IsLoading() const { return bIsShowingSpinner; }
	
	UE_API void SetContent(const TSharedRef<SWidget>& InContent);
	UE_API void SetThrobberHAlign(EHorizontalAlignment InHAlign);
	UE_API void SetThrobber(const TSharedPtr<SWidget>& InThrobber);
	UE_API void SetGuardText(const FText& InText);
	UE_API void SetGuardTextStyle(const FTextBlockStyle& InGuardTextStyle);
	UE_API void SetGuardBackgroundBrush(const FSlateBrush* InGuardBackground);

	/**
	 * Displays the loading spinner until the asset is loaded
	 * Will pass a casted pointer to the given asset in the lambda callback - could be nullptr if you provide an incompatible type or invalid asset.
	 */
	UE_API void GuardAndLoadAsset(const TSoftObjectPtr<UObject>& InLazyAsset, FOnLoadGuardAssetLoaded OnAssetLoaded);

	template <typename ObjectType>
	void GuardAndLoadAsset(const TSoftObjectPtr<UObject>& InLazyAsset, TFunction<void(ObjectType*)> OnAssetLoaded)
	{
		GuardAndLoadAsset(InLazyAsset, FOnLoadGuardAssetLoaded::CreateLambda([OnAssetLoaded](UObject* LoadedObject) {
			OnAssetLoaded(Cast<ObjectType>(LoadedObject));
		}));
	}

	TSharedRef<SBorder> GetContentBorder() const { return ContentBorder.ToSharedRef(); };

private:
	UE_API void UpdateLoadingAppearance();

	TSoftObjectPtr<UObject> LazyAsset;

	TSharedPtr<SBorder> ContentBorder;
	TSharedPtr<SBorder> GuardBorder;
	TSharedPtr<STextBlock> GuardTextBlock;
	SHorizontalBox::FSlot* ThrobberSlot = nullptr;

	FOnLoadGuardStateChanged OnLoadingStateChanged;

	TSharedPtr<FStreamableHandle> StreamingHandle;
	bool bForceShowSpinner = false;
	bool bIsShowingSpinner = false;
};

//////////////////////////////////////////////////////////////////////////
// ULoadGuardSlot
//////////////////////////////////////////////////////////////////////////

/** Virtually identical to a UBorderSlot, but unfortunately that assumes (fairly) that its parent widget is a UBorder. */
UCLASS(MinimalAPI)
class ULoadGuardSlot : public UPanelSlot
{
	GENERATED_BODY()

public:
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	
	UE_API void BuildSlot(TSharedRef<SLoadGuard> InLoadGuard);

	UFUNCTION(BlueprintCallable, Category = "Layout|LoadGuard Slot")
	UE_API void SetPadding(FMargin InPadding);

	UFUNCTION(BlueprintCallable, Category = "Layout|LoadGuard Slot")
	UE_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UFUNCTION(BlueprintCallable, Category = "Layout|LoadGuard Slot")
	UE_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

private:
	UPROPERTY(EditAnywhere, Category = "Layout|LoadGuard Slot")
	FMargin Padding;

	UPROPERTY(EditAnywhere, Category = "Layout|LoadGuard Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = HAlign_Fill;

	UPROPERTY(EditAnywhere, Category = "Layout|LoadGuard Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = VAlign_Fill;

	TWeakPtr<SLoadGuard> LoadGuard;
	friend class UCommonLoadGuard;
};


//////////////////////////////////////////////////////////////////////////
// ULoadGuard
//////////////////////////////////////////////////////////////////////////

DECLARE_MULTICAST_DELEGATE_OneParam(FOnLoadGuardStateChangedEvent, bool);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLoadGuardStateChangedDynamic, bool, bIsLoading);


/** 
 * The Load Guard behaves similarly to a Border, but with the ability to hide its primary content and display a loading spinner
 * and optional message while needed content is loaded or otherwise prepared.
 * 
 * Use GuardAndLoadAsset to automatically display the loading state until the asset is loaded (then the content widget will be displayed).
 * For other applications (ex: waiting for an async backend call to complete), you can manually set the loading state of the guard.
 */
UCLASS(MinimalAPI, Config = Game, DefaultConfig)
class UCommonLoadGuard : public UContentWidget
{
	GENERATED_BODY()

public:
	UE_API UCommonLoadGuard(const FObjectInitializer& Initializer);

	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void SynchronizeProperties() override;

	virtual UClass* GetSlotClass() const override { return ULoadGuardSlot::StaticClass(); }
	UE_API virtual void OnSlotAdded(UPanelSlot* NewSlot) override;
	UE_API virtual void OnSlotRemoved(UPanelSlot* OldSlot) override;

	UFUNCTION(BlueprintCallable, Category = LoadGuard)
	UE_API void SetLoadingText(const FText& InLoadingText);
	
	UFUNCTION(BlueprintCallable, Category = LoadGuard)
	UE_API void SetIsLoading(bool bInIsLoading);

	UFUNCTION(BlueprintCallable, Category = LoadGuard)
	UE_API bool IsLoading() const;

	/**
	 * Displays the loading spinner until the asset is loaded
	 * Will pass a casted pointer to the given asset in the lambda callback - could be nullptr if you provide an incompatible type or invalid asset.
	 */
	template <typename ObjectType = UObject>
	void GuardAndLoadAsset(const TSoftObjectPtr<ObjectType>& InLazyAsset, TFunction<void(ObjectType*)> OnAssetLoaded)
	{
		if (MyLoadGuard.IsValid())
		{
			MyLoadGuard->GuardAndLoadAsset<ObjectType>(InLazyAsset, OnAssetLoaded);
		}
	}

	void GuardAndLoadAsset(const TSoftObjectPtr<UObject>& InLazyAsset, FOnLoadGuardAssetLoaded OnAssetLoaded)
	{
		if (MyLoadGuard.IsValid())
		{
			MyLoadGuard->GuardAndLoadAsset(InLazyAsset, OnAssetLoaded);
		}
	}

	FOnLoadGuardStateChangedEvent& OnLoadingStateChanged() { return OnLoadingStateChangedEvent; }

#if WITH_EDITOR
	UE_API virtual void OnCreationFromPalette() override;
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnAssetLoaded, UObject*, Object);

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;	

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = LoadGuard)
	bool bShowLoading = false;
#endif

private:
	UFUNCTION(BlueprintCallable, Category = LoadGuard, meta = (DisplayName = "Guard and Load Asset", ScriptName="GuardAndLoadAsset", AllowPrivateAccess = true))
	UE_API void BP_GuardAndLoadAsset(const TSoftObjectPtr<UObject>& InLazyAsset, const FOnAssetLoaded& OnAssetLoaded);

	UE_API void HandleLoadingStateChanged(bool bIsLoading);

	/** The background brush to display while loading the content */
	UPROPERTY(EditAnywhere, Category = LoadGuardThrobber)
	FSlateBrush LoadingBackgroundBrush;

	/** The loading throbber brush */
	UPROPERTY(EditAnywhere, Category = LoadGuardThrobber)
	FSlateBrush LoadingThrobberBrush;

	/** The horizontal alignment of the loading throbber & message */
	UPROPERTY(EditAnywhere, Category = LoadGuardThrobber)
	TEnumAsByte<EHorizontalAlignment> ThrobberAlignment;

	/** The padding of the loading throbber & message */
	UPROPERTY(EditAnywhere, Category = LoadGuardThrobber)
	FMargin ThrobberPadding;

	/** Loading message to display alongside the throbber */
	UPROPERTY(EditAnywhere, Category = LoadGuardText)
	FText LoadingText;

	/** Style to apply to the loading message */
	UPROPERTY(EditAnywhere, Category = LoadGuardText)
	TSubclassOf<UCommonTextStyle> TextStyle;

	UPROPERTY(BlueprintAssignable, Category = LoadGuard, meta = (DisplayName = "On Loading State Changed"))
	FOnLoadGuardStateChangedDynamic BP_OnLoadingStateChanged;

	UPROPERTY(Config)
	FSoftObjectPath SpinnerMaterialPath;

#if WITH_EDITORONLY_DATA
	/** Used to track widgets that were created before changing the default style pointer to null */
	UPROPERTY()
	bool bStyleNoLongerNeedsConversion;
#endif

	TSharedPtr<SLoadGuard> MyLoadGuard;

	FOnLoadGuardStateChangedEvent OnLoadingStateChangedEvent;
};

#undef UE_API
