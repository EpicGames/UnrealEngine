// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

/** 
 * Base item class for any UMG ListViews based on native, non-UObject items.
 *
 * Exclusively intended to provide bare-bones RTTI to the items to allow one array of list items to be multiple classes 
 * without needing a different, more awkward identification mechanism or an abstract virtual of every conceivable method in the base list item class
 */
class FCommonNativeListItem : public TSharedFromThis<FCommonNativeListItem>
{
public:
	virtual ~FCommonNativeListItem() = default;
	
	template <typename ListItemT>
	bool IsDerivedFrom() const
	{
		static_assert(TIsDerivedFrom<ListItemT, FCommonNativeListItem>::IsDerived, "FCommonNativeListItem::AsTypedItem<T> only supports FCommonNativeListItem types");
		return IsDerivedInternal(ListItemT::StaticItemType());
	}

	template <typename ListItemT>
	TSharedPtr<ListItemT> AsTypedItem()
	{
		static_assert(TIsDerivedFrom<ListItemT, FCommonNativeListItem>::IsDerived, "FCommonNativeListItem::AsTypedItem<T> only supports FCommonNativeListItem types");

		if (IsDerivedFrom<ListItemT>())
		{
			return StaticCastSharedRef<ListItemT>(AsShared());
		}
		return TSharedPtr<ListItemT>();
	}

	virtual FString GetStaticItemTypeName() const { return StaticItemType().ToString(); }
protected:
	static FName StaticItemType() { return TEXT("CommonNativeListItem"); }
	virtual bool IsDerivedInternal(FName ItemTypeName) const { return ItemTypeName == StaticItemType(); }
};

/**
 * Put this at the top of any list item classes that ultimately derive from FCommonNativeListItem.
 */
#define DERIVED_LIST_ITEM(ItemType, ParentItemType)	\
protected:	\
	virtual bool IsDerivedInternal(FName ItemTypeName) const override { return StaticItemType() == ItemTypeName || ParentItemType::IsDerivedInternal(ItemTypeName); }	\
public:	\
	virtual FString GetStaticItemTypeName() const override { return StaticItemType().ToString(); } \
	static FName StaticItemType() { return FName(#ItemType); }	\
private: //

/*
With the example classes below, we can have a list based on TSharedPtr<FMyCustomListItem> that contains a variety of items intended to generate
	entries formatted to represent normal and special-case items.

The exact item type can be safely determined by attempting to cast using MyListItem->AsTypedItem<T>.
It's just a really simple and cheap dynamic_cast alternative that doesn't require C++ RTTI to be enabled.

--------------------------------------------------------------------
Ex:

class FMyCustomListItem : public FCommonNativeListItem
{
	DERIVED_LIST_ITEM(FMyCustomListItem, FCommonNativeListItem);
}

class FMyCustomUsualListItem : public FMyCustomListItem
{
	DERIVED_LIST_ITEM(FMyCustomUsualListItem, FMyCustomListItem);
};

class FMyCustomSpecialCaseListItem : public FMyCustomListItem
{
	DERIVED_LIST_ITEM(FMyCustomSpecialCaseListItem, FMyCustomListItem);
}; 

class UMyCustomListView : public UListViewBase, ITypedUMGListView<TSharedPtr<FMyCustomListItem>>
{
	GENERATED_BODY()
	IMPLEMENT_TYPED_UMG_LIST(TSharedPtr<FMyCustomListItem, MyListView>)

public:
	...

private:
	TSharedPtr<SListView<TSharedPtr<FMyCustomListItem>>> MyListView;
}
*/
