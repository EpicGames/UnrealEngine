// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonTextBlock.h"
#include "CommonNumericTextBlock.generated.h"

#define UE_API COMMONUI_API

USTRUCT(BlueprintType)
struct FCommonNumberFormattingOptions
{
	GENERATED_BODY()

	FCommonNumberFormattingOptions()
		: RoundingMode(ERoundingMode::HalfFromZero)
		, AlwaysSign(false)
		, UseGrouping(true)
		, MinimumIntegralDigits(FNumberFormattingOptions::DefaultNoGrouping().MinimumIntegralDigits)
		, MaximumIntegralDigits(FNumberFormattingOptions::DefaultNoGrouping().MaximumIntegralDigits)
		, MinimumFractionalDigits(0)
		, MaximumFractionalDigits(0)
	{}

	// The rounding mode to be used when the actual value can not be precisely represented due to restrictions on the number of integral or fractional digits. See values for details.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating")
	TEnumAsByte<ERoundingMode> RoundingMode;

	// Should the numerals always display the sign. IE: "+1"
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating")
	bool AlwaysSign;

	// Should the numerals use group separators. IE: "1,000,000"
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating")
	bool UseGrouping;

	// How many integral digits should be shown at minimum? May cause digit "padding". IE: A minimum of 3 integral digits means 1.0 -> "001".
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating", meta = (ClampMin = "0"))
	int32 MinimumIntegralDigits;

	// How many integral digits should be shown at maximum? May cause rounding. IE: A maximum of 2 integral digits means 100.0 -> "99".
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating", meta = (ClampMin = "0"))
	int32 MaximumIntegralDigits;

	// How many fractional digits should be shown at minimum? May cause digit "padding". IE: A minimum of 2 fractional digits means 1.0 -> "1.00".
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating", meta = (ClampMin = "0"))
	int32 MinimumFractionalDigits;

	// How many fractional digits should be shown at maximum? May cause rounding. IE: HalfFromZero rounding and a maximum of 2 fractional digits means 0.009 -> "0.01".
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating", meta = (ClampMin = "0"))
	int32 MaximumFractionalDigits;
};

UENUM(BlueprintType)
enum class ECommonNumericType : uint8
{
	Number,
	Percentage,
	Seconds,
	Distance
};

/**
 * Numeric text block that provides interpolation, and some type support (numbers, percents, seconds, distance).
 */
UCLASS(MinimalAPI, BlueprintType)
class UCommonNumericTextBlock : public UCommonTextBlock
{
	GENERATED_BODY()

public:
	UE_API UCommonNumericTextBlock(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;


#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const;

	UE_API const FText GetPaletteCategory() override;
#endif // WITH_EDITOR

	UE_API virtual void SynchronizeProperties() override;

	// Returns the value this widget will ultimately show if it is interpolating, or the current value if it is not.
	UFUNCTION(BlueprintCallable, Category = "Numeric Text Block")
	UE_API float GetTargetValue() const;

	// Sets the current numeric value. NOTE: Cancels any ongoing interpolation!
	UFUNCTION(BlueprintCallable, Category = "Numeric Text Block")
	UE_API void SetCurrentValue(const float NewValue);

	/**
	 * Starts an ongoing process of interpolating the current numeric value to the specified target value.
	 * The interpolation process may take the specified maximum duration or complete sooner if the minimum change rate causes the target to be reached prematurely.
	 * Optionally, an outro duration can be specified in order to trigger an outro event before interpolation completes.
	 *
	 * TargetValue					The value to be interpolated to.
	 * MaximumInterpolationDuration	The duration, in seconds, for the interpolation to take, at most. Must be greater than 0.
	 * MinimumChangeRate			The minimum change in numeric value per second. Must be greater than or equal to 0.
	 * OutroDuration				The time offset, in seconds, *before* the end of the InterpolationDuration elapses, at which to trigger an outro event. Must be less than or equal to MaximumInterpolationDuration
	 */
	UFUNCTION(BlueprintCallable, Category = "Numeric Interpolation")
	UE_API void InterpolateToValue(const float TargetValue, float MaximumInterpolationDuration = 3.0f, float MinimumChangeRate = 1.0f, float OutroOffset = 0.0f);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Numeric Interpolation")
	UE_API bool IsInterpolatingNumericValue() const;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnInterpolationStarted, UCommonNumericTextBlock*, NumericTextBlock);
	// Event triggered when interpolation has started.
	UPROPERTY(BlueprintAssignable, Category = "Numeric Interpolation")
	FOnInterpolationStarted OnInterpolationStartedEvent;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnInterpolationUpdated, UCommonNumericTextBlock*, NumericTextBlock, float, LastValue, float, NewValue);
	// Event triggered when interpolation has updated.
	UPROPERTY(BlueprintAssignable, Category = "Numeric Interpolation")
	FOnInterpolationUpdated OnInterpolationUpdatedEvent;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnOutro, UCommonNumericTextBlock*, NumericTextBlock);
	// Event triggered at a specified time before the interpolation completes, for "outro" purposes.
	UPROPERTY(BlueprintAssignable, Category = "Numeric Interpolation")
	FOnOutro OnOutroEvent;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnInterpolationEnded, UCommonNumericTextBlock*, NumericTextBlock, const bool, HadCompleted);
	// Event triggered when interpolation has ended.
	UPROPERTY(BlueprintAssignable, Category = "Numeric Interpolation")
	FOnInterpolationEnded OnInterpolationEndedEvent;

	// The current numeric value being formatted for display, potentially being interpolated from. NOTE: The displayed text is very likely not identical to this value, due to formatting.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeric Text Block")
	float CurrentNumericValue;

	UFUNCTION(BlueprintCallable, Category = "Numeral Formating")
	UE_API void SetNumericType(ECommonNumericType InNumericType);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formating")
	ECommonNumericType NumericType;

	// The specifications for how the current numeric value should be formatted in to numeral text.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeral Formatting")
	FCommonNumberFormattingOptions FormattingSpecification;

	/** Exponent parameter for the "ease out" interpolation curve. Must be > 0, but should be > 1 in order to "ease out". */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeric Interpolation", meta = (ClampMin = "1.0"))
	float EaseOutInterpolationExponent;

	/** The desired interval, in seconds, between interpolation updates. 0.0 implies per-frame updates. NOTE: Interpolation updates may occur further apart due to tick rates. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Numeric Interpolation", meta = (ClampMin = "0.0"))
	float InterpolationUpdateInterval;

	/**
	 * The desired width of the formatted text may change rapidly and erratically during interpolation due to font glyph dimensions.
	 * To combat this, the desired width of the text will never shrink during interpolation.
	 * Once interpolation completes, the desired width will shrink over the duration specified.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Size Interpolation", meta = (ClampMin = "0.0"))
	float PostInterpolationShrinkDuration;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Size Interpolation")
	bool PerformSizeInterpolation;

protected:
	
	UE_API void UpdateUnderlyingText();
	UE_API virtual FText FormatText(float InCurrentNumericValue, const FNumberFormattingOptions& InNumberFormattingOptions) const;

private:
	FNumberFormattingOptions MakeNumberFormattingOptions() const;
	
	float CalculateInterpolationDuration(const float InTargetValue, const float InMinimumChangeRate, const float InMaximumInterpolationDuration) const;

	void OnTimerTick();
	void Tick(float DeltaTime);

	void EnterNumericInterpolation(const float InitialValue, const float FinalValue, const float Duration, const float OutroOffset);
	void UpdateNumericInterpolation();
	void ExitNumericInterpolation(const bool HasCompleted = false);

	void EnterSizeInterpolation(const float Duration);
	void UpdateSizeInterpolation();
	void ExitSizeInterpolation();

	void CancelInterpolation();

private:
	// Should the current numeric value be presented as a percentage? IE: 1.0 -> "100%"
	UPROPERTY()
	bool IsPercentage_DEPRECATED;

	// Enum for interpolation state machine.
	enum class EInterpolationState
	{
		None,
		NumericInterpolation,
		SizeInterpolation
	} CurrentInterpolationState;

	// Timer handle for timer-based ticking based on InterpolationUpdateInterval.
	FTimerHandle TimerTickHandle;
	float LastTimerTickTime;

	// State data for any current interpolation state.
	struct
	{
		float ElapsedStateDuration;
	} InterpolationState;


	// State data exclusively used when interpolating the numeric values.
	struct
	{
		float SourceValue;
		float DestinationValue;
		float OutroOffset;
		bool HasTriggeredOutro;
		float Duration;
	} NumericInterpolationState;

	// State data exclusively used when interpolating the minimum desired width to shrink after completing numeric interpolation.
	struct
	{
		int32 InitialWidth;
		float Duration;
	} SizeInterpolationState;
};

#undef UE_API
