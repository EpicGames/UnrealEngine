// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "CommonPoolableWidgetInterface.generated.h"

#define UE_API COMMONUI_API

UINTERFACE(MinimalAPI)
class UCommonPoolableWidgetInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Widget pool, if implemented WidgetFactory will attempt to reuse implementing widget objects.
 */
class ICommonPoolableWidgetInterface
{
	GENERATED_BODY()

protected:
	UFUNCTION(BlueprintNativeEvent, Category = "Common Poolable Widget")
	UE_API void OnAcquireFromPool();

	UFUNCTION(BlueprintNativeEvent, Category = "Common Poolable Widget")
	UE_API void OnReleaseToPool();
};

#undef UE_API
