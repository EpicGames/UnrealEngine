// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/RichTextBlock.h"

#include "CommonRichTextBlock.generated.h"

#define UE_API COMMONUI_API

class ITextDecorator;
enum class ETextTransformPolicy : uint8;

class STextScroller;
class UCommonTextStyle;
class UCommonTextScrollStyle;

// Various ways that we display inline icon that have an icon-name association
UENUM(BlueprintType)
enum class ERichTextInlineIconDisplayMode : uint8
{
	// Only show the icon - use when space is limited
	IconOnly,
	// Only show the text - use seldom if ever
	TextOnly,
	// Show both the icon and the text - use whenever there is space
	IconAndText,
	MAX
};

/**
 * Text block that supports custom scaling for mobile platforms, with option to only show icons if space is sparse.
 */
UCLASS(MinimalAPI)
class UCommonRichTextBlock : public URichTextBlock
{
	GENERATED_BODY()

public:
	ETextTransformPolicy GetTextTransformPolicy() const { return GetTransformPolicy(); }
	TSubclassOf<UCommonTextStyle> GetDefaultTextStyleClass() const { return DefaultTextStyleOverrideClass; }
	float GetMobileTextBlockScale() const { return MobileTextBlockScale; }

	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void SetText(const FText& InText) override;

	UFUNCTION(BlueprintCallable, Category = "Common Rich Text")
	UE_API void SetStyle(const TSubclassOf<UCommonTextStyle>& InStyle);

	UFUNCTION(BlueprintCallable, Category = "Common Rich Text|Scroll Style")
	UE_API void SetScrollingEnabled(bool bInIsScrollingEnabled);

#if WITH_EDITOR
	UE_API virtual void OnCreationFromPalette() override;
	UE_API const FText GetPaletteCategory() override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const;
#endif

	UPROPERTY(EditAnywhere, Category = InlineIcon)
	ERichTextInlineIconDisplayMode InlineIconDisplayMode;

	/** Toggle it on if the text color should also tint the inline icons. */
	UPROPERTY(EditAnywhere, Category = InlineIcon)
	bool bTintInlineIcon = false;
	static UE_API FString EscapeStringForRichText(FString InString);

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void CreateDecorators(TArray<TSharedRef<ITextDecorator>>& OutDecorators) override;

	UE_API virtual void ApplyUpdatedDefaultTextStyle() override;

private:
	UE_API void RefreshOverrideStyle();
	UE_API void ApplyTextBlockScale() const;

private:
	/** Mobile font size multiplier. Activated by default on mobile. See CVar Mobile_PreviewFontSize */
	UPROPERTY(EditAnywhere, Category = "Mobile", meta = (ClampMin = "0.01", ClampMax = "5"))
	float MobileTextBlockScale = 1.0f;
	
	UPROPERTY(EditAnywhere, Category = Appearance, meta = (EditCondition = bOverrideDefaultStyle))
	TSubclassOf<UCommonTextStyle> DefaultTextStyleOverrideClass;

	/** References the scroll style asset to use, no reference disables scrolling*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, meta = (ExposeOnSpawn = true, AllowPrivateAccess = true))
	TSubclassOf<UCommonTextScrollStyle> ScrollStyle;

	/** The orientation the text will scroll if out of bounds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, meta = (ExposeOnSpawn = true, AllowPrivateAccess = true))
	TEnumAsByte<EOrientation> ScrollOrientation = Orient_Horizontal;

	/** If scrolling is enabled/disabled initially, this can be updated in blueprint */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, meta = (ExposeOnSpawn = true, AllowPrivateAccess = true))
	bool bIsScrollingEnabled = true;

	/** True to always display text in ALL CAPS */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bDisplayAllCaps is deprecated. Please use TextTransformPolicy instead."))
	bool bDisplayAllCaps_DEPRECATED = false;

	/** True to automatically collapse this rich text block when set to display an empty string. Conversely, will be SelfHitTestInvisible when showing a non-empty string. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, meta = (AllowPrivateAccess = true))
	bool bAutoCollapseWithEmptyText = false;

	TSharedPtr<STextScroller> MyTextScroller;
};

#undef UE_API
