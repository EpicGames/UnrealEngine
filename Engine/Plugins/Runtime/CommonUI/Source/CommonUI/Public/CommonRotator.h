// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonButtonBase.h"


#include "Input/NavigationReply.h"
#include "CommonRotator.generated.h"

#define UE_API COMMONUI_API

class UCommonTextBlock;

UENUM(BlueprintType)
enum class ERotatorDirection : uint8
{
	Right,
	Left,
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnRotated, int32, Value);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnRotatedWithDirection, int32, Value, ERotatorDirection, RotatorDir);

/**
* A button that can rotate between given text labels.
*/
UCLASS(MinimalAPI, meta = (DisableNativeTick))
class UCommonRotator : public UCommonButtonBase
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual bool Initialize() override;
	UE_API virtual FNavigationReply NativeOnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent, const FNavigationReply& InDefaultReply);

	/** Handle and use controller navigation to rotate text */
	FNavigationDelegate OnNavigation;
	UE_API TSharedPtr<SWidget> HandleNavigation(EUINavigation UINavigation);

	/** Set the array of texts available */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UE_API void PopulateTextLabels(TArray<FText> Labels);

	/** Gets the current text value of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UE_API FText GetSelectedText() const;

	/** Sets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UE_API virtual void SetSelectedItem(int32 InValue);

	/** Gets the current selected index */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	int32 GetSelectedIndex() const { return SelectedIndex; }

	/** Shift the current text left. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UE_API void ShiftTextLeft();

	/** Shift the current text right. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UE_API void ShiftTextRight();

public:

	/** Called when the Selected state of this button changes. Provides the direction of rotation. */
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnRotatedWithDirection OnRotatedWithDirection;

	/** Called when the Selected state of this button changes */
	UE_DEPRECATED(5.4, "OnRotated is deprecated, please use OnRotatedWithDirection instead.")
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnRotated OnRotated;

	DECLARE_EVENT_TwoParams(UCommonRotator, FRotationEvent, int32 /*Value*/, bool /*bUserInitiated*/);
	FRotationEvent OnRotatedEvent;

protected:
	UE_API void ShiftTextLeftInternal(bool bFromNavigation);
	UE_API void ShiftTextRightInternal(bool bFromNavigation);

protected:
	UFUNCTION(BlueprintImplementableEvent, Category = Events, meta = (DisplayName = "On Options Populated"))
	UE_API void BP_OnOptionsPopulated(int32 Count);

	UFUNCTION(BlueprintImplementableEvent, Category = Events, meta = (DisplayName = "On Options Selected"))
	UE_API void BP_OnOptionSelected(int32 Index);

protected:

	/** The displayed text */
	UPROPERTY(BlueprintReadOnly, Category = CommonRotator, Meta = (BindWidget))
	TObjectPtr<UCommonTextBlock> MyText;

	// Holds the array of display texts
	TArray<FText> TextLabels;

	/** The index of the current text item */
	int32 SelectedIndex;
};

#undef UE_API
