// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidgetPool.h"
#include "CommonUserWidget.h"
#include "Engine/DataTable.h"

#include "CommonTabListWidgetBase.generated.h"

#define UE_API COMMONUI_API

class UCommonAnimatedSwitcher;

class UCommonButtonBase;
class UCommonButtonGroupBase;
class UInputAction;

/** Information about a registered tab in the tab list */
USTRUCT()
struct FCommonRegisteredTabInfo
{
	GENERATED_BODY()

public:
	/** The index of the tab in the list */
	UPROPERTY()
	int32 TabIndex;
	
	/** The class of our TabButton widget */
	UPROPERTY()
	TSubclassOf<UCommonButtonBase> TabButtonClass;
	
	/** The actual button widget that represents this tab on-screen */
	UPROPERTY()
	TObjectPtr<UCommonButtonBase> TabButton;

	/** The actual instance of the content widget to display when this tab is selected. Can be null if a load is required. */
	UPROPERTY()
	TObjectPtr<UWidget> ContentInstance;

	FCommonRegisteredTabInfo()
		: TabIndex(INDEX_NONE)
		, TabButton(nullptr)
		, ContentInstance(nullptr)
	{}
};

/** Base class for a list of selectable tabs that correspondingly activate and display an arbitrary widget in a linked switcher */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = UI, meta = (Category = "Common UI", DisableNativeTick))
class UCommonTabListWidgetBase : public UCommonUserWidget
{
	GENERATED_UCLASS_BODY()

public:
	/** Delegate broadcast when a new tab is selected. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnTabSelected, FName, TabId);

	/** Broadcasts when a new tab is selected. */
	UPROPERTY(BlueprintAssignable, Category = TabList)
	FOnTabSelected OnTabSelected;

	/** Delegate broadcast when a new tab is created. Allows hook ups after creation. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnTabButtonCreation, FName, TabId, UCommonButtonBase*, TabButton);

	/** Broadcasts when a new tab is created. */
	UPROPERTY(BlueprintAssignable, Category = TabList)
	FOnTabButtonCreation OnTabButtonCreation;

	/** Delegate broadcast when a tab is being removed. Allows clean ups after destruction. */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnTabButtonRemoval, FName, TabId, UCommonButtonBase*, TabButton);

	/** Broadcasts when a new tab is created. */
	UPROPERTY(BlueprintAssignable, Category = TabList)
	FOnTabButtonRemoval OnTabButtonRemoval;
	
	/** Delegate broadcast when the tab list has been rebuilt (after a new tab has been inserted rather than added to the end). */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnTabListRebuilt);
	
	/** Broadcasts when the tab list has been rebuilt (after a new tab has been inserted rather than added to the end). */
	UPROPERTY(BlueprintAssignable, Category = TabList)
	FOnTabListRebuilt OnTabListRebuilt;

	/** @return The currently active (selected) tab */
	UFUNCTION(BlueprintCallable, Category = TabList)
	FName GetActiveTab() const { return ActiveTabID; }

	/**
	 * Establishes the activatable widget switcher instance that this tab list should interact with
	 * @param CommonSwitcher The switcher that this tab list should be associated with and manipulate
	 */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API virtual void SetLinkedSwitcher(UCommonAnimatedSwitcher* CommonSwitcher);

	/** @return The switcher that this tab list is associated with and manipulates */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = TabList)
	UE_API UCommonAnimatedSwitcher* GetLinkedSwitcher() const;

	/**
	 * Registers and adds a new tab to the list that corresponds to a given widget instance. If not present in the linked switcher, it will be added.
	 * @param TabID The name ID used to keep track of this tab. Attempts to register a tab under a duplicate ID will fail.
	 * @param ButtonWidgetType The widget type to create for this tab
	 * @param ContentWidget The widget to associate with the registered tab
	 * @param TabIndex Determines where in the tab list to insert the new tab (-1 means tab will be added to end of the list)
	 * @return True if the new tab registered successfully and there were no name ID conflicts
	 */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API bool RegisterTab(FName TabNameID, TSubclassOf<UCommonButtonBase> ButtonWidgetType, UWidget* ContentWidget, const int32 TabIndex = -1 /*INDEX_NONE*/);

	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API bool RemoveTab(FName TabNameID);

	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API void RemoveAllTabs();

	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API int32 GetTabCount() const;

	/** 
	 * Selects the tab registered under the provided name ID
	 * @param TabNameID The name ID for the tab given when registered
	 */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API bool SelectTabByID(FName TabNameID, bool bSuppressClickFeedback = false );

	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API FName GetSelectedTabId() const;

	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API FName GetTabIdAtIndex(int32 Index) const;

	/** Sets the visibility of the tab associated with the given ID  */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API void SetTabVisibility(FName TabNameID, ESlateVisibility NewVisibility);

	/** Sets whether the tab associated with the given ID is enabled/disabled */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API void SetTabEnabled(FName TabNameID, bool bEnable);

	/** Sets whether the tab associated with the given ID is interactable */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API void SetTabInteractionEnabled(FName TabNameID, bool bEnable);

	/** Disables the tab associated with the given ID with a reason */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API void DisableTabWithReason(FName TabNameID, const FText& Reason);

	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API virtual void SetListeningForInput(bool bShouldListen);

	/** Returns the tab button matching the ID, if found */
	UFUNCTION(BlueprintCallable, Category = TabList)
	UE_API UCommonButtonBase* GetTabButtonBaseByID(FName TabNameID) const;

	/** Checks if a tab has an associated content widget */
	UFUNCTION(BlueprintCallable, Category = "Tab List")
	UE_API bool HasTabContentWidget(const FName TabNameId) const;

	/** Registers a content widget with a previously created tab with ID TabNameId. If a linked switcher has been setup, it will also be added to it */
	UFUNCTION(BlueprintCallable, Category = "Tab List")
	UE_API bool RegisterTabContentWidget(const FName TabNameId, UWidget* ContentWidget);

	/** Allows one to temporarily disable the selection-required behavior TabButtonGroup, useful during initialization and destruction of a UCommonTabListWidgetBase */
	UE_API void SetSelectionRequired(bool bSelectionRequired);

protected:
	// UUserWidget interface
	UE_API virtual void NativeOnInitialized() override;
	UE_API virtual void NativeConstruct() override;
	UE_API virtual void NativeDestruct() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End UUserWidget

	UE_API virtual void UpdateBindings();

	UE_API bool IsRebuildingList() const;

	UFUNCTION(BlueprintImplementableEvent, Category = TabList, meta = (BlueprintProtected = "true"))
	UE_API void HandlePreLinkedSwitcherChanged_BP();

	UE_API virtual void HandlePreLinkedSwitcherChanged();

	UFUNCTION(BlueprintImplementableEvent, Category = TabList, meta = (BlueprintProtected = "true"))
	UE_API void HandlePostLinkedSwitcherChanged_BP();

	UE_API virtual void HandlePostLinkedSwitcherChanged();

	UFUNCTION(BlueprintNativeEvent, Category = TabList, meta = (BlueprintProtected = "true"))
	UE_API void HandleTabCreation(FName TabNameID, UCommonButtonBase* TabButton);

	UFUNCTION(BlueprintNativeEvent, Category = TabList, meta = (BlueprintProtected = "true"))
	UE_API void HandleTabRemoval(FName TabNameID, UCommonButtonBase* TabButton);

	/** The input action to listen for causing the next tab to be selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TabList, meta = (RowType = "/Script/CommonUI.CommonInputActionDataBase"))
	FDataTableRowHandle NextTabInputActionData;

	/** The input action to listen for causing the previous tab to be selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TabList, meta = (RowType = "/Script/CommonUI.CommonInputActionDataBase"))
	FDataTableRowHandle PreviousTabInputActionData;

	/** The input action to listen for causing the next tab to be selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TabList, meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	TObjectPtr<UInputAction> NextTabEnhancedInputAction;

	/** The input action to listen for causing the previous tab to be selected */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TabList, meta = (EditCondition = "CommonInput.CommonInputSettings.IsEnhancedInputSupportEnabled", EditConditionHides))
	TObjectPtr<UInputAction> PreviousTabEnhancedInputAction;

	/** Whether to register to handle tab list input immediately upon construction */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TabList, meta = (ExposeOnSpawn = "true"))
	bool bAutoListenForInput;

	/** Whether pressing next/prev tab on the last/first tab should wrap selection to the beginning/end or stay at the end/beginning. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = TabList, meta = (ExposeOnSpawn = "true"))
	bool bShouldWrapNavigation = true;

	/**
	* Whether to defer until next tick rebuilding tab list when inserting new tab (rather than adding to the end).
	* Useful if inserting multiple tabs in the same tick as the tab list will only be rebuilt once.
	*/
	UPROPERTY(EditAnywhere, Category = TabList)
	bool bDeferRebuildingTabList;

protected:
	UE_API const TMap<FName, FCommonRegisteredTabInfo>& GetRegisteredTabsByID() const;

	UFUNCTION()
	UE_API void HandleTabButtonSelected(UCommonButtonBase* SelectedTabButton, int32 ButtonIndex);

	UFUNCTION()
	UE_API void HandlePreviousTabInputAction(bool& bPassthrough);
	
	UFUNCTION()
	UE_API void HandleNextTabInputAction(bool& bPassthrough);

	/** The activatable widget switcher that this tab list is associated with and manipulates */
	UPROPERTY(Transient)
	TWeakObjectPtr<UCommonAnimatedSwitcher> LinkedSwitcher;
	
	/** The button group that manages all the created tab buttons */
	UPROPERTY(Transient)
	TObjectPtr<UCommonButtonGroupBase> TabButtonGroup;

	/** Is the tab list currently listening for tab input actions? */
	bool bIsListeningForInput = false;

private:
	UE_API void HandleNextTabAction();
	UE_API void HandlePreviousTabAction();

	UE_API bool DeferredRebuildTabList(float DeltaTime);
	UE_API void RebuildTabList();

	UE_API void RemoveTab_Internal(const FName TabNameID, const FCommonRegisteredTabInfo& TabInfo);

	/** Info about each of the currently registered tabs organized by a given registration name ID */
	UPROPERTY(Transient)
	TMap<FName, FCommonRegisteredTabInfo> RegisteredTabsByID;

	UPROPERTY(Transient)
	FUserWidgetPool TabButtonWidgetPool;

	/** The registration ID of the currently active tab */
	FName ActiveTabID;

	bool bIsRebuildingList = false;
	bool bPendingRebuild = false;

	FUIActionBindingHandle NextTabActionHandle;
	FUIActionBindingHandle PrevTabActionHandle;
};

#undef UE_API
