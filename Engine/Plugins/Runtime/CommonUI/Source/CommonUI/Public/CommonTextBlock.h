// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/TextBlock.h"
#include "Widgets/Accessibility/SlateWidgetAccessibleTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "CommonTextBlock.generated.h"

#define UE_API COMMONUI_API

struct FTextBlockStyle;
struct FTextScrollerOptions;

class STextScroller;

/* 
 * ---- All properties must be EditDefaultsOnly, BlueprintReadOnly !!! -----
 * We return the CDO to blueprints, so we cannot allow any changes (blueprint doesn't support const variables)
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = UI, meta = (Category = "Common UI"))
class UCommonTextStyle : public UObject
{
	GENERATED_BODY()

public:
	UE_API UCommonTextStyle();

	/** The font to apply at each size */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Font")
	FSlateFontInfo Font;

	/** The color of the text */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Color")
	FLinearColor Color;

	/** Whether or not the style uses a drop shadow */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Shadow")
	bool bUsesDropShadow;

	/** The offset of the drop shadow at each size */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Shadow", meta = (EditCondition = "bUsesDropShadow"))
	FVector2D ShadowOffset;

	/** The drop shadow color */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Shadow", meta = (EditCondition = "bUsesDropShadow"))
	FLinearColor ShadowColor;

	/** The amount of blank space left around the edges of text area at each size */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	FMargin Margin;

	/** The brush used to draw an strike through the text (if any) */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	FSlateBrush StrikeBrush;

	/** The amount to scale each lines height by at each size */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	float LineHeightPercentage;

	/** Whether to leave extra space below the last line due to line height */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Properties")
	bool ApplyLineHeightToBottomLine;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API void GetFont(FSlateFontInfo& OutFont) const;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API void GetColor(FLinearColor& OutColor) const;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API void GetMargin(FMargin& OutMargin) const;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API float GetLineHeightPercentage() const;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API bool GetApplyLineHeightToBottomLine() const;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API void GetShadowOffset(FVector2D& OutShadowOffset) const;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API void GetShadowColor(FLinearColor& OutColor) const;

	UFUNCTION(BlueprintCallable, Category = "Common Text Style|Getters")
	UE_API void GetStrikeBrush(FSlateBrush& OutStrikeBrush) const;

	UE_API virtual void ToTextBlockStyle(FTextBlockStyle& OutTextBlockStyle) const;

	UE_API virtual void ApplyToTextBlock(const TSharedRef<STextBlock>& TextBlock) const;
};

/* 
 * ---- All properties must be EditDefaultsOnly, BlueprintReadOnly !!! -----
 * We return the CDO to blueprints, so we cannot allow any changes (blueprint doesn't support const variables)
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, ClassGroup = UI, meta = (Category = "Common UI"))
class UCommonTextScrollStyle : public UObject
{
	GENERATED_BODY()

public:
	UE_API FTextScrollerOptions ToScrollOptions() const;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Properties")
	float Speed;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Properties")
	float StartDelay;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Properties")
	float EndDelay;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Properties")
	float FadeInDelay;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Properties")
	float FadeOutDelay;
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Properties")
	EWidgetClipping Clipping = EWidgetClipping::OnDemand;
};

/**
 * Text block with automatic scrolling for FX / large texts, also supports a larger set of default styling, & custom mobile scaling.
 */
UCLASS(MinimalAPI, Config = CommonUI, DefaultConfig, ClassGroup = UI, meta = (Category = "Common UI", DisplayName = "Common Text", PrioritizeCategories = "Content"))
class UCommonTextBlock : public UTextBlock
{
	GENERATED_UCLASS_BODY()
public:
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API void SetWrapTextWidth(int32 InWrapTextAt);

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API void SetTextCase(bool bUseAllCaps);

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API void SetLineHeightPercentage(float InLineHeightPercentage);

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API void SetApplyLineHeightToBottomLine(bool InApplyLineHeightToBottomLine);

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API void SetStyle(TSubclassOf<UCommonTextStyle> InStyle);

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API void SetScrollOrientation(TEnumAsByte<EOrientation> InScrollOrientation);

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API const FMargin& GetMargin();

	UFUNCTION(BlueprintCallable, Category = "Common Text")
	UE_API void SetMargin(const FMargin& InMargin);

	UFUNCTION(BlueprintCallable, Category = "Common Text|Mobile")
	UE_API float GetMobileFontSizeMultiplier() const;
	
	/** Sets the new value and then applies the FontSizeMultiplier */
	UFUNCTION(BlueprintCallable, Category = "Common Text|Mobile")
	UE_API void SetMobileFontSizeMultiplier(float InMobileFontSizeMultiplier);

	UFUNCTION(BlueprintCallable, Category = "Common Text|Scroll Style")
	UE_API void ResetScrollState();

	UFUNCTION(BlueprintCallable, Category = "Common Text|Scroll Style")
	UE_API void SetScrollingEnabled(bool bInIsScrollingEnabled);

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void SynchronizeProperties() override;

	UE_API virtual void OnTextChanged() override;
	UE_API virtual void OnFontChanged() override;

	/** Mobile font size multiplier. Activated by default on mobile. See CVar Mobile_PreviewFontSize */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Mobile", meta = (ClampMin = "0.01", ClampMax = "5.0"))
	float MobileFontSizeMultiplier = 1.0f;
	
#if WITH_EDITOR
	UE_API virtual void OnCreationFromPalette() override;
	UE_API const FText GetPaletteCategory() override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const;
#endif // WITH_EDITOR

private:
	/** If scrolling is enabled/disabled initially, this can be updated in blueprint */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonText, meta = (ExposeOnSpawn = true, AllowPrivateAccess = true))
	bool bIsScrollingEnabled = true;

	/** True to always display text in ALL CAPS */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bDisplayAllCaps is deprecated. Please use TextTransformPolicy instead."))
	bool bDisplayAllCaps_DEPRECATED = false;

	/** True to automatically collapse this text block when set to display an empty string. Conversely, will be SelfHitTestInvisible when showing a non-empty string. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonText, meta = (AllowPrivateAccess = true))
	bool bAutoCollapseWithEmptyText = false;
	
	/** References the text style to use */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonText, meta = (ExposeOnSpawn = true, AllowPrivateAccess = true))
	TSubclassOf<UCommonTextStyle> Style;

	/** References the scroll style asset to use, no reference disables scrolling*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonText, meta = (ExposeOnSpawn = true, AllowPrivateAccess = true))
	TSubclassOf<UCommonTextScrollStyle> ScrollStyle;
	
	/** The orientation the text will scroll if out of bounds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CommonText, meta = (ExposeOnSpawn = true, AllowPrivateAccess = true))
	TEnumAsByte<EOrientation> ScrollOrientation = Orient_Horizontal;

#if WITH_EDITORONLY_DATA
	/** Used to track widgets that were created before changing the default style pointer to null */
	UPROPERTY()
	bool bStyleNoLongerNeedsConversion;
#endif

	void UpdateFromStyle();
	const UCommonTextStyle* GetStyleCDO() const;
	const UCommonTextScrollStyle* GetScrollStyleCDO() const;

	TSharedPtr<class STextScroller> TextScroller;

	void ApplyFontSizeMultiplier() const;
};

#undef UE_API
