// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/TileView.h"
#include "CommonTileView.generated.h"

#define UE_API COMMONUI_API

class STableViewBase;

/**
 * TileView specialized to navigate on focus for consoles & enable scrolling when not focused for touch
 */
UCLASS(MinimalAPI, meta = (DisableNativeTick))
class UCommonTileView : public UTileView
{
	GENERATED_BODY()

public:
	UE_API UCommonTileView(const FObjectInitializer& ObjectInitializer);

protected:
	UE_API virtual TSharedRef<STableViewBase> RebuildListWidget() override;
	UE_API virtual UUserWidget& OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable) override;
};

#undef UE_API
