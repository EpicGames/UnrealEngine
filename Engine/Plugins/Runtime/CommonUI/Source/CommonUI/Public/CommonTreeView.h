// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/TreeView.h"
#include "CommonTreeView.generated.h"

#define UE_API COMMONUI_API

class STableViewBase;

//////////////////////////////////////////////////////////////////////////
// SCommonTreeView
//////////////////////////////////////////////////////////////////////////

template <typename ItemType>
class SCommonTreeView : public STreeView<ItemType>
{
public:
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		if (bScrollToSelectedOnFocus && (InFocusEvent.GetCause() == EFocusCause::Navigation || InFocusEvent.GetCause() == EFocusCause::SetDirectly))
		{
			// Set selection to the first item in a list if no items are selected.
			// If bReturnFocusToSelection is true find the last selected object and focus on that.
			if (this->HasValidRootItemsSource() && this->GetRootItems().Num() > 0)
			{
				typename TListTypeTraits<ItemType>::NullableType ItemNavigatedTo = TListTypeTraits<ItemType>::MakeNullPtr();
				if (this->GetNumItemsSelected() == 0)
				{
					ItemNavigatedTo = this->GetRootItems()[0];
				}
				else if (this->bReturnFocusToSelection && TListTypeTraits<ItemType>::IsPtrValid(this->SelectorItem))
				{
					ItemNavigatedTo = this->SelectorItem;
				}

				if (TListTypeTraits<ItemType>::IsPtrValid(ItemNavigatedTo))
				{
					ItemType SelectedItem = TListTypeTraits<ItemType>::NullableItemTypeConvertToItemType(ItemNavigatedTo);
					this->SetSelection(SelectedItem, ESelectInfo::OnNavigation);
					this->RequestNavigateToItem(SelectedItem, InFocusEvent.GetUser());
				}
			}
		}
		bScrollToSelectedOnFocus = true;

		return SListView<ItemType>::OnFocusReceived(MyGeometry, InFocusEvent);
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		STreeView<ItemType>::OnMouseLeave(MouseEvent);

		if (MouseEvent.IsTouchEvent() && this->HasMouseCapture())
		{
			// Regular list views will clear this flag when the pointer leaves the list. To
			// continue scrolling outside the list, we need this to remain on.
			this->bStartedTouchInteraction = true;
		}
	}

	virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		FReply Reply = STreeView<ItemType>::OnTouchMoved(MyGeometry, InTouchEvent);

		if (Reply.IsEventHandled() && this->HasMouseCapture())
		{
			bScrollToSelectedOnFocus = false;
			Reply.SetUserFocus(this->AsShared());
		}
		return Reply;
	}

	virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		return STreeView<ItemType>::OnTouchEnded(MyGeometry, InTouchEvent);
	}

protected:
	bool bScrollToSelectedOnFocus = true;
};

//////////////////////////////////////////////////////////////////////////
// UCommonTreeView
//////////////////////////////////////////////////////////////////////////

/**
 * TreeView specialized to navigate on focus for consoles & enable scrolling when not focused for touch
 */
UCLASS(MinimalAPI)
class UCommonTreeView : public UTreeView
{
	GENERATED_BODY()

public:
	UE_API UCommonTreeView(const FObjectInitializer& ObjectInitializer);

protected:
	UE_API virtual TSharedRef<STableViewBase> RebuildListWidget() override;
	UE_API virtual UUserWidget& OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable) override;
};

#undef UE_API
