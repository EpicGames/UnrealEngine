// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CommonTextBlock.h"
#include "CommonButtonBase.h"
#include "CommonBorder.h"
#include "CommonUIEditorSettings.generated.h"

#define UE_API COMMONUI_API

UCLASS(MinimalAPI, config = Editor, defaultconfig)
class UCommonUIEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API UCommonUIEditorSettings(const FObjectInitializer& Initializer);
#if WITH_EDITOR
	/* Called to load CommonUIEditorSettings data */
	UE_API void LoadData();

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	
	/*  Template Styles - Only accessible in editor builds should be transfered to new widgets in OnCreationFromPalette() overrides */
	UE_API const TSubclassOf<UCommonTextStyle>& GetTemplateTextStyle() const;

	UE_API const TSubclassOf<UCommonButtonStyle>& GetTemplateButtonStyle() const;

	UE_API const TSubclassOf<UCommonBorderStyle>& GetTemplateBorderStyle() const;

private:
	void LoadEditorData();
#endif

private:
	/** Newly created CommonText Widgets will use this style. */
	UPROPERTY(config, EditAnywhere, Category = "Text")
	TSoftClassPtr<UCommonTextStyle> TemplateTextStyle;

	/** Newly created CommonButton Widgets will use this style. */
	UPROPERTY(config, EditAnywhere, Category = "Buttons")
	TSoftClassPtr<UCommonButtonStyle> TemplateButtonStyle;

	/** Newly created CommonBorder Widgets will use this style. */
	UPROPERTY(config, EditAnywhere, Category = "Border")
	TSoftClassPtr<UCommonBorderStyle> TemplateBorderStyle;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TSubclassOf<UCommonTextStyle> TemplateTextStyleClass;

	UPROPERTY(Transient)
	TSubclassOf<UCommonButtonStyle> TemplateButtonStyleClass;

	UPROPERTY(Transient)
	TSubclassOf<UCommonBorderStyle> TemplateBorderStyleClass;
#endif
private:
	bool bDefaultDataLoaded;
};

#undef UE_API
