// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "UITag.h"
#include "CommonUILibrary.generated.h"

#define UE_API COMMONUI_API

class UWidget;
template <typename T> class TSubclassOf;

UCLASS(MinimalAPI)
class UCommonUILibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Request a focus refresh from any widget that is a descendant of the leaf-most activated widget.
	 * This should be used to handle asynchronous requests to route focus from outside the leaf-most activatable widget. 
	 * A very common use case is a visualization or asynchronous widget creation from a plugin-injected widget 
	 * or a widget otherwise unavailable or cumbersome to access from the leaf-most activatable widget.
	 * 
	 * @return Whether or not the request for focus was made
	 */
	UFUNCTION(BlueprintCallable, Category = "CommonUI|Focus", meta=(DefaultToSelf="ContextWidget", WorldContext="ContextWidget"))
	static UE_API bool RefreshFocusIfLeafmostDescendant(UWidget* ContextWidget);

	/**
	 * Finds the first parent widget of the given type and returns it, or null if no parent could be found.
	 */
	UFUNCTION(BlueprintCallable, Category="Common UI", meta=(DeterminesOutputType=Type))
	static UE_API UWidget* FindParentWidgetOfType(UWidget* StartingWidget, TSubclassOf<UWidget> Type);

	/**
	* Finds the first parent widget implementing the given interface and returns it, or null if no parent could be found.
	*/
	UFUNCTION(BlueprintCallable, Category="Common UI")
	static UE_API UWidget* FindParentWidgetImplementingInterface(UWidget* StartingWidget, TSubclassOf<UInterface> Interface);

	/**
	 * Interpret a UITag as GameplayTag. Enables connecting UITags to wildcard comparison nodes that map to explicit GameplayTag functions.
	 */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (BlueprintAutocast))
	static UE_API FGameplayTag Conv_UITagToGameplayTag(FUITag InValue);

	/**
	 * Interpret a UIActionTag as GameplayTag. Enables connecting FUIActionTags to wildcard comparison nodes that map to explicit GameplayTag functions.
	 */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (BlueprintAutocast))
	static UE_API FGameplayTag Conv_UIActionTagToGameplayTag(FUIActionTag InValue);
};

#undef UE_API
