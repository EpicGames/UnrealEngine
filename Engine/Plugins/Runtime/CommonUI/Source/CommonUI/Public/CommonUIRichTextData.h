// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"

#include "CommonUIRichTextData.generated.h"

#define UE_API COMMONUI_API

USTRUCT(BlueprintType)
struct FRichTextIconData : public FTableRowBase
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category = "RichText Icon")
	FText DisplayName;

	UPROPERTY(EditDefaultsOnly, Category = "RichText Icon", meta = (DisplayThumbnail = "true", DisplayName = "Image", AllowedClasses = "/Script/Engine.Texture2D,/Script/Engine.MaterialInterface,/Script/Engine.SlateTextureAtlasInterface", DisallowedClasses = "/Script/MediaAssets.MediaTexture"))
	TSoftObjectPtr<UObject> ResourceObject;

	UPROPERTY(EditDefaultsOnly, Category = "RichText Icon")
	FVector2D ImageSize = FVector2D(64.f, 64.f);
};

/** 
 * Derive from this class for rich text data per game
 * it is referenced in Common UI Settings, found in project settings UI
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (Category = "Common UI"))
class UCommonUIRichTextData : public UObject
{
	GENERATED_BODY()

public:
	static UE_API UCommonUIRichTextData* Get();

	UE_API const FRichTextIconData* FindIcon(const FName& InKey);
	const TMap<FName, uint8*>& GetIconMap() const { return InlineIconSet->GetRowMap(); }

private:
	UPROPERTY(EditDefaultsOnly, Category = "Inline Icons", meta = (RowType = "/Script/CommonUI.RichTextIconData"))
	TObjectPtr<UDataTable> InlineIconSet;
};

#undef UE_API
