// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CommonUIRichTextData.h"
#include "NativeGameplayTags.h"
#include "Styling/SlateBrush.h"
#include "CommonUISettings.generated.h"

#define UE_API COMMONUI_API

class UMaterial;
class UMaterialInterface;

COMMONUI_API UE_DECLARE_GAMEPLAY_TAG_EXTERN(TAG_PlatformTrait_PlayInEditor);

UENUM()
enum class ECommonButtonAcceptKeyHandling
{
	// (Default for projects created prior to 5.6) When a CommonButton is focused, it will ignore the configured keys for slate navigation's Accept action (see FNavigationConfig::KeyActionRules),
	// leaving it up to the button's Triggering Input Action to trigger the button's click event.
	// This allows other objects to bind input actions to the accept action keys while focusing a common button.
	Ignore,

	// (Default for new projects as of 5.6) When a CommonButton is focused, it will let all inputs events flow through to the underlying SButton, therefore triggering the button's click event when
	// pressing the configured keys for slate navigation's Accept action (see FNavigationConfig::KeyActionRules)
	// This will prevent input actions that are bound to those keys from triggering while a CommonButton is focused.
	TriggerClick,
};

UCLASS(MinimalAPI, config = Game, defaultconfig)
class UCommonUISettings : public UObject
{
	GENERATED_BODY()

public:
	UE_API UCommonUISettings(const FObjectInitializer& Initializer = FObjectInitializer::Get());
	UE_API UCommonUISettings(FVTableHelper& Helper);
	UE_API ~UCommonUISettings();

	// Called to load CommonUISetting data, if bAutoLoadData if set to false then game code must call LoadData().
	UE_API void LoadData();

	//~UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	UE_API virtual void PostReloadConfig(FProperty* PropertyThatWasLoaded) override;
	UE_API virtual void PostInitProperties() override;
	//~End of UObject interface

	// Called by the module startup to auto load CommonUISetting data if bAutoLoadData is true.
	UE_API void AutoLoadData();

	UE_API UCommonUIRichTextData* GetRichTextData() const;
	UE_API const FSlateBrush& GetDefaultThrobberBrush() const;
	UE_API UObject* GetDefaultImageResourceObject() const;
	UE_API const FGameplayTagContainer& GetPlatformTraits() const;
	UE_API ECommonButtonAcceptKeyHandling GetCommonButtonAcceptKeyHandling() const;

private:

	/** Controls if the data referenced is automatically loaded.
	 *  If False then game code must call LoadData() on it's own.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Default")
	bool bAutoLoadData;

	/** The Default Image Resource, newly created CommonImage Widgets will use this style. */
	UPROPERTY(config, EditAnywhere, Category = "Image", meta = (AllowedClasses = "/Script/Engine.Texture2D,/Script/Engine.MaterialInterface"))
	TSoftObjectPtr<UObject> DefaultImageResourceObject;

	/** The Default Throbber Material, newly created CommonLoadGuard Widget will use this style. */
	UPROPERTY(config, EditAnywhere, Category = "Throbber")
	TSoftObjectPtr<UMaterialInterface> DefaultThrobberMaterial;

	/** The Default Data for rich text to show inline icon and others. */
	UPROPERTY(config, EditAnywhere, Category = "RichText", meta=(AllowAbstract=false))
	TSoftClassPtr<UCommonUIRichTextData> DefaultRichTextDataClass;

	/** The set of traits defined per-platform (e.g., the default input mode, whether or not you can exit the application, etc...) */
	UPROPERTY(config, EditAnywhere, Category = "Visibility", meta=(Categories="Platform.Trait", ConfigHierarchyEditable))
	TArray<FGameplayTag> PlatformTraits;

	/**
	 * How should CommonButton widgets handle SlateNavigation Accept actions?
	 */
	UPROPERTY(config, EditAnywhere, Category = "Button")
	ECommonButtonAcceptKeyHandling CommonButtonAcceptKeyHandling;

private:
	void LoadEditorData();
	void RebuildTraitContainer();

	bool bDefaultDataLoaded;

	// Merged version of PlatformTraits
	// This is not the config property because there is no direct ini inheritance for structs
	// (even ones like tag containers that represent a set), unlike arrays
	FGameplayTagContainer PlatformTraitContainer;

	UPROPERTY(Transient)
	TObjectPtr<UObject> DefaultImageResourceObjectInstance;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> DefaultThrobberMaterialInstance;

	UPROPERTY(Transient)
	FSlateBrush DefaultThrobberBrush;

	UPROPERTY(Transient)
	TObjectPtr<UCommonUIRichTextData> RichTextDataInstance;
};

#undef UE_API
