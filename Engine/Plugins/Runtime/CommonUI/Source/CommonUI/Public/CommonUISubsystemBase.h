// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/GameInstanceSubsystem.h"

#include "CommonUISubsystemBase.generated.h"

#define UE_API COMMONUI_API

enum class ECommonInputType : uint8;
struct FDataTableRowHandle;
struct FSlateBrush;

class IAnalyticsProviderET;
class UWidget;
class ULocalPlayer;
class UInputAction;

UCLASS(MinimalAPI, DisplayName = "CommonUI")
class UCommonUISubsystemBase : public UGameInstanceSubsystem
{
	GENERATED_BODY()

public:
	static UE_API UCommonUISubsystemBase* Get(const UWidget& Widget);

	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	/** set the analytic provider for the CommonUI Widgets to use */
	UE_API void SetAnalyticProvider(const TSharedPtr<IAnalyticsProviderET>& AnalyticProvider);

	// Gets Action Button Icon for current gamepad
	UFUNCTION(BlueprintCallable, Category = CommonUISubsystem)
	UE_API FSlateBrush GetInputActionButtonIcon(const FDataTableRowHandle& InputActionRowHandle, ECommonInputType InputType, const FName& GamepadName) const;

	// Gets Action Button Icon for given action and player, enhanced input API currently does not allow input type specification
	UFUNCTION(BlueprintCallable, Category = CommonUISubsystem)
	UE_API FSlateBrush GetEnhancedInputActionButtonIcon(const UInputAction* InputAction, const ULocalPlayer* LocalPlayer) const;

	/** Analytic Events **/

	//CommonUI.ButtonClicked
	UE_API void FireEvent_ButtonClicked(const FString& InstanceName, const FString& ABTestName, const FString& ExtraData) const;

	//CommonUI.PanelPushed
	UE_API void FireEvent_PanelPushed(const FString& PanelName) const;
	
	UE_API virtual void SetInputAllowed(bool bEnabled, const FName& Reason, const ULocalPlayer& LocalPlayer);
	UE_API virtual bool IsInputAllowed(const ULocalPlayer* LocalPlayer) const;

private:

	void HandleInputMethodChanged(ECommonInputType bNewInputType);

	TWeakPtr<class IAnalyticsProviderET> AnalyticProviderWeakPtr;
};

#undef UE_API
