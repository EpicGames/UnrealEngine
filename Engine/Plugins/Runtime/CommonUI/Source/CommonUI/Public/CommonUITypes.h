// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "Engine/DataAsset.h"
#include "InputCoreTypes.h"
#include "Styling/SlateBrush.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "CommonUITypes.generated.h"

#define UE_API COMMONUI_API

class UUserWidget;
enum class ECommonInputType : uint8;
struct FScrollBoxStyle;

class UCommonInputSubsystem;
class UInputAction;
class ULocalPlayer;
struct FInputActionValue;

UENUM(BlueprintType)
enum class EInputActionState : uint8
{
	/** Enabled, will call all callbacks */
	Enabled,
	/** Disabled, will call all the disabled callback if specified otherwise do nothing */
	Disabled,
	
	/** The common input reflector will not visualize this but still calls all callbacks. NOTE: Use this sparingly */
	Hidden,
	
	/** Hidden and disabled behaves as if it were never added with no callbacks being called */
	HiddenAndDisabled,	
};

USTRUCT(BlueprintType)
struct FCommonInputTypeInfo
{
	GENERATED_USTRUCT_BODY()

	UE_API FCommonInputTypeInfo();
private:
	/** Key this action is bound to	*/
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	FKey Key;

	/** Additional keys that also trigger this action (the primary Key is still used for display) */
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	TArray<FKey> AdditionalKeys;

public:

	/** Get the input type key bound to this input type, with a potential override */
	UE_API FKey GetKey() const;

	/** Get all keys (primary + additional) that trigger this action */
	UE_API TArray<FKey> GetAllKeys() const;

	/** Returns true if the given key matches the primary key or any additional key */
	UE_API bool IsKeyBound(const FKey& InKey) const;

	/** Get the input type key bound to this input type, with a potential override */
	void SetKey(FKey InKey)
	{
		Key = InKey;
	};

	/** EInputActionState::Enabled means that the state isn't overriden and the games dynamic control will work */
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	EInputActionState OverrrideState;

	/** Enables hold time if true */
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	bool bActionRequiresHold;

	/** The hold time in seconds */
	UPROPERTY(EditAnywhere, Category = "CommonInput", meta = (EditCondition = "bActionRequiresHold", ClampMin = "0.0", UIMin = "0.0"))
	float HoldTime;

	/**
	*	Time (in seconds) for hold progress to go from 1.0 (completed) to 0.0.
	*	If the hold interaction was interrupted, then hold progress starts to roll back decreasing its value.
	*	Set to 0.0 to disable the rollback functionality.
	*/
	UPROPERTY(EditAnywhere, Category = "CommonInput", meta = (EditCondition = "bActionRequiresHold", ClampMin = "0", UIMin = "0", ClampMax = "10.0", UIMax = "10"))
	float HoldRollbackTime;
	
	/** Override the brush specified by the Key Display Data  */
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	FSlateBrush OverrideBrush;

	bool operator==(const FCommonInputTypeInfo& Other) const
	{
		return Key == Other.Key &&
			AdditionalKeys == Other.AdditionalKeys &&
			OverrrideState == Other.OverrrideState &&
			bActionRequiresHold == Other.bActionRequiresHold &&
			HoldTime == Other.HoldTime && HoldRollbackTime == Other.HoldRollbackTime &&
			OverrideBrush == Other.OverrideBrush;
	}

	bool operator!=(const FCommonInputTypeInfo& Other) const
	{
		return !(*this == Other);
	}
};

USTRUCT(BlueprintType)
struct FCommonInputActionDataBase : public FTableRowBase
{
	GENERATED_BODY()

	UE_API FCommonInputActionDataBase();
	
	/** User facing name (used when NOT a hold action) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "CommonInput")
	FText DisplayName;

	/** User facing name used when it IS a hold action */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "CommonInput")
	FText HoldDisplayName;

	/** Priority in nav-bar */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "CommonInput")
	int32 NavBarPriority = 0;

protected:
	/**
	* Key to bind to for each input method
	*/
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	FCommonInputTypeInfo KeyboardInputTypeInfo;

	/**
	* Default input state for gamepads
	*/
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	FCommonInputTypeInfo DefaultGamepadInputTypeInfo;

	/**
	* Override the input state for each input method
	*/
	UPROPERTY(EditAnywhere, Category = "CommonInput", Meta = (GetOptions = "CommonInput.CommonInputBaseControllerData.GetRegisteredGamepads"))
	TMap<FName, FCommonInputTypeInfo> GamepadInputOverrides;

	/**
	* Override the displayed brush for each input method
	*/
	UPROPERTY(EditAnywhere, Category = "CommonInput")
	FCommonInputTypeInfo TouchInputTypeInfo;

	friend class UCommonInputActionDataProcessor;

public:
	UE_API bool CanDisplayInReflector(ECommonInputType InputType, const FName& GamepadName) const;

	UE_API virtual const FCommonInputTypeInfo& GetCurrentInputTypeInfo(const UCommonInputSubsystem* CommonInputSubsystem) const;

	UE_API virtual const FCommonInputTypeInfo& GetInputTypeInfo(ECommonInputType InputType, const FName& GamepadName) const;

	UE_API virtual bool IsKeyBoundToInputActionData(const FKey& Key) const;

	UE_API bool IsKeyBoundToInputActionData(const FKey& Key, const UCommonInputSubsystem* CommonInputSubsystem) const;

	UE_API FSlateBrush GetCurrentInputActionIcon(const UCommonInputSubsystem* CommonInputSubsystem) const;

	UE_API virtual void OnPostDataImport(const UDataTable* InDataTable, const FName InRowName, TArray<FString>& OutCollectedImportProblems) override;

	UE_API virtual bool HasHoldBindings() const;

	UE_API const FCommonInputTypeInfo& GetDefaultGamepadInputTypeInfo() const;

	UE_API bool HasGamepadInputOverride(const FName& GamepadName) const;

	UE_API void AddGamepadInputOverride(const FName& GamepadName, const FCommonInputTypeInfo& InputInfo);

	bool operator==(const FCommonInputActionDataBase& Other) const
	{
		return DisplayName.EqualTo(Other.DisplayName) &&
			HoldDisplayName.EqualTo(Other.HoldDisplayName) &&
			KeyboardInputTypeInfo == Other.KeyboardInputTypeInfo &&
			DefaultGamepadInputTypeInfo == Other.DefaultGamepadInputTypeInfo &&
			GamepadInputOverrides.OrderIndependentCompareEqual(Other.GamepadInputOverrides) &&
			TouchInputTypeInfo == Other.TouchInputTypeInfo;
	}

	bool operator!=(const FCommonInputActionDataBase& Other) const
	{
		check(this);
		return !(*this == Other);
	}

	UE_API bool Serialize(FArchive& Ar);
	UE_API void PostSerialize(const FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FCommonInputActionDataBase> : public TStructOpsTypeTraitsBase2<FCommonInputActionDataBase>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

/** 
 * Metadata CommonUI will try to acquire from Enhanced Input Mapping Contexts (IMC)
 * 
 * You can Inherit from this class if you have any info that may need to be parsed per platform
 * by CommonUI. IMC's can be specified per platform, so each platform may have different
 * Common Input Metadata
 * 
 * Note: We intentionally do not define any context-independant metadata. Even though some
 * metadata should be context-independant (Like NavBarPriority below). Locking it that info
 * to a seperate metadata type prevents any chance of future overriding. Instead, we prefer
 * info for all metadata to be set across all instances.
 */
UCLASS(MinimalAPI, Blueprintable, EditInlineNew, CollapseCategories)
class UCommonInputMetadata : public UObject
{
	GENERATED_BODY()

public:

	/** Priority in nav-bar */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "CommonInput")
	int32 NavBarPriority = 0;

	/** 
	 * Generic actions like accept or face button top will be subscribed to by multiple
	 * UI elements. These actions will not broadcast enhanced input action delegates
	 * such as "Triggered, Ongoing, Canceled, or Completed." Since those delegates
	 * would be fired by multiple UI elements.
	 * 
	 * Non-generic input actions will fire Enhanced Input events. However they will 
	 * not fire CommonUI action bindings (Since those can be manually fired in BP).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "CommonInput")
	bool bIsGenericInputAction = true;
};

/**
 * Interface for metadata objects specified in Enhanced Input Mapping Contexts (IMC).
 * 
 * We provide an interface since it's possible you may need the IMC metadata for 
 * non-CommonUI info. In this scenario you can implement this interface and CommonUI
 * will still be able to gather info it needs to function correctly with your 
 * Enhanced Input Actions / IMC's.
 * 
 * If you don't have any metadata needs or your UI IMC's are for CommonUI only, 
 * then you should use the provided 'UCommonMappingContextMetadata' below.
 */
UINTERFACE(MinimalAPI)
class UCommonMappingContextMetadataInterface : public UInterface
{
	GENERATED_BODY()
};

class ICommonMappingContextMetadataInterface
{
	GENERATED_BODY()

public:
	/** 
	 * Gets base info needed from CommonUI from this IMC metadata 
	 * Accepts InputAction as an arg to allow for user to create one metadata with multiple
	 * values per action, rather than having to create one metadata asset per unique value
	 * 
	 * @param InputAction the input action that may or may not be used to help determine correct metadata
	 */
	virtual const UCommonInputMetadata* GetCommonInputMetadata(const UInputAction* InInputAction) const = 0;
};

/**
 * Base CommonUI metadata implementation for specification in IMC's.
 * 
 * Utilizes a map of input actions to metadata to prevent users from having to create
 * multiple metadata assets / instances. Using this map is not mandatory.
 */
UCLASS(MinimalAPI, BlueprintType)
class UCommonMappingContextMetadata : public UDataAsset, public ICommonMappingContextMetadataInterface
{
	GENERATED_BODY()

public:
	/** Fallback or default metadata CommonUI relies on if no per-action meta is found below */
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadOnly, Category = "CommonInput")
	TObjectPtr<UCommonInputMetadata> EnhancedInputMetadata;

	/** Map of action to metadata, allows creation of single metadata asset rather than one per input action type */
	UPROPERTY(EditAnywhere, Instanced, BlueprintReadOnly, Category = "CommonInput")
	TMap<TObjectPtr<UInputAction>, TObjectPtr<const UCommonInputMetadata>> PerActionEnhancedInputMetadata;

public:
	UE_API virtual const UCommonInputMetadata* GetCommonInputMetadata(const UInputAction* InInputAction) const override;
};

class CommonUI
{
public:
	static UE_API void SetupStyles();
	static UE_API FScrollBoxStyle EmptyScrollBoxStyle;

	static UE_API const FCommonInputActionDataBase* GetInputActionData(const FDataTableRowHandle& InputActionRowHandle);
	static UE_API FSlateBrush GetIconForInputActions(const UCommonInputSubsystem* CommonInputSubsystem, const TArray<FDataTableRowHandle>& InputActions);

	static UE_API bool IsEnhancedInputSupportEnabled();

	static UE_API TObjectPtr<const UCommonInputMetadata> GetEnhancedInputActionMetadata(const UInputAction* InputAction);
	static UE_API void GetEnhancedInputActionKeys(const ULocalPlayer* LocalPlayer, const UInputAction* InputAction, TArray<FKey>& OutKeys);
	static UE_API void InjectEnhancedInputForAction(const ULocalPlayer* LocalPlayer, const UInputAction* InputAction, FInputActionValue RawValue);
	static UE_API FSlateBrush GetIconForEnhancedInputAction(const UCommonInputSubsystem* CommonInputSubsystem, const UInputAction* InputAction);
	static UE_API bool ActionValidForInputType(const ULocalPlayer* LocalPlayer, ECommonInputType InputType, const UInputAction* InputAction);
	static UE_API bool ActionValidForInputType(const ULocalPlayer* LocalPlayer, ECommonInputType InputType, const FCommonInputActionDataBase* InputAction);
	static UE_API bool IsKeyValidForInputType(const FKey& Key, ECommonInputType InputType);
	static UE_API FKey GetFirstKeyForInputType(const ULocalPlayer* LocalPlayer, ECommonInputType InputType, const UInputAction* InputAction);
};

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnItemClicked, UUserWidget*, Widget);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnItemSelected, UUserWidget*, Widget, bool, Selected);

#undef UE_API
