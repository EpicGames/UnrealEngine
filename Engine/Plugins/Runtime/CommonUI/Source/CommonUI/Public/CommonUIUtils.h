// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/WidgetTree.h"

class UWidget;
class UUserWidget;
class UWidgetTree;
class IWidgetCompilerLog;

namespace CommonUIUtils
{
	COMMONUI_API extern float UISmallTextScaling;
	extern float UISmallTextScalingThresholdMax;

	enum class ECollisionPolicy
	{
		/*Widgets in a switcher may occupy different slots of the switcher*/
		Allow,
		/*Widgets in a switcher must occupy the same slot of the switcher*/
		Require,
		/*Widgets in a switcher must occupy different slots of the switcher*/
		Forbid
	};

	bool COMMONUI_API ShouldDisplayMobileUISizes();

	bool COMMONUI_API ShouldScaleText();

	/** 
	 * Traverses the UserWidgets that the given widget resides in until we find one matching the given type (or we run out of parents)
	 * @param Widget The widget whose parent we're seeking
	 * @return The first parent of the given type, or nullptr if there is none
	 */
	template <typename UserWidgetT = UUserWidget>
	UserWidgetT* GetOwningUserWidget(const UWidget* Widget)
	{
		static_assert(TIsDerivedFrom<UserWidgetT, UUserWidget>::IsDerived, "CommonUIUtils::GetOwningUserWidgetTyped can only search for UUserWidget types");

		UUserWidget* ParentWidget = nullptr;
		const UWidget* CurrentWidget = Widget;

		while (CurrentWidget)
		{
			// The outer of every widget is the UWidgetTree it's in, and the outer of every UWidgetTree is the UUserWidget that owns it
			UWidgetTree* WidgetTree = Cast<UWidgetTree>(CurrentWidget->GetOuter());
			if (WidgetTree == nullptr)
			{
				break;
			}

			ParentWidget = Cast<UUserWidget>(WidgetTree->GetOuter());
			if (ParentWidget && ParentWidget->IsA<UserWidgetT>())
			{
				return Cast<UserWidgetT>(ParentWidget);
			}

			CurrentWidget = ParentWidget;
		}

		return nullptr;
	}

	/**
	 * Traverses the UserWidgets the given widget resides in and prints their names. Useful for debug logging.
	 * @param Widget The widget whose parents we'll print
	 * @return An FString containing the names of the parents
	 */
	FString PrintAllOwningUserWidgets(const UWidget* Widget);

#if WITH_EDITOR
	/**
	 * Validates that a given widget tree hierarchy satisfies the condition that a given widget contains N other widgets (optionally requiring individual slots for each)
	 * Intended to be called from within a widget subclass's ValidateCompiledWidgetTree override
	 * If the given hierarchy does not satisfy the requirements, we'll log an error in the compiler log
	 *
	 *	void MyWidget::ValidateCompiledWidgetTree(const UWidgetTree& BlueprintWidgetTree, IWidgetCompilerLog& CompileLog) const
	 *	{
	 *		Super::ValidateCompiledWidgetTree(BlueprintWidgetTree, CompileLog);
	 *
	 *		const CommonUIUtils::ECollisionPolicy CollisionPolicy = CommonUIUtils::ECollisionPolicy::Forbid;
	 *		CommonUIUtils::ValidateBoundWidgetHierarchy(BlueprintWidgetTree, CompileLog, CollisionPolicy,
	 *			GET_MEMBER_NAME_CHECKED(MyWidget, MySubWidget1Name),
	 *			GET_MEMBER_NAME_CHECKED(MyWidget, MySubWidget2Name));
	 *	}
	 *
	 */
	COMMONUI_API void ValidateBoundWidgetHierarchy(const UWidgetTree& WidgetTree, IWidgetCompilerLog& CompileLog, ECollisionPolicy CollisionPolicy, FName ParentWidgetName, TArray<FName>&& ChildNames);

	template <typename... ChildNameArgs>
	void ValidateBoundWidgetHierarchy(const UWidgetTree& WidgetTree, IWidgetCompilerLog& CompileLog, ECollisionPolicy CollisionPolicy, FName ParentWidgetName, ChildNameArgs&&... ChildNames)
	{
		ValidateBoundWidgetHierarchy(WidgetTree, CompileLog, CollisionPolicy, ParentWidgetName, TArray<FName>{ChildNames...});
	}
#endif
};
