// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/LocalPlayerSubsystem.h"
#include "GameplayTagContainer.h"
#include "UObject/WeakObjectPtr.h"
#include "CommonUIVisibilitySubsystem.generated.h"

#define UE_API COMMONUI_API

class UWidget;
class ULocalPlayer;
class APlayerController;
struct FGameplayTagContainer;
class UCommonUIVisibilitySubsystem;
enum class ECommonInputType : uint8;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FHardwareVisibilityTagsChangedDynamicEvent, UCommonUIVisibilitySubsystem*, TagSubsystem);

UCLASS(MinimalAPI, DisplayName = "UI Visibility Subsystem")
class UCommonUIVisibilitySubsystem : public ULocalPlayerSubsystem
{
	GENERATED_BODY()

public:
	static UE_API UCommonUIVisibilitySubsystem* Get(const ULocalPlayer* LocalPlayer);
	static UE_API UCommonUIVisibilitySubsystem* GetChecked(const ULocalPlayer* LocalPlayer);

	UE_API UCommonUIVisibilitySubsystem();
	
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	DECLARE_EVENT_OneParam(UCommonUIVisibilitySubsystem, FHardwareVisibilityTagsChangedEvent, UCommonUIVisibilitySubsystem*);
	FHardwareVisibilityTagsChangedEvent OnVisibilityTagsChanged;

	/**
	 * Get the visibility tags currently in play (the combination of platform traits and current input tags).
	 * These can change over time, if input mode changes, or other groups are removed/added.
	 */
	const FGameplayTagContainer& GetVisibilityTags() const { return ComputedVisibilityTags; }
	
	/* Returns true if the player currently has the specified visibility tag
	 * (note: this value should not be cached without listening for OnVisibilityTagsChanged as it can change at runtime)
	 */
	bool HasVisibilityTag(const FGameplayTag VisibilityTag) const { return ComputedVisibilityTags.HasTag(VisibilityTag); }

	UE_API void AddUserVisibilityCondition(const FGameplayTag UserTag);
	UE_API void RemoveUserVisibilityCondition(const FGameplayTag UserTag);

#if WITH_EDITOR
	static UE_API void SetDebugVisibilityConditions(const FGameplayTagContainer& TagsToEnable, const FGameplayTagContainer& TagsToSuppress);
#endif

protected:
	UE_API void RefreshVisibilityTags();
	UE_API void OnInputMethodChanged(ECommonInputType CurrentInputType);
	UE_API virtual FGameplayTagContainer ComputeVisibilityTags() const;

private:
	FGameplayTagContainer ComputedVisibilityTags;
	FGameplayTagContainer UserVisibilityTags;
#if WITH_EDITOR
	static UE_API FGameplayTagContainer DebugTagsToEnable;
	static UE_API FGameplayTagContainer DebugTagsToSuppress;
#endif
};

#undef UE_API
