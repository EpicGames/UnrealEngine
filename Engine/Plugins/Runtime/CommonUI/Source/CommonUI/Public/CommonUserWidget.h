// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "Input/UIActionBindingHandle.h"

#include "CommonUserWidget.generated.h"

#define UE_API COMMONUI_API

class UInputMappingContext;
class UInputAction;
class UCommonInputSubsystem;
class UCommonUISubsystemBase;
class FSlateUser;

struct FUIActionTag;
struct FBindUIActionArgs;
enum class ECommonInputMode : uint8;

/** Enum to represent which common UI tree node takes scroll input */
enum ECommonUIScrollRecipientOwningNodeSource : uint8
{
	/** The scroll recipient owner takes scroll input */
	ScrollRecipient,

	/** The userwidget that is registering the scroll recipient takes scroll input. It may or may not be the same as scroll recipient owner. */
	RegisteringUserWidget
};

UCLASS(MinimalAPI, ClassGroup = UI, meta = (Category = "Common UI", DisableNativeTick))
class UCommonUserWidget : public UUserWidget
{
	GENERATED_UCLASS_BODY()

public:
	/** Sets whether or not this widget will consume ALL pointer input that reaches it */
	UFUNCTION(BlueprintCallable, Category = CommonUserWidget)
	UE_API void SetConsumePointerInput(bool bInConsumePointerInput);

	/** Add a widget to the list of widgets to get scroll events for this input root node */
	UFUNCTION(BlueprintCallable, Category = CommonUserWidget)
	UE_API void RegisterScrollRecipientExternal(const UWidget* AnalogScrollRecipient);

	/** Remove a widget from the list of widgets to get scroll events for this input root node */
	UFUNCTION(BlueprintCallable, Category = CommonUserWidget)
	UE_API void UnregisterScrollRecipientExternal(const UWidget* AnalogScrollRecipient);

#if WITH_EDITOR
	UE_API virtual class UWorld* GetWorld() const override;
#endif

public:

	const TArray<FUIActionBindingHandle>& GetActionBindings() const { return ActionBindings; }
	const TArray<TWeakObjectPtr<const UWidget>> GetScrollRecipients() const { return ScrollRecipients; }

	/**
	 * Convenience methods for menu action registrations (any UWidget can register via FCommonUIActionRouter directly, though generally that shouldn't be needed).
	 * Persistent bindings are *always* listening for input while registered, while normal bindings are only listening when all of this widget's activatable parents are activated.
	 */
	UE_API FUIActionBindingHandle RegisterUIActionBinding(const FBindUIActionArgs& BindActionArgs);
	
	// Helper to register a UI Enhanced Input Action from BP, they are meant to be global UI action to work with Enhanced Input
	// See the UCommonMappingContextMetadata set in the Input Action User Settings Section
	UFUNCTION(BlueprintCallable, Category = CommonUserWidget)
	FUIActionBindingHandle RegisterUIAction(const UInputAction* EnhancedInputAction, bool ShouldDisplayInActionBar = true);
	
	// Similar to RegisterUIAction but uses a Mapping Context 
	UFUNCTION(BlueprintCallable, Category = CommonUserWidget)
	TArray<FUIActionBindingHandle> RegisterUIActionsFromMappingContext(const UInputMappingContext* MappingContext, bool ShouldDisplayInActionBar = true);
	
	// Remove a register UI Action. Will silently fail as CommonActionRouter doesn't give Feedback
	UFUNCTION(BlueprintCallable, Category = CommonUserWidget)
	void RemoveUIAction(FUIActionBindingHandle ActionBinding);
	
	// Remove All the UI Action that were registered on this widget
	UFUNCTION(BlueprintCallable, Category = CommonUserWidget)
	void RemoveAllUIActionBinding();

	UE_API void RemoveActionBinding(FUIActionBindingHandle ActionBinding);
	UE_API void AddActionBinding(FUIActionBindingHandle ActionBinding);

protected:
	virtual ERequiresLegacyPlayer GetLegacyPlayerRequirement() const override { return ERequiresLegacyPlayer::No; }

	UE_API virtual void OnWidgetRebuilt() override;
	UE_API virtual void NativeDestruct() override;
	
	UE_API virtual FReply NativeOnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply NativeOnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply NativeOnMouseWheel(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply NativeOnMouseButtonDoubleClick(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply NativeOnTouchGesture(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent) override;
	UE_API virtual FReply NativeOnTouchStarted(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent) override;
	UE_API virtual FReply NativeOnTouchMoved(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent) override;
	UE_API virtual FReply NativeOnTouchEnded(const FGeometry& InGeometry, const FPointerEvent& InGestureEvent) override;
	
	UE_API UCommonInputSubsystem* GetInputSubsystem() const;
	UE_API UCommonUISubsystemBase* GetUISubsystem() const;
	UE_API TSharedPtr<FSlateUser> GetOwnerSlateUser() const;

	template <typename GameInstanceT = UGameInstance>
	GameInstanceT& GetGameInstanceChecked() const
	{
		GameInstanceT* GameInstance = GetGameInstance<GameInstanceT>();
		check(GameInstance);
		return *GameInstance;
	}

	template <typename PlayerControllerT = APlayerController>
	PlayerControllerT& GetOwningPlayerChecked() const
	{
		PlayerControllerT* PC = GetOwningPlayer<PlayerControllerT>();
		check(PC);
		return *PC;
	}

	UE_API void RegisterScrollRecipient(const UWidget& AnalogScrollRecipient, ECommonUIScrollRecipientOwningNodeSource OwningNodeSource = ECommonUIScrollRecipientOwningNodeSource::ScrollRecipient);
	UE_API void UnregisterScrollRecipient(const UWidget& AnalogScrollRecipient, ECommonUIScrollRecipientOwningNodeSource OwningNodeSource = ECommonUIScrollRecipientOwningNodeSource::ScrollRecipient);

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif // WITH_EDITOR

	/** True to generally display this widget's actions in the action bar, assuming it has actions. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (AllowPrivateAccess = true))
	bool bDisplayInActionBar = false;

private:

	/** Set this to true if you don't want any pointer (mouse and touch) input to bubble past this widget */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Input, meta = (AllowPrivateAccess = true))
	bool bConsumePointerInput = false;

private:

	TArray<FUIActionBindingHandle> ActionBindings;
	TArray<TWeakObjectPtr<const UWidget>> ScrollRecipients;
};

#undef UE_API
