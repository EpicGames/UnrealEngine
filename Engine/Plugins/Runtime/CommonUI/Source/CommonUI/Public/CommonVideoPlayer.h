// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"

#include "CommonVideoPlayer.generated.h"

#define UE_API COMMONUI_API

class SImage;
class UMaterial;
class UMediaSource;
class UMediaPlayer;
class UMediaTexture;
class UMediaSoundComponent;
class USoundClass;

enum class EMediaEvent;

UCLASS(MinimalAPI, ClassGroup = UI, meta = (Category = "Common UI"))
class UCommonVideoPlayer : public UWidget
{
	GENERATED_BODY()

public:
	UE_API UCommonVideoPlayer(const FObjectInitializer& Initializer);
	UE_API virtual void PostInitProperties() override;

	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void SetVideo(UMediaSource* NewVideo);
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void Seek(float PlaybackTime);
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void Close();

	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void SetPlaybackRate(float PlaybackRate);
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void SetLooping(bool bShouldLoopPlayback);
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void SetIsMuted(bool bInIsMuted);
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void SetShouldMatchSize(bool bInMatchSize);

	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void Play();
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void Reverse();
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void Pause();
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API void PlayFromStart();

	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API float GetVideoDuration() const;
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API float GetPlaybackTime() const;
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API float GetPlaybackRate() const;

	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API bool IsLooping() const;
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API bool IsPaused() const;
	UFUNCTION(BlueprintCallable, Category="Video Player")
	UE_API bool IsPlaying() const;
	UFUNCTION(BlueprintCallable, Category="Video Player")
	bool IsMuted() const { return bIsMuted; }

	FSimpleMulticastDelegate& OnPlaybackResumed() { return OnPlaybackResumedEvent; }
	FSimpleMulticastDelegate& OnPlaybackPaused() { return OnPlaybackPausedEvent; }
	FSimpleMulticastDelegate& OnPlaybackComplete() { return OnPlaybackCompleteEvent; }

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	UE_API void PlayInternal() const;
	const UMediaPlayer& GetMediaPlayer() const { return *MediaPlayer; }
	UE_API virtual void HandleMediaPlayerEvent(EMediaEvent EventType);
	UE_API virtual void PlaybackTick(double InCurrentTime, float InDeltaTime);

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif // WITH_EDITOR

private:
	UE_API EActiveTimerReturnType HandlePlaybackTick(double InCurrentTime, float InDeltaTime);

private:
	UPROPERTY(EditAnywhere, Category = VideoPlayer)
	TObjectPtr<UMediaSource> Video;

	// Should we match the size of the media source when it opens?
	UPROPERTY(EditAnywhere, Category = VideoPlayer)
	bool bMatchSize = false;

	UPROPERTY(Transient)
	TObjectPtr<UMediaPlayer> MediaPlayer;

	UPROPERTY(Transient)
	TObjectPtr<UMediaTexture> MediaTexture;

	UPROPERTY(Transient)
	TObjectPtr<UMaterial> VideoMaterial;

	UPROPERTY(Transient)
	TObjectPtr<UMediaSoundComponent> SoundComponent;

	UPROPERTY(Transient)
	FSlateBrush VideoBrush;

	mutable FSimpleMulticastDelegate OnPlaybackResumedEvent;
	mutable FSimpleMulticastDelegate OnPlaybackPausedEvent;
	mutable FSimpleMulticastDelegate OnPlaybackCompleteEvent;


	bool bIsMuted = false;
	TSharedPtr<SImage> MyImage;
};

#undef UE_API
