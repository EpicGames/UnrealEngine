// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Overlay.h"

#include "CommonVisibilitySwitcher.generated.h"

#define UE_API COMMONUI_API

/**
 * Basic switcher that toggles visibility on its children to only show one widget at a time. Activates visible widget if possible.
 */
UCLASS(MinimalAPI, meta = (DisableNativeTick))
class UCommonVisibilitySwitcher : public UOverlay
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	UE_API void SetActiveWidgetIndex(int32 Index);

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	int32 GetActiveWidgetIndex() const { return ActiveWidgetIndex; }

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	UE_API UWidget* GetActiveWidget() const;

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	UE_API void SetActiveWidget(const UWidget* Widget);

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	UE_API void IncrementActiveWidgetIndex(bool bAllowWrapping = true);

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	UE_API void DecrementActiveWidgetIndex(bool bAllowWrapping = true);

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	UE_API void ActivateVisibleSlot();

	UFUNCTION(BlueprintCallable, Category = CommonVisibilitySwitcher)
	UE_API void DeactivateVisibleSlot();

	UFUNCTION(BlueprintPure, Category = CommonVisibilitySwitcher)
	bool IsCurrentlySwitching() const { return bCurrentlySwitching; }

	UE_API UWidget* GetWidgetAtIndex(int32 Index) const;

	UE_API virtual void SynchronizeProperties() override;

	UE_API void MoveChild(int32 CurrentIdx, int32 NewIdx);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActiveWidgetIndexChanged, int32)
	FOnActiveWidgetIndexChanged& OnActiveWidgetIndexChanged() const { return OnActiveWidgetIndexChangedEvent; }

#if WITH_EDITOR
public:

	UE_API virtual const FText GetPaletteCategory() override;
	UE_API virtual void OnDescendantSelectedByDesigner(UWidget* DescendantWidget) override;
	UE_API virtual void OnDescendantDeselectedByDesigner(UWidget* DescendantWidget) override;

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

protected:

	UE_API virtual void ValidateCompiledDefaults(class IWidgetCompilerLog& CompileLog) const override;

private:

	int32 DesignTime_ActiveIndex = INDEX_NONE;
#endif

protected:

	UE_API virtual void OnWidgetRebuilt() override;
	UE_API virtual UClass* GetSlotClass() const override;
	UE_API virtual void OnSlotAdded(UPanelSlot* InSlot) override;
	UE_API virtual void OnSlotRemoved(UPanelSlot* InSlot) override;

protected:

	UE_API virtual void SetActiveWidgetIndex_Internal(int32 Index, bool bBroadcastChange = true);
	UE_API void ResetSlotVisibilities();

	UPROPERTY(EditAnywhere, Category = CommonVisibilitySwitcher)
	ESlateVisibility ShownVisibility = ESlateVisibility::SelfHitTestInvisible;

	UPROPERTY(EditAnywhere, Category = CommonVisibilitySwitcher, meta = (ClampMin = -1))
	int32 ActiveWidgetIndex = 0;

	// Whether or not to automatically activate a slot when it becomes visible
	UPROPERTY(EditAnywhere, Category = CommonVisibilitySwitcher)
	bool bAutoActivateSlot = true;

	// Whether or not to activate the first slot if one is added dynamically
	UPROPERTY(EditAnywhere, Category = CommonVisibilitySwitcher)
	bool bActivateFirstSlotOnAdding = false;

	bool bCurrentlySwitching = false;

	mutable FOnActiveWidgetIndexChanged OnActiveWidgetIndexChangedEvent;
};

#undef UE_API
