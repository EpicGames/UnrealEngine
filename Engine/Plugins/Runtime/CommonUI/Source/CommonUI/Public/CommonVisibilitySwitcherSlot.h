// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/OverlaySlot.h"

#include "CommonVisibilitySwitcherSlot.generated.h"

#define UE_API COMMONUI_API

class SOverlay;

class SBox;
enum class ESlateVisibility : uint8;

UCLASS(MinimalAPI)
class UCommonVisibilitySwitcherSlot : public UOverlaySlot
{
	GENERATED_BODY()

public:

	UE_API UCommonVisibilitySwitcherSlot(const FObjectInitializer& Initializer);

	UE_API virtual void BuildSlot(TSharedRef<SOverlay> InOverlay) override;

	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	UE_API void SetSlotVisibility(ESlateVisibility Visibility);

	const TSharedPtr<SBox>& GetVisibilityBox() const { return VisibilityBox; }

private:

	TSharedPtr<SBox> VisibilityBox;
};

#undef UE_API
