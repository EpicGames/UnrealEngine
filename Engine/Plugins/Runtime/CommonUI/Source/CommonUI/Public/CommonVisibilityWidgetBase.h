// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonBorder.h"
#include "CommonVisibilityWidgetBase.generated.h"

#define UE_API COMMONUI_API

enum class ECommonInputType : uint8;
enum class ESlateVisibility : uint8;

/**
 * A container that controls visibility based on Input type and Platform
 *
 */
UCLASS(MinimalAPI, Deprecated)
class UDEPRECATED_UCommonVisibilityWidgetBase : public UCommonBorder
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Visibility", meta = (GetOptions = GetRegisteredPlatforms))
	TMap<FName, bool> VisibilityControls;

	UPROPERTY(EditAnywhere, Category = "Visibility")
	bool bShowForGamepad;

	UPROPERTY(EditAnywhere, Category = "Visibility")
	bool bShowForMouseAndKeyboard;

	UPROPERTY(EditAnywhere, Category = "Visibility")
	bool bShowForTouch;

	UPROPERTY(EditAnywhere, Category = "Visibility")
	ESlateVisibility VisibleType;
	
	UPROPERTY(EditAnywhere, Category = "Visibility")
	ESlateVisibility HiddenType;

protected:
	// Begin UWidget
	UE_API virtual void OnWidgetRebuilt() override;
	// End UWidget

	UE_API void UpdateVisibility();

	UE_API void ListenToInputMethodChanged(bool bListen = true);

	UE_API void HandleInputMethodChanged(ECommonInputType input);

	UFUNCTION()
	static UE_API const TArray<FName>& GetRegisteredPlatforms();
};

#undef UE_API
