// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SizeBox.h"

#include "CommonVisualAttachment.generated.h"

#define UE_API COMMONUI_API

/**
 * Adds a widget as a zero-size attachment to another. Think icons to the left of labels, without changing the computed size of the label.
 */
UCLASS(MinimalAPI, ClassGroup = UI, meta = (Category = "Common UI"))
class UCommonVisualAttachment : public USizeBox
{
	GENERATED_BODY()

public:
	UE_API UCommonVisualAttachment(const FObjectInitializer& ObjectInitializer);

	UE_DEPRECATED(5.4, "Direct access to ContentAnchor is deprecated. Please use the getter or setter.")
	/** Content Anchor Point as a ratio of the content size. Use (1.0, 1.0) to anchor the content on the bottom right, (0.0) to anchor top left, (0.5, 0.5) to anchor centered.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Child Layout")
	FVector2D ContentAnchor;

	/** Get Content Anchor Point*/
	UE_API FVector2D GetContentAnchor() const;

	/** Set Content Anchor Point*/
	UE_API void SetContentAnchor(FVector2D InContentAnchor);

protected:
	// UVisual interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

private:
	TSharedPtr<class SVisualAttachmentBox> MyAttachmentBox;
};

#undef UE_API
