// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PanelWidget.h"
#include "SWidgetCarousel.h"

class UCommonWidgetCarousel;

#include "CommonWidgetCarousel.generated.h"

#define UE_API COMMONUI_API

enum EHorizontalAlignment : int;
enum EVerticalAlignment : int;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnCurrentPageIndexChanged, UCommonWidgetCarousel*, CarouselWidget, int32, CurrentPageIndex);

/**
 * A widget switcher is like a tab control, but without tabs. At most one widget is visible at time.
 */
UCLASS(MinimalAPI, Blueprintable)
class UCommonWidgetCarousel : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:
	/** The slot index to display */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Carousel", meta=( UIMin=0, ClampMin=0 ))
	int32 ActiveWidgetIndex;

protected:

	/** How quickly the carousel transitions when changing active widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetMoveSpeed", BlueprintSetter="SetMoveSpeed", Category="Carousel", meta=( UIMin=0, ClampMin=0 ))
	float MoveSpeed;

	/** Whether we should cache children to prevent them from being destroyed when not visible. Enable this to avoid ConstructWidget costs when changing index, disable to save memory  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetCacheChildren", Setter = "SetCacheChildren", BlueprintGetter = "GetCacheChildren", BlueprintSetter = "SetCacheChildren", Category = "Carousel")
	bool bCacheChildren;
	
public:

	/** Gets the slot index of the currently active widget */
	UFUNCTION(BlueprintCallable, Category="Carousel")
	UE_API int32 GetActiveWidgetIndex() const;

	/** Activates the widget at the specified index. */
	UFUNCTION(BlueprintCallable, Category="Carousel")
	UE_API virtual void SetActiveWidgetIndex( int32 Index );

	/** Activates the widget and makes it the active index. */
	UFUNCTION(BlueprintCallable, Category="Carousel")
	UE_API virtual void SetActiveWidget(UWidget* Widget);

	/** Get a widget at the provided index */
	UFUNCTION( BlueprintCallable, Category = "Carousel" )
	UE_API UWidget* GetWidgetAtIndex( int32 Index ) const;

	UFUNCTION(BlueprintCallable, Category="Carousel")
	UE_API void BeginAutoScrolling(float ScrollInterval = 10);

	UFUNCTION(BlueprintCallable, Category="Carousel")
	UE_API void EndAutoScrolling();

	UFUNCTION(BlueprintCallable, Category="Carousel")
	UE_API void NextPage();

	UFUNCTION(BlueprintCallable, Category="Carousel")
	UE_API void PreviousPage();

	/** Sets the Move Speed. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UE_API void SetMoveSpeed(float InMoveSpeed);

	/** Gets the Move Speed. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UE_API float GetMoveSpeed() const;

	/** Sets the current caching behavior. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UE_API void SetCacheChildren(bool InCacheChildren);

	/** Gets the current caching behavior. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UE_API bool GetCacheChildren() const;
	
	UPROPERTY(BlueprintAssignable, Category = "Carousel")
	FOnCurrentPageIndexChanged OnCurrentPageIndexChanged;

	// UWidget interface
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UWidget interface

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
	UE_API virtual void OnDescendantSelectedByDesigner(UWidget* DescendantWidget) override;
	UE_API virtual void OnDescendantDeselectedByDesigner(UWidget* DescendantWidget) override;

	// UObject interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif

protected:
	UE_API bool AutoScrollCallback(float DeltaTime);

	UE_API TSharedRef<SWidget> OnGenerateWidgetForCarousel(UPanelSlot* PanelSlot);
	UE_API void HandlePageChanged(int32 PageIndex);

	// UPanelWidget
	UE_API virtual UClass* GetSlotClass() const override;
	UE_API virtual void OnSlotAdded(UPanelSlot* InSlot) override;
	UE_API virtual void OnSlotRemoved(UPanelSlot* InSlot) override;
	// End UPanelWidget

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

protected:
	FTSTicker::FDelegateHandle TickerHandle;

	TSharedPtr< SWidgetCarousel<UPanelSlot*> > MyCommonWidgetCarousel;

	TArray< TSharedRef<SWidget> > CachedSlotWidgets;
};

#undef UE_API
