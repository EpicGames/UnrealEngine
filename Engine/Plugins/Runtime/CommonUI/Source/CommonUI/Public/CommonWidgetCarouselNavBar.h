// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "CommonWidgetCarouselNavBar.generated.h"

#define UE_API COMMONUI_API

class UCommonWidgetCarousel;
class UCommonButtonBase;
class UCommonButtonGroupBase;
class SHorizontalBox;

/**
 * A Navigation control for a Carousel
 */
UCLASS(MinimalAPI, Blueprintable)
class UCommonWidgetCarouselNavBar : public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "CarouselNavBar")
	TSubclassOf<UCommonButtonBase> ButtonWidgetType;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "CarouselNavBar")
	FMargin ButtonPadding;

	/**
	 * Establishes the Widget Carousel instance that this Nav Bar should interact with
	 * @param CommonCarousel The carousel that this nav bar should be associated with and manipulate
	 */
	UFUNCTION(BlueprintCallable, Category = "CarouselNavBar")
	UE_API void SetLinkedCarousel(UCommonWidgetCarousel* CommonCarousel);


	// UWidget interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UWidget interface

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	UFUNCTION()
	UE_API void HandlePageChanged(UCommonWidgetCarousel* CommonCarousel, int32 PageIndex);

	UFUNCTION()
	UE_API void HandleButtonClicked(UCommonButtonBase* AssociatedButton, int32 ButtonIndex);

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	UE_API void RebuildButtons();

protected:
	TSharedPtr<SHorizontalBox> MyContainer;
	
	UPROPERTY()
	TObjectPtr<UCommonWidgetCarousel> LinkedCarousel;

	UPROPERTY()
	TObjectPtr<UCommonButtonGroupBase> ButtonGroup;

	UPROPERTY()
	TArray<TObjectPtr<UCommonButtonBase>> Buttons;
};

#undef UE_API
