// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Groups/CommonWidgetGroupBase.h"
#include "CommonButtonBase.h"

#include "CommonButtonGroupBase.generated.h"

#define UE_API COMMONUI_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FSimpleButtonBaseGroupDelegate, UCommonButtonBase*, AssociatedButton, int32, ButtonIndex );
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FButtonLostSelectionDelegate, UCommonButtonBase*, PreviouslySelectedButton);
DECLARE_DYNAMIC_MULTICAST_DELEGATE( FOnSelectionCleared );

/** 
 * Manages an arbitrary collection of CommonButton widgets.
 * Ensures that no more (and optionally, no less) than one button in the group is selected at a time
 */
UCLASS(MinimalAPI,  BlueprintType )
class UCommonButtonGroupBase : public UCommonWidgetGroupBase
{
	GENERATED_BODY()
public:
	UE_API UCommonButtonGroupBase();

	virtual TSubclassOf<UWidget> GetWidgetType() const override { return UCommonButtonBase::StaticClass(); }

	/** 
	 * Sets whether the group should always have a button selected.
	 * @param bRequireSelection True to force the group to always have a button selected.
	 * If true and nothing is selected, will select the first entry. If empty, will select the first button added.
	 */
	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API void SetSelectionRequired(bool bRequireSelection);

	UE_API bool GetSelectionRequired() const;

	/** Deselects all buttons in the group. */
	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API void DeselectAll();

	/** 
	 * Selects the next button in the group 
	 * @param bAllowWrap Whether to wrap to the first button if the last one is currently selected
	 * @param bAllowSound Whether the selected button should play its click sound
	 */
	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API void SelectNextButton(bool bAllowWrap = true, const bool bAllowSound = true);

	/** 
	 * Selects the previous button in the group 
	 * @param bAllowWrap Whether to wrap to the first button if the last one is currently selected
	 * @param bAllowSound Whether the selected button should play its click sound
	 */
	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API void SelectPreviousButton(bool bAllowWrap = true, const bool bAllowSound = true);

	/**
	 * Selects a button at a specific index in the group. Clears all selection if given an invalid index.
	 * @param ButtonIndex The index of the button in the group to select
	 * @param bAllowSound Whether the selected button should play its click sound
	 */
	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API void SelectButtonAtIndex(int32 ButtonIndex, const bool bAllowSound = true);

	/**
	 * Get the index of the currently selected button, if any.
	 * @param The index of the currently selected button in the group, or -1 if there is no selected button.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = BaseButtonGroup)
	UE_API int32 GetSelectedButtonIndex() const;

	/**
	 * Get the index of the currently hovered button, if any.
	 * @param The index of the currently hovered button in the group, or -1 if there is no hovered button.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = BaseButtonGroup)
	UE_API int32 GetHoveredButtonIndex() const;

	/**
	 * Find the button index of the specified button, if possible
	 * @param ButtonToFind	Button to find the index of
	 * @return Index of the button in the group. INDEX_NONE if not found
	 */
	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API int32 FindButtonIndex(const UCommonButtonBase* ButtonToFind) const;

	UE_API void ForEach(TFunctionRef<void(UCommonButtonBase&, int32)> Functor);

	DECLARE_EVENT_TwoParams(UCommonButtonGroupBase, FNativeSimpleButtonBaseGroupDelegate, UCommonButtonBase*, int32);

	UPROPERTY(BlueprintAssignable)
	FButtonLostSelectionDelegate OnSelectedButtonLostSelection;

	UPROPERTY(BlueprintAssignable)
	FSimpleButtonBaseGroupDelegate OnSelectedButtonBaseChanged;
	FNativeSimpleButtonBaseGroupDelegate NativeOnSelectedButtonBaseChanged;

	UPROPERTY(BlueprintAssignable)
	FSimpleButtonBaseGroupDelegate OnHoveredButtonBaseChanged;
	FNativeSimpleButtonBaseGroupDelegate NativeOnHoveredButtonBaseChanged;

	UPROPERTY(BlueprintAssignable)
	FSimpleButtonBaseGroupDelegate OnButtonBaseClicked;
	FNativeSimpleButtonBaseGroupDelegate NativeOnButtonBaseClicked;

	UPROPERTY(BlueprintAssignable)
	FSimpleButtonBaseGroupDelegate OnButtonBaseDoubleClicked;
	FNativeSimpleButtonBaseGroupDelegate NativeOnButtonBaseDoubleClicked;

	DECLARE_EVENT(UCommonButtonGroupBase, FNativeOnSelectionCleared);

	UPROPERTY(BlueprintAssignable)
	FOnSelectionCleared OnSelectionCleared;
	FNativeOnSelectionCleared NativeOnSelectionCleared;
		
	UPROPERTY(BlueprintAssignable)
	FSimpleButtonBaseGroupDelegate OnButtonBaseLockClicked;
	FNativeSimpleButtonBaseGroupDelegate NativeOnButtonBaseLockClicked;

	UPROPERTY(BlueprintAssignable)
	FSimpleButtonBaseGroupDelegate OnButtonBaseLockDoubleClicked;
	FNativeSimpleButtonBaseGroupDelegate NativeOnButtonBaseLockDoubleClicked;

	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API UCommonButtonBase* GetButtonBaseAtIndex(int32 Index) const;
	
	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API UCommonButtonBase* GetSelectedButtonBase() const;

	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API bool HasAnyButtons() const;

	UFUNCTION(BlueprintCallable, Category = BaseButtonGroup)
	UE_API int32 GetButtonCount() const;

protected:
	/** If true, the group will force that a button be selected at all times */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = BaseButtonGroup, meta = (ExposeOnSpawn="true"))
	bool bSelectionRequired;

	UE_API virtual void OnWidgetAdded( UWidget* NewWidget ) override;
	UE_API virtual void OnWidgetRemoved( UWidget* OldWidget ) override;
	UE_API virtual void OnRemoveAll() override;

	UFUNCTION()
	UE_API virtual void OnSelectionStateChangedBase( UCommonButtonBase* BaseButton, bool bIsSelected );

	UFUNCTION()
	UE_API virtual void OnHandleButtonBaseClicked(UCommonButtonBase* BaseButton);

	UFUNCTION()
	UE_API virtual void OnHandleButtonBaseDoubleClicked(UCommonButtonBase* BaseButton);

	UFUNCTION()
	UE_API virtual void OnButtonBaseHovered(UCommonButtonBase* BaseButton);

	UFUNCTION()
	UE_API virtual void OnButtonBaseUnhovered(UCommonButtonBase* BaseButton);
		
	UFUNCTION()
	UE_API virtual void OnHandleButtonBaseLockClicked(UCommonButtonBase* BaseButton);

	UFUNCTION()
	UE_API virtual void OnHandleButtonBaseLockDoubleClicked(UCommonButtonBase* BaseButton);

protected:

	TArray<TWeakObjectPtr<UCommonButtonBase>> Buttons;
	int32 SelectedButtonIndex = INDEX_NONE;
	int32 HoveredButtonIndex = INDEX_NONE;
};

#undef UE_API
