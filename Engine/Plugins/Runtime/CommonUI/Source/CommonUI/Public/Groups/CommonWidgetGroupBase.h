// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "CommonWidgetGroupBase.generated.h"

#define UE_API COMMONUI_API

/**
 * Base class for CommonUI widget groups, currently only used for button groups
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UCommonWidgetGroupBase : public UObject
{
	GENERATED_BODY()

public:
	UE_API UCommonWidgetGroupBase();

	virtual TSubclassOf<UWidget> GetWidgetType() const { return UWidget::StaticClass(); }

	UFUNCTION(BlueprintCallable, Category = Group)
	UE_API void AddWidget(UWidget* InWidget);

	UFUNCTION(BlueprintCallable, Category = Group)
	UE_API void AddWidgets(const TArray<UWidget*>& Widgets);

	UFUNCTION(BlueprintCallable, Category = Group)
	UE_API void RemoveWidget(UWidget* InWidget);

	UFUNCTION(BlueprintCallable, Category = Group)
	UE_API void RemoveAll();

	template <typename WidgetT>
	void AddWidgets(const TArray<WidgetT>& Widgets)
	{
		for (UWidget* Widget : Widgets)
		{
			AddWidget(Widget);
		}
	}

protected:
	virtual void OnWidgetAdded(UWidget* NewWidget) PURE_VIRTUAL(UCommonWidgetGroupBase::OnWidgetAdded, );
	virtual void OnWidgetRemoved(UWidget* OldWidget) PURE_VIRTUAL(UCommonWidgetGroupBase::OnWidgetRemoved, );
	virtual void OnRemoveAll() PURE_VIRTUAL(UCommonWidgetGroupBase::OnRemoveAll, );
};

#undef UE_API
