// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Modules/ModuleManager.h"

struct FStreamableManager;
class UCommonUISettings;
typedef int32 TAsyncLoadPriority;

#if WITH_EDITOR
class UCommonUIEditorSettings;
#endif 

/**
 * Interface for the purchase flow module.
 */
class ICommonUIModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline ICommonUIModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ICommonUIModule>("CommonUI");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("CommonUI");
	}

	/**
	 * @return reference to Common UI settings
	 */
	static inline UCommonUISettings& GetSettings()
	{
		ICommonUIModule& Module = IsInGameThread() ? Get() : FModuleManager::GetModuleChecked<ICommonUIModule>("CommonUI");
		UCommonUISettings* Settings = Module.GetSettingsInstance();
		check(Settings);
		return *Settings;
	}

#if WITH_EDITOR
	/**
 * @return reference to Common UI Editor settings
 */
	static inline UCommonUIEditorSettings& GetEditorSettings()
	{
		ICommonUIModule& Module = IsInGameThread() ? Get() : FModuleManager::GetModuleChecked<ICommonUIModule>("CommonUI");
		UCommonUIEditorSettings* Settings = Module.GetEditorSettingsInstance();
		check(Settings);
		return *Settings;
	}
#endif

	// Streamable Management
	virtual FStreamableManager& GetStreamableManager() const = 0;

	// Lazy Load Priority
	virtual TAsyncLoadPriority GetLazyLoadPriority() const = 0;

protected:

	virtual UCommonUISettings* GetSettingsInstance() const = 0;

#if WITH_EDITOR
	virtual UCommonUIEditorSettings* GetEditorSettingsInstance() const = 0;
#endif
};
