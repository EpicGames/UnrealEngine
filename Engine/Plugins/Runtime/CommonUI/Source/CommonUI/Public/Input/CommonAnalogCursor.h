// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Framework/Application/AnalogCursor.h"
#include "Input/CommonAnalogCursorTypes.h"
#include "InputCoreTypes.h"
#include "Input/Events.h"
#include "Rendering/SlateRenderTransform.h"
#include "UObject/GCObject.h"

#define UE_API COMMONUI_API

class UCommonUIActionRouterBase;
class UCommonInputSubsystem;
class SWidget;
class UWidget;
class UUserWidget;
class UGameViewportClient;

struct FInputEvent;
struct FStreamableHandle;
enum class ECommonInputType : uint8;
enum EOrientation : int;

DECLARE_EVENT_OneParam(FCommonAnalogCursor, FVirtualPointerEnabledChanged, bool);

/**
 * Analog cursor preprocessor that tastefully hijacks things a bit to support controller navigation by moving a hidden cursor around based on focus.
 *
 * Introduces a separate focus-driven mode of operation, wherein the cursor is made invisible and automatically updated
 * to be centered over whatever widget is currently focused (except the game viewport - we completely hide it then)
 */
class FCommonAnalogCursor : public FAnalogCursor, public FGCObject
{
public:
	template <typename AnalogCursorT = FCommonAnalogCursor>
	static TSharedRef<AnalogCursorT> CreateAnalogCursor(const UCommonUIActionRouterBase& InActionRouter)
	{
		TSharedRef<AnalogCursorT> NewCursor = MakeShareable(new AnalogCursorT(InActionRouter));
		NewCursor->Initialize();
		return NewCursor;
	}

	UE_API virtual void Deinitialize();

	UE_API virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override;
	UE_API virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
	UE_API virtual bool HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;

	UE_API virtual bool CanReleaseMouseCapture() const;

	UE_API virtual bool HandleAnalogInputEvent(FSlateApplication& SlateApp, const FAnalogInputEvent& InAnalogInputEvent) override;
	UE_API virtual bool HandleMouseMoveEvent(FSlateApplication& SlateApp, const FPointerEvent& MouseEvent) override;
	UE_API virtual bool HandleMouseButtonDownEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent) override;
	UE_API virtual bool HandleMouseButtonUpEvent(FSlateApplication& SlateApp, const FPointerEvent& InPointerEvent) override;

	UE_API void SetCursorMovementStick(EAnalogStick InCursorMovementStick);

	UE_API virtual int32 GetOwnerUserIndex() const override;

	UE_API virtual void ShouldHandleRightAnalog(bool bInShouldHandleRightAnalog);

	virtual bool IsAnalogMovementEnabled() const { return bIsAnalogMovementEnabled; }

	UE_API virtual bool ShouldVirtualAcceptSimulateMouseButton(const FKeyEvent& InKeyEvent, EInputEvent InputEvent) const;

	UE_API void OnVirtualAcceptHoldCanceled();	
	
	UE_API TWeakPtr<SWidget> GetLastCursorTarget() const;

	UE_API bool IsUsingFakeTouch() const;
	UE_API bool IsVirtualPointerEnabled() const;
	UE_API bool IsUsingVirtualPointer() const;
	UE_API void SetVirtualPointerVisibility(bool bVisible);

	UE_API FVirtualPointerEnabledChanged& GetOnVirtualPointerEnabledChanged();

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(DefaultVirtualPointerWidget);
		Collector.AddReferencedObject(VirtualPointerWidget);
	}
	virtual FString GetReferencerName() const override { return TEXT("FCommonAnalogCursor"); }

protected:
	UE_API FCommonAnalogCursor(const UCommonUIActionRouterBase& InActionRouter);
	UE_API virtual void Initialize();

	UE_API virtual EOrientation DetermineScrollOrientation(const UWidget& Widget) const;

	UE_API virtual bool IsRelevantInput(const FKeyEvent& KeyEvent) const override;
	UE_API virtual bool IsRelevantInput(const FAnalogInputEvent& AnalogInputEvent) const override;
	
	UE_API void SetNormalizedCursorPosition(const FVector2D& RelativeNewPosition);
	UE_API bool IsInViewport(const FVector2D& Position) const;
	UE_API FVector2D ClampPositionToViewport(const FVector2D& InPosition) const;
	UE_API void HideCursor();

	UE_API UGameViewportClient* GetViewportClient() const;
	
	/**
	 * A ridiculous function name, but we have this exact question in a few places.
	 * We don't care about input while our owning player's game viewport isn't involved in the focus path,
	 * but we also want to hold off doing anything while that game viewport has full capture.
	 * So we need that "relevant, but not exclusive" sweet spot.
	 */
	UE_API bool IsGameViewportInFocusPathWithoutCapture() const;

	UE_API virtual void RefreshCursorSettings();
	UE_API virtual void RefreshCursorVisibility();
	
	UE_API virtual void HandleInputMethodChanged(ECommonInputType NewInputMethod);
	
	UE_API bool IsUsingGamepad() const;

	UE_API bool ShouldHideCursor() const;

	// Knowingly unorthodox member reference to a UObject - ok because we are a subobject of the owning router and will never outlive it
	const UCommonUIActionRouterBase& ActionRouter;
	ECommonInputType ActiveInputMethod;

	bool bEnableCommonUIVirtualPointer = false;
	bool bIsAnalogMovementEnabled = false;
	
	bool bShouldHandleRightAnalog = true;

	FVirtualPointerEnabledChanged OnVirtualPointerEnabledChanged;

private:

	UE_API void HandleOnEnableVirtualPointerChanged(IConsoleVariable* ConsoleVariable);
	UE_API void UpdateInputPreProcessor();
	UE_API TObjectPtr<UUserWidget> GetOrLoadDefaultVirtualPointer();
	UE_API void OnDefaultVirtualPointerClassLoaded();
	UE_API void UpdateVirtualPointerVisualState();

	/** The current set of pointer buttons being used as keys. */
	TSet<FKey> PointerButtonDownKeys;

	TWeakPtr<SWidget> LastCursorTarget;
	FSlateRenderTransform LastCursorTargetTransform;

	TOptional<FKeyEvent> ActiveKeyUpEvent;

	TObjectPtr<UUserWidget> DefaultVirtualPointerWidget = nullptr;
	TObjectPtr<UUserWidget> VirtualPointerWidget = nullptr;

	/** In-flight async load handle for the default virtual pointer class. Reset once load completes or is cancelled. */
	TSharedPtr<FStreamableHandle> DefaultVirtualPointerLoadHandle;
	/** Set when the configured class fails to resolve so we don't keep firing async loads every frame. */
	bool bDefaultVirtualPointerLoadFailed = false;

	float TimeUntilScrollUpdate = 0.f;

	/** UE-380247: was the last cursor input a gamepad stick? */
	bool bLastCursorInputWasGamepad = false;
	/** Swallow the synthetic mouse-move that follows our SetCursorPosition. */
	bool bIgnoreNextMouseMove = false;

	bool bShowVirtualPointer = false;
	bool bLastBroadcastVirtualPointerEnabled = false;

	/** Skips per-tick UUserWidget::TakeWidget() rebuilds in RefreshCursorVisibility when unchanged. */
	TWeakObjectPtr<UUserWidget> LastAppliedSoftwareCursorWidget;
	ECursorVisualState CurrentVirtualPointerVisualState = ECursorVisualState::Default;
	bool bVirtualAcceptDown = false;
	float HoldElapsedTime = 0.f;
	
#if !UE_BUILD_SHIPPING
	enum EShoulderButtonFlags
	{
		None = 0,
		LeftShoulder = 1 << 0,
		RightShoulder = 1 << 1,
		LeftTrigger = 1 << 2,
		RightTrigger = 1 << 3,
		All = LeftShoulder | RightShoulder | LeftTrigger | RightTrigger
	};
	int32 ShoulderButtonStatus = EShoulderButtonFlags::None;
#endif
};

#undef UE_API
