// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "CommonAnalogCursorTypes.generated.h"

/**
 * Cumulative cursor visual states - flags can combine (e.g. Hover + Pressed + Drag).
 * Default (no flags) represents the idle state.
 */
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ECursorVisualState : uint8
{
	Default = 0           UMETA(Hidden),
	Hover   = 1 << 0,
	Pressed = 1 << 1,
	Drag    = 1 << 2,
	Hold    = 1 << 3,
};
ENUM_CLASS_FLAGS(ECursorVisualState)

/** Implement this interface on a widget to receive cursor visual state changes from FCommonAnalogCursor. */
UINTERFACE(BlueprintType, MinimalAPI)
class UVirtualPointerVisualStateInterface : public UInterface
{
	GENERATED_BODY()
};

class IVirtualPointerVisualStateInterface
{
	GENERATED_BODY()
public:
	/** Fired when the cumulative state flags change. PreviousStates and ActiveStates carry combined flags. */
	UFUNCTION(BlueprintImplementableEvent, Category = "CommonUI|VirtualPointer")
	void OnVirtualPointerVisualStateChanged(ECursorVisualState PreviousStates, ECursorVisualState ActiveStates);

	/** Fired each tick while the accept button is held. Progress goes from 0.0 to 1.0 over MaxHoldDuration. */
	UFUNCTION(BlueprintImplementableEvent, Category = "CommonUI|VirtualPointer")
	void OnVirtualPointerHoldProgress(float Progress);
};
