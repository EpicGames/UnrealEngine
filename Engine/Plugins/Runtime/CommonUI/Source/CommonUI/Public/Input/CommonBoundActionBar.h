// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonButtonBase.h"
#include "Components/DynamicEntryBoxBase.h"
#include "CommonInputTypeEnum.h"
#include "CommonBoundActionBar.generated.h"

#define UE_API COMMONUI_API

class ICommonBoundActionButtonInterface;
class IConsoleVariable;
class IWidgetCompilerLog;

struct FUIActionBindingHandle;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FActionBarUpdated);

/**
 * A box populated with current actions available per CommonUI's Input Handler.
 */
UCLASS(MinimalAPI, Blueprintable, ClassGroup = UI, meta = (Category = "Common UI"))
class UCommonBoundActionBar : public UDynamicEntryBoxBase
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = CommonBoundActionBar)
	UE_API void SetDisplayOwningPlayerActionsOnly(bool bShouldOnlyDisplayOwningPlayerActions);

protected:
	UE_API virtual void OnWidgetRebuilt() override;
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	virtual void NativeOnActionButtonCreated(ICommonBoundActionButtonInterface* ActionButton, const FUIActionBindingHandle& RepresentedAction) { }

	virtual void ActionBarUpdateBeginImpl() {}
	virtual void ActionBarUpdateEndImpl() {}
	
	UE_API virtual UUserWidget* CreateActionButton(const FUIActionBindingHandle& BindingHandle);

	TSubclassOf<UCommonButtonBase> GetActionButtonClass() { return ActionButtonClass; }

#if WITH_EDITOR
	UE_API void ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const override;
#endif

private:
	UE_API void HandledInputTypeUpdated(ECommonInputType InputType);
	UE_API void HandleBoundActionsUpdated(bool bFromOwningPlayer);
	UFUNCTION()
	UE_API void HandleInputMappingsRebuiltUpdated();
	UE_API void UpdateDisplay();
	UE_API bool IsSafeToUpdateDisplay() const;
	UE_API void HandleDeferredDisplayUpdate();
	UE_API void HandlePlayerAdded(int32 PlayerIdx);
	UE_API void HandlePlayerRemoved(int32 PlayerIdx);
	
	UE_API void MonitorPlayerActions(const ULocalPlayer* NewPlayer);

	UE_API void ActionBarUpdateBegin();
	UE_API void ActionBarUpdateEnd();
	UE_API bool DoAnyActionButtonsHaveMouseCapture() const;

	UPROPERTY(EditAnywhere, Category = EntryLayout, meta=(MustImplement = "/Script/CommonUI.CommonBoundActionButtonInterface"))
	TSubclassOf<UCommonButtonBase> ActionButtonClass;

	UPROPERTY(EditAnywhere, Category = Display)
	bool bDisplayOwningPlayerActionsOnly = true;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Display)
	bool bIgnoreDuplicateActions = true;

	UPROPERTY(BlueprintAssignable, Category = "Events", meta = (AllowPrivateAccess = true))
	FActionBarUpdated OnActionBarUpdated;

	bool bIsRefreshQueued = false;
};

#undef UE_API
