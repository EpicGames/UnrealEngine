// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonBoundActionButtonInterface.h"
#include "CommonButtonBase.h"
#include "UIActionBindingHandle.h"
#include "CommonBoundActionButton.generated.h"

#define UE_API COMMONUI_API

class UCommonTextBlock;

UCLASS(MinimalAPI, Abstract, meta = (DisableNativeTick))
class UCommonBoundActionButton : public UCommonButtonBase, public ICommonBoundActionButtonInterface
{
	GENERATED_BODY()

public:
	//~ Begin ICommonBoundActionButtonInterface
	UE_API virtual void SetRepresentedAction(FUIActionBindingHandle InBindingHandle) override;
	//~ End ICommonBoundActionButtonInterface
	
protected:
	UE_API virtual void NativeOnClicked() override;
	UE_API virtual void NativeOnCurrentTextStyleChanged() override;

	UE_API virtual void UpdateInputActionWidget() override;
	UE_API virtual void UpdateHoldData(ECommonInputType CurrentInputType) override;

	UFUNCTION(BlueprintImplementableEvent, Category = "Common Bound Action")
	UE_API void OnUpdateInputAction();

protected:
	UPROPERTY(BlueprintReadOnly, meta = (BindWidget), Category = "Text Block")
	TObjectPtr<UCommonTextBlock> Text_ActionName;
	
	/** Set to true if clicking this button should require holding it when the bound action is a hold action */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input|Hold")
	bool bLinkRequiresHoldToBindingHold = false;

private:
	FUIActionBindingHandle BindingHandle;
};

#undef UE_API
