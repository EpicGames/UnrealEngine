// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "UIActionBindingHandle.h"

#include "CommonBoundActionButtonInterface.generated.h"

UINTERFACE(MinimalAPI)
class UCommonBoundActionButtonInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for runtime bindable action buttons. Allows button widget to be used by CommonBoundActionBar.
 */
class ICommonBoundActionButtonInterface
{
	GENERATED_BODY()

public:
	virtual void SetRepresentedAction(FUIActionBindingHandle InBindingHandle) = 0;
};
