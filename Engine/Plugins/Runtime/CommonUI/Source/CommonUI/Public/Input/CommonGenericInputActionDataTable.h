// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"

#include "CommonGenericInputActionDataTable.generated.h"

#define UE_API COMMONUI_API

class UCommonGenericInputActionDataTable;

/**
 * Overrides postload to allow for derived classes to perform code-level changes to the datatable.
 * Ex: Per-platform edits. Allows modification of datatable data without checking out the data table asset.
 */
UCLASS(MinimalAPI, BlueprintType)
class UCommonGenericInputActionDataTable : public UDataTable
{
	GENERATED_BODY()

public:
	UE_API UCommonGenericInputActionDataTable();
	virtual ~UCommonGenericInputActionDataTable() = default;

	//~ Begin UObject Interface.
	UE_API virtual void PostLoad() override;
	//~ End UObject Interface
};

/**
 * Derive from to process common input action datatable
 */
UCLASS(MinimalAPI, Transient)
class UCommonInputActionDataProcessor : public UObject
{
	GENERATED_BODY()

public:
	UCommonInputActionDataProcessor() = default;
	virtual ~UCommonInputActionDataProcessor() = default;

	UE_API virtual void ProcessInputActions(UCommonGenericInputActionDataTable* InputActionDataTable);
};

#undef UE_API
