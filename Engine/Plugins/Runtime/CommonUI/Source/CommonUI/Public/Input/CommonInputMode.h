// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.1, "Use CommonInputModeTypes.h from CommonInput module instead of Input/CommonInputMode.h from CommonUI module.")

#include "CommonInputModeTypes.h"