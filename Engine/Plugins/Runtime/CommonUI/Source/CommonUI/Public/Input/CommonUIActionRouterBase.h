// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonInputModeTypes.h"
#include "Subsystems/LocalPlayerSubsystem.h"
#include "Containers/Ticker.h"
#include "Containers/CircularBuffer.h"

#include "Engine/EngineBaseTypes.h"
#include "Input/UIActionBindingHandle.h"
#include "InputCoreTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Framework/Application/SlateApplication.h"
#include "GameplayTagContainer.h"
#include "CommonUIActionRouterBase.generated.h"

#define UE_API COMMONUI_API

class SWidget;
struct FBindUIActionArgs;

class AHUD;
class FWeakWidgetPath;
class FWidgetPath;
class UCanvas;
class UWidget;
class UCommonUserWidget;
class UCommonActivatableWidget;
class UCommonInputSubsystem;
class UInputComponent;
class UPlayerInput;
class FCommonAnalogCursor;
class IInputProcessor;
class UCommonInputActionDomainTable;

enum class EProcessHoldActionResult;
class FActivatableTreeNode;
class UCommonInputSubsystem;
class UCommonInputActionDomain;
struct FUIActionBinding;
class FDebugDisplayInfo;
struct FAutoCompleteCommand;
using FActivatableTreeNodePtr = TSharedPtr<FActivatableTreeNode>;
using FActivatableTreeNodeRef = TSharedRef<FActivatableTreeNode>;

class FActivatableTreeRoot;
using FActivatableTreeRootPtr = TSharedPtr<FActivatableTreeRoot>;
using FActivatableTreeRootRef = TSharedRef<FActivatableTreeRoot>;
struct FFocusEvent;

enum class ERouteUIInputResult : uint8
{
	Handled,
	BlockGameInput,
	Unhandled
};

/**
 * The nucleus of the CommonUI input routing system. 
 * 
 * Gathers input from external sources such as game viewport client and forwards them to widgets 
 * via activatable tree node representation.
 */
UCLASS(MinimalAPI)
class UCommonUIActionRouterBase : public ULocalPlayerSubsystem
{
	GENERATED_BODY()

public:
	static UE_API UCommonUIActionRouterBase* Get(const UWidget& ContextWidget);

	/** searches up the SWidget tree until it finds the nearest UCommonActivatableWidget (excluding checking Widget itself) */
	static UE_API UCommonActivatableWidget* FindOwningActivatable(TSharedPtr<SWidget> Widget, ULocalPlayer* OwningLocalPlayer);
	/** searches up the SWidget tree until it finds the nearest UCommonActivatableWidget (including checking Widget itself) */
	static UE_API UCommonActivatableWidget* FindActivatable(TSharedPtr<SWidget> Widget, ULocalPlayer* OwningLocalPlayer);

	UE_API UCommonUIActionRouterBase();
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;

	/** Sets whether the underlying activatable tree system is enabled - when disabled, all we really do is process Persistent input actions */
	UE_API virtual void SetIsActivatableTreeEnabled(bool bInIsTreeEnabled);

	UE_API virtual FUIActionBindingHandle RegisterUIActionBinding(const UWidget& Widget, const FBindUIActionArgs& BindActionArgs);
	UE_DEPRECATED(5.5, "The version of RegisterLinkedPreprocessor taking an int32 DesiredIndex parameter is deprecated and uses EInputPreProcessorType::Game as a default. For greater control in pre-processor priority, please use the new version with a FInputPreprocessorRegistrationInfo parameter")
	UE_API bool RegisterLinkedPreprocessor(const UWidget& Widget, const TSharedRef<IInputProcessor>& InputPreprocessor, int32 DesiredIndex);
	UE_API bool RegisterLinkedPreprocessor(const UWidget& Widget, const TSharedRef<IInputProcessor>& InputPreprocessor);
	UE_API bool RegisterLinkedPreprocessor(const UWidget& Widget, const TSharedRef<IInputProcessor>& InputPreprocessor, const FInputPreprocessorRegistrationKey& RegistrationInfo);

	DECLARE_EVENT_OneParam(UCommonUIActionRouterBase, FOnActiveInputModeChanged, ECommonInputMode);
	FOnActiveInputModeChanged& OnActiveInputModeChanged() const { return OnActiveInputModeChangedEvent; }
	UE_API ECommonInputMode GetActiveInputMode(ECommonInputMode DefaultInputMode = ECommonInputMode::All) const;
	UE_API EMouseCaptureMode GetActiveMouseCaptureMode(EMouseCaptureMode DefaultMouseCapture = EMouseCaptureMode::NoCapture) const;
	
	DECLARE_EVENT_OneParam(UCommonUIActionRouterBase, FOnActiveInputConfigChanged, const FUIInputConfig);
	FOnActiveInputConfigChanged& OnActiveInputConfigChanged() const { return OnActiveInputConfigChangedEvent; }

	DECLARE_EVENT_OneParam(UCommonUIActionRouterBase, FOnActivationMetadataChanged, FActivationMetadata);
	FOnActivationMetadataChanged& OnActivationMetadataChanged() const { return OnActivationMetadataChangedEvent; }

	UE_API void RegisterScrollRecipient(const UWidget& ScrollableWidget, UWidget* OwningNodeWidget = nullptr);
	UE_API void UnregisterScrollRecipient(const UWidget& ScrollableWidget, UWidget* OwningNodeWidget = nullptr);
	UE_API TArray<const UWidget*> GatherActiveAnalogScrollRecipients() const;

	UE_API TArray<FUIActionBindingHandle> GatherActiveBindings() const;
	FSimpleMulticastDelegate& OnBoundActionsUpdated() const { return OnBoundActionsUpdatedEvent; }

	UE_API UCommonInputSubsystem& GetInputSubsystem() const;

	UE_API virtual ERouteUIInputResult ProcessInput(FKey Key, EInputEvent InputEvent) const;
	UE_API virtual bool CanProcessNormalGameInput() const;

	UE_API bool IsPendingTreeChange() const;

	TSharedPtr<FCommonAnalogCursor> GetCommonAnalogCursor() const { return AnalogCursor; }

	UE_API void FlushInput();

	UE_API bool IsWidgetInActiveRoot(const UCommonActivatableWidget* Widget) const;
	UE_API bool IsWidgetInLeafmostNodeHierarchy(const UWidget& Widget) const;

	UE_API UCommonActivatableWidget* GetLeafmostActivatableWidget() const;

	/** 
	 * Sets Input Config 
	 * 
	 * @param NewConfig config to set
	 * @param InConfigSource optional source of config. If exists, will be used to log input config source
	 */
	UE_API void SetActiveUIInputConfig(const FUIInputConfig& NewConfig, const UObject* InConfigSource = nullptr);

public:
	UE_API void NotifyUserWidgetConstructed(const UCommonUserWidget& Widget);
	UE_API void NotifyUserWidgetDestructed(const UCommonUserWidget& Widget);
	
	UE_API virtual void AddBinding(FUIActionBindingHandle Binding);
	UE_API virtual void RemoveBinding(FUIActionBindingHandle Binding);

	UE_API int32 GetLocalPlayerIndex() const;

	UE_API void RefreshActiveRootFocusRestorationTarget() const;
	UE_API void RefreshActiveRootFocus();
	UE_API void RefreshUIInputConfig();

	UE_API bool ShouldAlwaysShowCursor() const;

protected:
	UE_API virtual TSharedRef<FCommonAnalogCursor> MakeAnalogCursor() const;
	UE_API virtual void PostAnalogCursorCreate();
	UE_API void RegisterAnalogCursorTick();

	UE_API TWeakPtr<FActivatableTreeRoot> GetActiveRoot() const;
	UE_API virtual void SetActiveRoot(FActivatableTreeRootPtr NewActiveRoot);
	UE_API void SetForceResetActiveRoot(bool bInForceResetActiveRoot);

	UE_API virtual void ApplyUIInputConfig(const FUIInputConfig& NewConfig, bool bForceRefresh);
	UE_API void UpdateLeafNodeAndConfig(FActivatableTreeRootPtr DesiredRoot, FActivatableTreeNodePtr DesiredLeafNode);
	UE_API void FlushPressedKeys() const;

	UE_API void RefreshActionDomainLeafNodeConfig();

	bool bIsActivatableTreeEnabled = true;

	/** The currently applied UI input configuration */
	TOptional<FUIInputConfig> ActiveInputConfig;

	TSharedPtr<FCommonAnalogCursor> AnalogCursor;
	FTSTicker::FDelegateHandle TickHandle;

private:
	UE_API bool Tick(float DeltaTime);

	UE_API void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);
	UE_API void PopulateAutoCompleteEntries(TArray<FAutoCompleteCommand>& AutoCompleteList);

	UE_API void RegisterWidgetBindings(const FActivatableTreeNodePtr& TreeNode, const TArray<FUIActionBindingHandle>& BindingHandles);

	UE_API FActivatableTreeNodePtr FindNode(const UCommonActivatableWidget* Widget) const;
	UE_API FActivatableTreeNodePtr FindOwningNode(const UWidget& Widget) const;
	UE_API FActivatableTreeNodePtr FindNodeRecursive(const FActivatableTreeNodePtr& CurrentNode, const UCommonActivatableWidget& Widget) const;
	UE_API FActivatableTreeNodePtr FindNodeRecursive(const FActivatableTreeNodePtr& CurrentNode, const TSharedPtr<SWidget>& Widget) const;
	UE_API void SetActiveActivationMetadata(const FActivationMetadata& NewConfig);

	// Returns the top-most active root that is receiving input
	// Action Domain's analogue to ActiveRootNode
	UE_API FActivatableTreeRootPtr FindActiveActionDomainRootNode() const;
	
	UE_API void HandleActivatableWidgetRebuilding(UCommonActivatableWidget& RebuildingWidget);
	UE_API void ProcessRebuiltWidgets();
	UE_API void RefreshRootNodes();
	UE_API void RefreshBoundActions();
	UE_API void AssembleTreeRecursive(const FActivatableTreeNodeRef& CurNode, TMap<UCommonActivatableWidget*, TArray<UCommonActivatableWidget*>>& WidgetsByDirectParent);

	UE_API void HandleRootWidgetSlateReleased(TWeakPtr<FActivatableTreeRoot> WeakRoot);
	UE_API void HandleRootNodeActivated(TWeakPtr<FActivatableTreeRoot> WeakActivatedRoot);
	UE_API void HandleRootNodeDeactivated(TWeakPtr<FActivatableTreeRoot> WeakDeactivatedRoot);
	UE_API void HandleLeafmostActiveNodeChanged();

	UE_API void HandleSlateFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget);

	UE_API void HandlePostGarbageCollect();
	UE_API void HandleVirtualPointerEnabled(bool bEnabled);

	UE_API const UCommonInputActionDomainTable* GetActionDomainTable() const;
	UE_API bool ProcessInputOnActionDomains(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const;
	UE_API EProcessHoldActionResult ProcessHoldInputOnActionDomains(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent, int32 UserIndex) const;
	UE_API FGameplayTagContainer GetGameplayTagsForInputMode(const ECommonInputMode Mode) const;
	
	void DebugDumpActionDomainRootNodes(int32 UserIndex, int32 ControllerId, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const;

	struct FPendingWidgetRegistration
	{
		TWeakObjectPtr<const UWidget> Widget;
		TArray<FUIActionBindingHandle> ActionBindings;
		bool bIsScrollRecipient = false;
		
		struct UE_DEPRECATED(5.5, "This struct is deprecated, please use FInputPreprocessorRegistration instead.") FPreprocessorRegistration
		{
			TSharedPtr<IInputProcessor> Preprocessor;
			int32 DesiredIdx = 0;
			bool operator==(const TSharedRef<IInputProcessor>& OtherPreprocessor) const { return Preprocessor == OtherPreprocessor; }
		};
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.5, "This variable is using a deprecated type, please use InputPreProcessors instead.")
		TArray<FPreprocessorRegistration> Preprocessors;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TArray<FInputPreprocessorRegistration> InputPreProcessors;

		bool operator==(const UWidget* OtherWidget) const { return OtherWidget == Widget.Get(); }
		bool operator==(const UWidget& OtherWidget) const { return &OtherWidget == Widget.Get(); }

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPendingWidgetRegistration() = default;
		FPendingWidgetRegistration(const FPendingWidgetRegistration&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
	UE_API FPendingWidgetRegistration& GetOrCreatePendingRegistration(const UWidget& Widget);
	TArray<FPendingWidgetRegistration> PendingWidgetRegistrations;
	
	TArray<TWeakObjectPtr<UCommonActivatableWidget>> RebuiltWidgetsPendingNodeAssignment;
	TSet<TWeakPtr<FActivatableTreeRoot>> ActiveActionDomainRootsPendingPaint;
	
	TArray<FActivatableTreeRootRef> RootNodes;
	FActivatableTreeRootPtr ActiveRootNode;

	// Note: Treat this as a TSharedRef - only reason it isn't is because TSharedRef doesn't play nice with forward declarations :(
	TSharedPtr<class FPersistentActionCollection> PersistentActions;

	TCircularBuffer<FString> InputConfigSources = TCircularBuffer<FString>(5, "None");
	int32 InputConfigSourceIndex = 0;

	bool bForceResetActiveRoot = false;

	mutable FSimpleMulticastDelegate OnBoundActionsUpdatedEvent;
	mutable FOnActiveInputModeChanged OnActiveInputModeChangedEvent;
	mutable FOnActivationMetadataChanged OnActivationMetadataChangedEvent;
	mutable FOnActiveInputConfigChanged OnActiveInputConfigChangedEvent;

	friend class FActionRouterBindingCollection;
	friend class FActivatableTreeNode;
	friend class FActivatableTreeRoot;
	friend class FActionRouterDebugUtils;

	mutable TArray<FKey> HeldKeys;

	/** A wrapper around TArray that keeps RootList sorted by PaintLayer */
	struct FActionDomainSortedRootList
	{
		TArray<FActivatableTreeRootRef>& GetRootList()
		{
			return RootList;
		}
		const TArray<FActivatableTreeRootRef>& GetRootList() const
		{
			return RootList;
		}
		
		// Inserts RootNode into RootList based on its paint layer
		void Add(FActivatableTreeRootRef RootNode);

		// Trivial removal
		int32 Remove(FActivatableTreeRootRef RootNode);

		// Trivial Contains check
		bool Contains(FActivatableTreeRootRef RootNode) const;
		
		void Sort();
		
		// Needed for debug purposes
		void DebugDumpRootList(FString& OutputStr, bool bIncludeActions, bool bIncludeChildren, bool bIncludeInactive) const;

	private:
		TArray<FActivatableTreeRootRef> RootList;
	};

	TMap<TObjectPtr<UCommonInputActionDomain>, FActionDomainSortedRootList> ActionDomainRootNodes;
};

#undef UE_API
