// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/SlateApplication.h"
#include "InputCoreTypes.h"
#include "UITag.h"
#include "CommonUIInputSettings.generated.h"

#define UE_API COMMONUI_API

class UPlayerInput;
class UUserWidget;

USTRUCT()
struct FUIActionKeyMapping
{
	GENERATED_BODY()

public:
	FUIActionKeyMapping() {}
	FUIActionKeyMapping(FKey InKey, float InHoldTime, float InHoldRollbackTime)
		: Key(InKey)
		, HoldTime(InHoldTime)
		, HoldRollbackTime(InHoldRollbackTime)
	{}

	/** A key that triggers this action */
	UPROPERTY(EditAnywhere, Config, Category = "UI Action Key Mapping")
	FKey Key;

	/** How long must the key be held down for the action to be executed? */
	UPROPERTY(EditAnywhere, Config, Category = "UI Action Key Mapping")
	float HoldTime = 0.f;
	
	/** Time (in seconds) for hold progress to go from 1.0 (completed) to 0.0. */
	UPROPERTY(EditAnywhere, Config, Category = "UI Action Key Mapping")
	float HoldRollbackTime = 0.f;
};

USTRUCT()
struct FUIInputAction
{
	GENERATED_BODY()
		
public:
	/** The UI.Action tag that acts as the universal identifier of this action. */
	UPROPERTY(EditAnywhere, Config, Category = "UI Input Action")
	FUIActionTag ActionTag;

	/** 
	 * Whenever a UI input action is bound, an override display name can optionally be provided.
	 * This is the default generic display name of this action for use in the absence of such an override.
	 */
	UPROPERTY(EditAnywhere, Config, Category = "UI Input Action")
	FText DefaultDisplayName;

	/** All key mappings that will trigger this action */
	UPROPERTY(EditAnywhere, Config, Category = "UI Input Action", meta = (TitleProperty = "Key"))
	TArray<FUIActionKeyMapping> KeyMappings;

	UE_API bool HasAnyHoldMappings() const;
};

USTRUCT()
struct FCommonAnalogCursorSettings
{
	GENERATED_BODY();

public:
	/** The registration priority of the analog cursor preprocessor. */
	UE_DEPRECATED(5.5, "PreprocessorPriority is deprecated, please use PreprocessorRegistrationInfo from now on.")
	UPROPERTY(EditAnywhere, Config, Category = General)
	int32 PreprocessorPriority = 2;

	UPROPERTY(EditAnywhere, Config, Category = General)
	FInputPreprocessorRegistrationKey PreprocessorRegistrationInfo = FInputPreprocessorRegistrationKey{ EInputPreProcessorType::Game, 2 };

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor)
	bool bEnableCursorAcceleration = true;

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor, meta = (EditCondition = bEnableCursorAcceleration))
	float CursorAcceleration = 1500.f;

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor)
	float CursorMaxSpeed = 2200.f;

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor, meta = (ClampMin = "0.0", ClampMax = "0.9"))
	float CursorDeadZone = 0.25f;

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float HoverSlowdownFactor = 0.4f;

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor, meta = (ClampMin = "0.0", ClampMax = "0.9"))
	float ScrollDeadZone = 0.2f;

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor)
	float ScrollUpdatePeriod = 0.1f;

	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor)
	float ScrollMultiplier = 2.5f;

	/** Duration in seconds for hold progress to go from 0.0 to 1.0 while the accept button is held. */
	UPROPERTY(EditDefaultsOnly, Category = AnalogCursor)
	float MaxHoldDuration = 1.0f;

	FCommonAnalogCursorSettings() {}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCommonAnalogCursorSettings& operator=(const FCommonAnalogCursorSettings&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/** Project-wide input settings for UI input actions */
UCLASS(config=Input, defaultconfig, MinimalAPI)
class UCommonUIInputSettings : public UObject
{
	GENERATED_BODY()

public:
	COMMONUI_API static const UCommonUIInputSettings& Get();
	
	virtual void PostInitProperties() override;

	int32 GetUIActionProcessingPriority() const { return UIActionProcessingPriority; }
	COMMONUI_API const FUIInputAction* FindAction(FUIActionTag ActionTag) const;
	const TArray<FUIInputAction>& GetUIInputActions() const { return InputActions; }
	const FCommonAnalogCursorSettings& GetAnalogCursorSettings() const { return AnalogCursorSettings; }
	const TSoftClassPtr<UUserWidget>& GetDefaultVirtualPointerClass() const { return DefaultVirtualPointerClass; }
	bool ShouldLinkCursorToGamepadFocus() const { return bLinkCursorToGamepadFocus; }
//
private:
	/** True to have the mouse pointer automatically moved to the center of whatever widget is currently focused while using a gamepad. */
	UPROPERTY(EditAnywhere, Config, Category = General)
	bool bLinkCursorToGamepadFocus = true;

	/** 
	 * The input priority of the input components that process UI input actions.
	 * The lower the value, the higher the priority of the component.
	 *
	 * By default, this value is incredibly high to ensure UI action processing priority over game elements.
	 * Adjust as needed for the UI input components to be processed at the appropriate point in the input stack in your project.
	 *
	 * NOTE: When the active input mode is ECommonInputMode::Menu, ALL input components with lower priority than this will be fully blocked.
	 *		 Thus, if any game agent input components are registered with higher priority than this, behavior will not match expectation.
	 */
	UPROPERTY(EditAnywhere, Config, Category = General)
	int32 UIActionProcessingPriority = 10000;

	/** All UI input action mappings for the project */
	UPROPERTY(EditAnywhere, Config, Category = Actions, meta = (TitleProperty = "ActionTag"))
	TArray<FUIInputAction> InputActions;

	/** Config-only set of input action overrides - if an entry for a given action is both here and in the InputActions array, this entry wins completely. */
	UPROPERTY(Config)
	TArray<FUIInputAction> ActionOverrides;

	UPROPERTY(EditAnywhere, Config, Category = AnalogCursor)
	FCommonAnalogCursorSettings AnalogCursorSettings;

	/** Widget class used as the software cursor when the virtual pointer is active. Stored here so the cooker tracks it as a dependency. Default is set in the plugin's Config/Input.ini. */
	UPROPERTY(EditAnywhere, Config, Category = AnalogCursor)
	TSoftClassPtr<UUserWidget> DefaultVirtualPointerClass;
};

#undef UE_API
