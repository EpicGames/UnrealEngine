// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "UITag.h"
#include "CommonInputModeTypes.h"
#include "CommonInputTypeEnum.h"
#include "Engine/EngineBaseTypes.h"
#include "InputAction.h"

#define UE_API COMMONUI_API

struct FBindUIActionArgs
{
	FBindUIActionArgs(FUIActionTag InActionTag, const FSimpleDelegate& InOnExecuteAction)
		: ActionTag(InActionTag)
		, OnExecuteAction(InOnExecuteAction)
	{}

	FBindUIActionArgs(FUIActionTag InActionTag, bool bShouldDisplayInActionBar, const FSimpleDelegate& InOnExecuteAction)
		: ActionTag(InActionTag)
		, bDisplayInActionBar(bShouldDisplayInActionBar)
		, OnExecuteAction(InOnExecuteAction)
	{}

	// @TODO: Rename non-legacy in 5.3. We no longer have any active plans to remove data tables in CommonUI.
	FBindUIActionArgs(const FDataTableRowHandle& InLegacyActionTableRow, const FSimpleDelegate& InOnExecuteAction)
		: LegacyActionTableRow(InLegacyActionTableRow)
		, OnExecuteAction(InOnExecuteAction)
	{}

	// @TODO: Rename non-legacy in 5.3. We no longer have any active plans to remove data tables in CommonUI.
	FBindUIActionArgs(const FDataTableRowHandle& InLegacyActionTableRow, bool bShouldDisplayInActionBar, const FSimpleDelegate& InOnExecuteAction)
		: LegacyActionTableRow(InLegacyActionTableRow)
		, bDisplayInActionBar(bShouldDisplayInActionBar)
		, OnExecuteAction(InOnExecuteAction)
	{}

	FBindUIActionArgs(const UInputAction* InInputAction, const FSimpleDelegate & InOnExecuteAction)
		: InputAction(InInputAction)
		, OnExecuteAction(InOnExecuteAction)
	{}

	FBindUIActionArgs(const UInputAction* InInputAction, bool bShouldDisplayInActionBar, const FSimpleDelegate & InOnExecuteAction)
		: InputAction(InInputAction)
		, bDisplayInActionBar(bShouldDisplayInActionBar)
		, OnExecuteAction(InOnExecuteAction)
	{}

	UE_API FName GetActionName() const;

	UE_API bool ActionHasHoldMappings() const;

	FUIActionTag ActionTag;

	// @TODO: Rename non-legacy in 5.3. We no longer have any active plans to remove data tables in CommonUI.
	FDataTableRowHandle LegacyActionTableRow;

	TWeakObjectPtr<const UInputAction> InputAction;

	ECommonInputMode InputMode = ECommonInputMode::Menu;
	EInputEvent KeyEvent = IE_Pressed;

	/**
	 * By default, the action bar only displays prompts for actions with keys valid for the current input type, any input types added here will
	 * skip that check and display this action regardless of the bound keys
	 */
	TSet<ECommonInputType> InputTypesExemptFromValidKeyCheck = { ECommonInputType::MouseAndKeyboard, ECommonInputType::Touch };

	/**
	 * A persistent binding is always registered and will be executed regardless of the activation status of the binding widget's parentage.
	 * Persistent bindings also never stomp one another - if two are bound to the same action, both will execute. Use should be kept to a minimum.
	 */
	bool bIsPersistent = false;

	/**
	 * True to have this binding consume the triggering key input.
	 * Persistent bindings that consume will prevent the key reaching non-persistent bindings and game agents.
	 * Non-persistent bindings that consume will prevent the key reaching game agents.
	 */
	bool bConsumeInput = true;

	/** Whether this binding can/should be displayed in a CommonActionBar (if one exists) */
	bool bDisplayInActionBar = true;

	/** True implies we will add default hold times if the current action is not a hold action */
    bool bForceHold = false;

	/** Optional display name to associate with this binding instead of the default */
	FText OverrideDisplayName;

	/**
	 * Normally, actions on a widget are triggered in the order they're registered. We can assign a priority to ensure a certain order of execution.
	 * 0 is the order of registration.
	 */
	int32 PriorityWithinCollection = 0;

	FSimpleDelegate OnExecuteAction;

	/** If the bound action has any hold mappings, this will fire each frame while held. Has no bearing on actual execution and wholly irrelevant for non-hold actions */
	DECLARE_DELEGATE_OneParam(FOnHoldActionProgressed, float);
	FOnHoldActionProgressed OnHoldActionProgressed;
	
	/** If the bound action has any hold mappings, this will fire when the hold begins. Has no bearing on actual execution and wholly irrelevant for non-hold actions */
	DECLARE_DELEGATE(FOnHoldActionPressed);
	FOnHoldActionPressed OnHoldActionPressed;

	/** If the bound action has any hold mappings, this will fire when the hold is interrupted. Has no bearing on actual execution and wholly irrelevant for non-hold actions */
	DECLARE_DELEGATE(FOnHoldActionReleased);
	FOnHoldActionReleased OnHoldActionReleased;
};

#undef UE_API
