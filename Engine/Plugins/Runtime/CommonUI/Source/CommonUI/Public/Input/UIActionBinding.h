// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Engine/DataTable.h"
#include "Input/CommonUIInputSettings.h" // IWYU pragma: keep
#include "Input/UIActionBindingHandle.h"

#define UE_API COMMONUI_API

enum class ECommonInputType : uint8;
enum EInputEvent : int;
struct FKey;
struct FUIActionKeyMapping;

class FActionRouterBindingCollection;
struct FBindUIActionArgs;
struct FCommonInputActionDataBase;
class UInputAction;

enum class EProcessHoldActionResult
{
	Handled,
	GeneratePress,
	Unhandled
};

struct FUIActionBinding
{
	FUIActionBinding() = delete;
	FUIActionBinding(const FUIActionBinding&) = delete;
	FUIActionBinding(FUIActionBinding&&) = delete;

	UE_DEPRECATED(5.6, "Calling FUIActionBinding::TryCreate without a user index is deprecated. Call the version with the UserIndex param.")
	static UE_API FUIActionBindingHandle TryCreate(const UWidget& InBoundWidget, const FBindUIActionArgs& BindArgs);
	static UE_API FUIActionBindingHandle TryCreate(const UWidget& InBoundWidget, const FBindUIActionArgs& BindArgs, int32 UserIndex);
	
	static UE_API TSharedPtr<FUIActionBinding> FindBinding(FUIActionBindingHandle Handle);
	static UE_API void CleanRegistrations();

	bool operator==(const FUIActionBindingHandle& OtherHandle) const { return Handle == OtherHandle; }

	// @TODO: Rename non-legacy in 5.3. We no longer have any active plans to remove data tables in CommonUI.
	UE_API FCommonInputActionDataBase* GetLegacyInputActionData() const;

	UE_API EProcessHoldActionResult ProcessHoldInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent);
	UE_API bool ProcessNormalInput(ECommonInputMode ActiveInputMode, FKey Key, EInputEvent InputEvent);
	UE_API FString ToDebugString() const;

	UE_API void BeginHold();
	UE_API bool UpdateHold(float TargetHoldTime);
	UE_API void CancelHold();
	UE_API void BeginRollback(float TargetHoldRollbackTime, float HoldTime, FUIActionBindingHandle BindingHandle);
	UE_API double GetSecondsHeld() const;
	UE_API bool IsHoldActive() const;
	UE_API void ResetHold();

	FName ActionName;
	EInputEvent InputEvent;
	bool bConsumesInput = true;
	bool bIsPersistent = false;
	
	int32 PriorityWithinCollection = 0;

	TWeakObjectPtr<const UWidget> BoundWidget;
	ECommonInputMode InputMode;

	int32 UserIndex = INDEX_NONE;

	bool bDisplayInActionBar = false;
	TSet<ECommonInputType> InputTypesExemptFromValidKeyCheck;
	FText ActionDisplayName;
	
	TWeakPtr<FActionRouterBindingCollection> OwningCollection;
	FSimpleDelegate OnExecuteAction;
	FUIActionBindingHandle Handle;

	TArray<FUIActionKeyMapping> NormalMappings;
	TArray<FUIActionKeyMapping> HoldMappings;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnHoldActionProgressedMulticast, float);
	FOnHoldActionProgressedMulticast OnHoldActionProgressed;

	DECLARE_MULTICAST_DELEGATE(FOnHoldActionPressed);
	FOnHoldActionPressed OnHoldActionPressed;

	DECLARE_MULTICAST_DELEGATE(FOnHoldActionReleased);
	FOnHoldActionPressed OnHoldActionReleased;

	// @TODO: Rename non-legacy in 5.3. We no longer have any active plans to remove data tables in CommonUI.
	FDataTableRowHandle LegacyActionTableRow;

	TWeakObjectPtr<const UInputAction> InputAction;

private:
	UE_API FUIActionBinding(const UWidget& InBoundWidget, const FBindUIActionArgs& BindArgs);
	
	// At what time in seconds did the hold start?
	double HoldStartTime = -1.0;
    	
	// At what second will the hold start?
	double HoldStartSecond = 0.0;
	
	// At what second is the hold progress at?
	double CurrentHoldSecond = 0.0;
	
	// Multiplier for the time (in seconds) for hold progress to go from 1.0 (completed) to 0.0.
    double HoldRollbackMultiplier = 1.0;
	
	// Target time (in seconds) for the hold progress to go from 0.0 to 1.0 (completed).
	double HoldTime = 0.0;
    	
	// Handle for ticker spawned for button hold rollback
	FTSTicker::FDelegateHandle HoldProgressRollbackTickerHandle;

	static UE_API int32 IdCounter;
	static UE_API TMap<FUIActionBindingHandle, TSharedPtr<FUIActionBinding>> AllRegistrationsByHandle;
	
	// All keys currently being tracked for a hold action
	static UE_API TMap<FKey, FUIActionBindingHandle> CurrentHoldActionKeys;

	friend struct FUIActionBindingHandle;
};

#undef UE_API
