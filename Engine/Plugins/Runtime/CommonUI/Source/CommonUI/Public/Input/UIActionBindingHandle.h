// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Optional.h"
#include "UIActionBindingHandle.generated.h"

#define UE_API COMMONUI_API

enum class EMouseCaptureMode : uint8;
enum class EMouseLockMode : uint8;
struct FScriptContainerElement;
class ULocalPlayer;
class UWidget;
enum class ECommonInputMode : uint8;

USTRUCT(BlueprintType, DisplayName = "UI Action Binding Handle")
struct FUIActionBindingHandle
{
	GENERATED_BODY()

public:
	UE_API bool IsValid() const;
	UE_API void Unregister();

	/** Calls ResetHold() on the Action Binding, which in turn resets the hold progress to 0.0  */
	UE_API void ResetHold();

	UE_API FName GetActionName() const;

	UE_API FText GetDisplayName() const;

	/** Should not be called often as broadcasts UCommonUIActionRouterBase::OnBoundActionsUpdated event */
	UE_API void SetDisplayName(const FText& DisplayName);

	UE_API bool GetDisplayInActionBar() const;

	/** Should not be called often as broadcasts UCommonUIActionRouterBase::OnBoundActionsUpdated event */
	UE_API void SetDisplayInActionBar(const bool bDisplayInActionBar);

	UE_API const UWidget* GetBoundWidget() const;
	
	UE_API ULocalPlayer* GetBoundLocalPlayer() const;

	FUIActionBindingHandle() {}
	bool operator==(const FUIActionBindingHandle& Other) const { return RegistrationId == Other.RegistrationId; }
	bool operator!=(const FUIActionBindingHandle& Other) const { return !operator==(Other); }

	friend uint32 GetTypeHash(const FUIActionBindingHandle& Handle)
	{
		return ::GetTypeHash(Handle.RegistrationId);
	}

private:
	friend struct FUIActionBinding;
	
#if !UE_BUILD_SHIPPING
	// Using FString since the FName visualizer gets confused after live coding atm
	FString CachedDebugActionName;
#endif

	FUIActionBindingHandle(int32 InRegistrationId)
		: RegistrationId(InRegistrationId)
	{}

	int32 RegistrationId = INDEX_NONE;
};


/**
 * Metadata that can be set on activatable widgets via GetActivationMetadata & listened to via OnActivationMetadataChanged.
 * Useful for game-specific behaviors to be triggered by activation of certain widgets generically. For example changing
 * camera config of your game when certain widgets activate.
 * 
 * By default only has an enum for this metadata.
 */
struct FActivationMetadata
{
	FActivationMetadata() { }
	FActivationMetadata(uint8 InMetadataEnum) : MetadataEnum(InMetadataEnum) {}

	TOptional<uint8> GetMetadataEnum() const { return MetadataEnum; }

private:
	TOptional<uint8> MetadataEnum;
};

/**
 * Input Config that can be applied on widget activation. Allows for input setup  (Mouse capture, 
 * UI-only input, move / look ignore, etc), to be controlled by widget activation.
 */
USTRUCT(BlueprintType)
struct FUIInputConfig
{
	GENERATED_BODY()

	ECommonInputMode GetInputMode() const { return InputMode; }
	EMouseCaptureMode GetMouseCaptureMode() const { return MouseCaptureMode; }
	EMouseLockMode GetMouseLockMode() const { return MouseLockMode; }
	bool HideCursorDuringViewportCapture() const { return bHideCursorDuringViewportCapture; }

	UE_API FUIInputConfig();
	UE_API FUIInputConfig(ECommonInputMode InInputMode, EMouseCaptureMode InMouseCaptureMode, bool bInHideCursorDuringViewportCapture = true);
	UE_API FUIInputConfig(ECommonInputMode InInputMode, EMouseCaptureMode InMouseCaptureMode, EMouseLockMode InMouseLockMode, bool bInHideCursorDuringViewportCapture = true);

	bool operator==(const FUIInputConfig& Other) const
	{
		return bIgnoreMoveInput == Other.bIgnoreMoveInput
			&& bIgnoreLookInput == Other.bIgnoreLookInput
			&& InputMode == Other.InputMode
			&& MouseCaptureMode == Other.MouseCaptureMode
			&& MouseLockMode == Other.MouseLockMode
			&& bHideCursorDuringViewportCapture == Other.bHideCursorDuringViewportCapture;
	}

	bool operator!=(const FUIInputConfig& Other) const
	{
		return !operator==(Other);
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InputConfig)
	bool bIgnoreMoveInput = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InputConfig)
	bool bIgnoreLookInput = false;

	/** Simplification of config as string */
	UE_API FString ToString() const;

protected:


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InputConfig)
	ECommonInputMode InputMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InputConfig)
	EMouseCaptureMode MouseCaptureMode;


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InputConfig)
	EMouseLockMode MouseLockMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InputConfig)
	bool bHideCursorDuringViewportCapture = true;
};

#undef UE_API
