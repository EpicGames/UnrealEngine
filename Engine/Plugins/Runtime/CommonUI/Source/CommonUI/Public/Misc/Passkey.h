// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 * A Passkey is an object that allows friend-esque access to a single function by allowing only the pseudo-friend to construct the passkey.
 * In practice, the caller doesn't have to care, as passkey will be created implicitly from the property you're actually interested in.
 * 
 * Not intended to be used widely or with great regularity. Prefer proper encapsulation methods, and only use this as a last resort,
 *	as using it does undeniably make the code more cryptic to others.
 */
#define DEFINE_PASSKEY(FriendClass, PropertyType, PropertyName)	\
struct F##PropertyName##Passkey	\
{	\
	inline PropertyType& operator*() { return PropertyName; }	\
private:	\
	friend class FriendClass;	\
	PropertyType& PropertyName;	\
	F##PropertyName##Passkey(PropertyType& In##PropertyName) : PropertyName(In##PropertyName) {}	\
}

/*
Example:

struct FPseudoFriend
{
	void DoSomething()
	{
		UObject* Object = GetAUObjectFromSomewhere();
		MyPasskeyExample.RestrictedFunc(Object);
	}
	FPasskeyExample MyPasskeyExample;
};

struct FPasskeyExample
{
	// This method can be called successfully ONLY from within FPseudoFriend, as it's the only one able to construct the passkey
	//	Also note that we accept the key by copy, as that triggers the implicit construction that makes life easier for the caller
	
	DEFINE_PASSKEY(FPseudoFriend, UObject*, Object);
	void RestrictedFunc(FObjectPasskey ObjectPasskey)
	{
		UObject* Object = *ObjectPasskey;
		...
	}
}

*/