// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SSlider.h"

#define UE_API COMMONUI_API

struct FAnalogInputEvent;

/**
 * A Slate slider control is a linear scale and draggable handle.
 */
class SAnalogSlider : public SSlider
{
public:

	SLATE_BEGIN_ARGS(SAnalogSlider)
		: _IndentHandle(true)
		, _Locked(false)
		, _Orientation(EOrientation::Orient_Horizontal)
		, _SliderBarColor(FLinearColor::White)
		, _SliderHandleColor(FLinearColor::White)
		, _Style(&FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider"))
		, _StepSize(0.01f)
		, _Value(1.f)
		, _IsFocusable(true)
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
		, _OnAnalogCapture()
	{ }

		/** Whether the slidable area should be indented to fit the handle. */
		SLATE_ATTRIBUTE(bool, IndentHandle)

		/** Whether the handle is interactive or fixed. */
		SLATE_ATTRIBUTE(bool, Locked)

		/** The slider's orientation. */
		SLATE_ARGUMENT(EOrientation, Orientation)

		/** The color to draw the slider bar in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderBarColor)

		/** The color to draw the slider handle in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderHandleColor)

		/** The style used to draw the slider. */
		SLATE_STYLE_ARGUMENT(FSliderStyle, Style)

		/** The input mode while using the controller. */
		SLATE_ATTRIBUTE(float, StepSize)

		/** A value that drives where the slider handle appears. Value is normalized between 0 and 1. */
		SLATE_ATTRIBUTE(float, Value)

		/** Sometimes a slider should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT(bool, IsFocusable)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

		/** Invoked when the Controller is pressed and capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureBegin)

		/** Invoked when the controller capture is released.  */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureEnd)

		/** Called when the value is changed by the slider. */
		SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FOnFloatValueChanged, OnAnalogCapture)

	SLATE_END_ARGS()
	
	/**
	* Construct the widget.
	*
	* @param InDeclaration A declaration from which to construct the widget.
	*/
	UE_API void Construct(const SAnalogSlider::FArguments& InDeclaration);

	// Input overrides, for adding controller input to the slider
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent) override;
	UE_API virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;

	UE_API void SetUsingGamepad(bool InUsingGamepad);

private:
	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FOnFloatValueChanged OnAnalogCapture;

	bool bIsUsingGamepad = false;

	/** The last app time we stepped with analog input */
	double LastAnalogStepTime = 0;
};

#undef UE_API
