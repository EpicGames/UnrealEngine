// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Slate/SObjectTableRow.h"
#include "CommonButtonBase.h"

class STableViewBase;

/** 
 * A CommonUI version of the object table row that is aware of UCommonButtonBase.
 * Instead of bothering with handling mouse events directly, we rely on the entry being a button itself and respond to events from it.
 */
template <typename ItemType>
class SCommonButtonTableRow : public SObjectTableRow<ItemType>
{
public:
	SLATE_BEGIN_ARGS(SCommonButtonTableRow<ItemType>)
		:_bAllowDragging(true),
		_bAllowKeepPreselectedItems(true)
	{}
		SLATE_ARGUMENT(bool, bAllowDragging)
		SLATE_ARGUMENT(bool, bAllowDragDrop)
		SLATE_ARGUMENT(bool, bAllowKeepPreselectedItems)
		SLATE_EVENT(FOnRowHovered, OnHovered)
		SLATE_EVENT(FOnRowHovered, OnUnhovered)
		//	Drag and Drop functionality
		SLATE_EVENT(FOnObjectRowCanAcceptDrop, OnRowCanAcceptDrop)
		SLATE_EVENT(FOnObjectRowAcceptDrop, OnRowAcceptDrop)
		SLATE_EVENT(FOnObjectRowDragDetected, OnRowDragDetected)
		SLATE_EVENT(FOnObjectRowDragEnter, OnRowDragEnter)
		SLATE_EVENT(FOnObjectRowDragLeave, OnRowDragLeave)
		SLATE_EVENT(FOnObjectRowDragCancelled, OnRowDragCancelled)
		//	End Drag and Drop
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, UUserWidget& InChildWidget, UListViewBase* InOwnerListView)
	{
		SObjectTableRow<ItemType>::Construct(
			typename SObjectTableRow<ItemType>::FArguments()
			.bAllowDragging(InArgs._bAllowDragging)
			.bAllowDragDrop(InArgs._bAllowDragDrop)
			.bAllowKeepPreselectedItems(InArgs._bAllowKeepPreselectedItems)
			.OnHovered(InArgs._OnHovered)
			.OnUnhovered(InArgs._OnUnhovered)
			.OnRowCanAcceptDrop(InArgs._OnRowCanAcceptDrop)
			.OnRowAcceptDrop(InArgs._OnRowAcceptDrop)
			.OnRowDragDetected(InArgs._OnRowDragDetected)
			.OnRowDragLeave(InArgs._OnRowDragLeave)
			.OnRowDragEnter(InArgs._OnRowDragEnter)
			.OnRowDragCancelled(InArgs._OnRowDragCancelled)
			[
				InArgs._Content.Widget
			], 
			InOwnerTableView, InChildWidget, InOwnerListView);

		UCommonButtonBase* CommonButton = Cast<UCommonButtonBase>(&InChildWidget);
		if (ensureMsgf(CommonButton, TEXT("The widget object attached to an SCommonButtonTableRow is always expected to be a UCommonButtonBase.")))
		{
			// We override whatever settings this button claimed to have - the owning list still has the authority on the selection and toggle behavior of its rows
			ESelectionMode::Type SelectionMode = this->GetSelectionMode();
			CommonButton->SetIsToggleable(SelectionMode == ESelectionMode::SingleToggle || SelectionMode == ESelectionMode::Multi);
			CommonButton->SetIsSelectable(SelectionMode != ESelectionMode::None);
			CommonButton->SetIsInteractableWhenSelected(SelectionMode != ESelectionMode::None);
			CommonButton->SetAllowDragDrop(InArgs._bAllowDragDrop);
			CommonButton->SetTouchMethod(EButtonTouchMethod::PreciseTap);
		}
	}

	// We rely on the button to handle all of these things for us
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { SObjectWidget::OnMouseEnter(MyGeometry, MouseEvent); }
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override { SObjectWidget::OnMouseLeave(MouseEvent); }
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return SObjectWidget::OnMouseButtonDown(MyGeometry, MouseEvent); }
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return SObjectWidget::OnMouseButtonUp(MyGeometry, MouseEvent); }
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override { return FReply::Handled(); }

protected:
	virtual void InitializeObjectRow() override
	{
		SObjectTableRow<ItemType>::InitializeObjectRow();

		if (UCommonButtonBase* CommonButton = Cast<UCommonButtonBase>(this->WidgetObject))
		{
			CommonButton->OnClicked().AddSP(this, &SCommonButtonTableRow::HandleButtonClicked);
			CommonButton->OnDoubleClicked().AddSP(this, &SCommonButtonTableRow::HandleButtonDoubleClicked);
			CommonButton->OnHovered().AddSP(this, &SCommonButtonTableRow::HandleButtonHovered);
			CommonButton->OnUnhovered().AddSP(this, &SCommonButtonTableRow::HandleButtonUnhovered);
			CommonButton->OnIsSelectedChanged().AddSP(this, &SCommonButtonTableRow::HandleButtonSelectionChanged);

			if (this->GetAllowDragDrop())
			{
				CommonButton->OnCommonButtonDragDetected().BindSP(this, &SCommonButtonTableRow::OnDragDetected);
				CommonButton->OnCommonButtonDragEnter().BindSP(this, &SCommonButtonTableRow::OnDragEnter);
				CommonButton->OnCommonButtonDragLeave().BindSP(this, &SCommonButtonTableRow::OnDragLeave);
				CommonButton->OnCommonButtonDragOver().BindSP(this, &SCommonButtonTableRow::OnDragOver);
				CommonButton->OnCommonButtonDrop().BindSP(this, &SCommonButtonTableRow::OnDrop);
			}

			if (this->IsItemSelectable())
			{
				const bool bIsItemSelected = this->IsItemSelected();
				if (bIsItemSelected != CommonButton->GetSelected())
				{
					// Quietly set the button to reflect the item selection
					CommonButton->SetSelectedInternal(bIsItemSelected, false, false);
				}
			}
			else
			{
				CommonButton->SetIsSelectable(false);
			}
		}
	}

	virtual void ResetObjectRow() override
	{
		SObjectTableRow<ItemType>::ResetObjectRow();

		if (UCommonButtonBase* CommonButton = Cast<UCommonButtonBase>(this->WidgetObject))
		{
			CommonButton->OnClicked().RemoveAll(this);
			CommonButton->OnDoubleClicked().RemoveAll(this);
			CommonButton->OnHovered().RemoveAll(this);
			CommonButton->OnUnhovered().RemoveAll(this);
			CommonButton->OnIsSelectedChanged().RemoveAll(this);

			CommonButton->OnCommonButtonDragDetected().Unbind();
			CommonButton->OnCommonButtonDragEnter().Unbind();
			CommonButton->OnCommonButtonDragLeave().Unbind();
			CommonButton->OnCommonButtonDragOver().Unbind();
			CommonButton->OnCommonButtonDrop().Unbind();

			if (CommonButton->GetSelected())
			{
				// Quietly deselect the button to reset its visual state
				CommonButton->SetSelectedInternal(false, false, false);
			}
		}
	}

	virtual void DetectItemSelectionChanged() override
	{
		SObjectTableRow<ItemType>::DetectItemSelectionChanged();
		
		TSharedRef<ITypedTableView<ItemType>> OwnerTable = this->OwnerTablePtr.Pin().ToSharedRef();
		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = this->GetItemForThis(OwnerTable))
		{
			// Selection changes at the list level can happen directly or in response to another item being selected.
			// Regardless, just make sure the button's selection state is inline with the item's
			const UCommonButtonBase* CommonButton = Cast<UCommonButtonBase>(this->WidgetObject);
			const bool bIsItemSelected = OwnerTable->Private_IsItemSelected(*MyItemPtr);
			if (CommonButton && CommonButton->GetSelected() != bIsItemSelected)
			{
				OnItemSelectionChanged(bIsItemSelected);
			}
		}
	}

	virtual void OnItemSelectionChanged(bool bIsItemSelected) override
	{
		SObjectTableRow<ItemType>::OnItemSelectionChanged(bIsItemSelected);

		// Selection changes at the list level can happen directly or in response to another item being selected.
		// Regardless, just make sure the button's selection state is inline with the item's
		UCommonButtonBase* CommonButton = Cast<UCommonButtonBase>(this->WidgetObject);
		if (CommonButton && CommonButton->GetSelected() != bIsItemSelected)
		{
			CommonButton->SetSelectedInternal(bIsItemSelected, false);
		}
	}

private:
	void HandleButtonClicked()
	{
		TSharedRef<ITypedTableView<ItemType>> OwnerTable = this->OwnerTablePtr.Pin().ToSharedRef();

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = this->GetItemForThis(OwnerTable))
		{
			OwnerTable->Private_OnItemClicked(*MyItemPtr);
		}
	}

	void HandleButtonDoubleClicked()
	{
		TSharedRef<ITypedTableView<ItemType>> OwnerTable = this->OwnerTablePtr.Pin().ToSharedRef();

		if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = this->GetItemForThis(OwnerTable))
		{
			OwnerTable->Private_OnItemDoubleClicked(*MyItemPtr);
		}

		// Do we want to behave differently or normally on double clicks?
		// HandleButtonClicked();
	}

	void HandleButtonHovered()
	{
		this->OnHovered.ExecuteIfBound(*this->WidgetObject);
	}

	void HandleButtonUnhovered()
	{
		this->OnUnhovered.ExecuteIfBound(*this->WidgetObject);
	}

	void HandleButtonSelectionChanged(bool bIsButtonSelected)
	{
		const ESelectionMode::Type SelectionMode = this->GetSelectionMode();
		if (ensure(SelectionMode != ESelectionMode::None) && bIsButtonSelected != this->IsItemSelected())
		{
			TSharedRef<ITypedTableView<ItemType>> OwnerTable = this->OwnerTablePtr.Pin().ToSharedRef();

			if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = this->GetItemForThis(OwnerTable))
			{
				if (bIsButtonSelected)
				{
					if (SelectionMode != ESelectionMode::Multi)
					{
						OwnerTable->Private_ClearSelection();
					}
					OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
				}
				else
				{
					OwnerTable->Private_SetItemSelection(*MyItemPtr, false, true);
				}

				OwnerTable->Private_SignalSelectionChanged(ESelectInfo::Direct);
			}
		}
	}
};
