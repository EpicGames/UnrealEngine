// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SScrollBox.h"

/** SScrollBox can scroll through an arbitrary number of widgets. */
class SCommonHierarchicalScrollBox : public SScrollBox
{
public:

	// SWidget interface
	// This is a near copy of SScrollBox::OnNavigation. Ideally AppendFocusableChildren would be a virtual function in SScrollbox, but I
	//	don't have enough time to test functional changes to a core slate widget at the time of this writing. It was safer to isolate it.
	//	to this widget where I can test every case where it is used.
	virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;
	// End of SWidget interface

private:
	void AppendFocusableChildren(TArray<TSharedRef<SWidget>>& OutChildren, TSharedRef<SWidget> Owner);
};
