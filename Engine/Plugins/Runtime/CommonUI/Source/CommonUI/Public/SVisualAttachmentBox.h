// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBox.h"

#define UE_API COMMONUI_API

/**
 * Wrapper widget meant to handle native-side painting for UCommonVisualAttachment.
 */
class SVisualAttachmentBox : public SBox
{
public:
	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	UE_API void SetContentAnchor(FVector2D InContentAnchor);

private:
	mutable FVector2D InnerDesiredSize;

	FVector2D ContentAnchor = FVector2D::ZeroVector;
};

#undef UE_API
