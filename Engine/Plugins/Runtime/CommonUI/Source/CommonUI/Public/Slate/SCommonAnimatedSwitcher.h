// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Animation/CurveSequence.h"

#include "SCommonAnimatedSwitcher.generated.h"

#define UE_API COMMONUI_API

UENUM(BlueprintType)
enum class ECommonSwitcherTransition : uint8
{
	/** Fade transition only with no movement */
	FadeOnly,
	/** Increasing the active index goes right, decreasing goes left */
	Horizontal,
	/** Increasing the active index goes up, decreasing goes down */
	Vertical,
	/** Increasing the active index zooms in, decreasing zooms out */
	Zoom
};

UENUM(BlueprintType)
enum class ETransitionCurve : uint8
{
	/** Linear interpolation, with no easing */
	Linear,
	/** Quadratic ease in */
	QuadIn,
	/** Quadratic ease out */
	QuadOut,
	/** Quadratic ease in, quadratic ease out */
	QuadInOut,
	/** Cubic ease in */
	CubicIn,
	/** Cubic ease out */
	CubicOut,
	/** Cubic ease in, cubic ease out */
	CubicInOut,
};

/** Determines the switcher's behavior if the target of a transition is removed before it becomes the active widget. */
UENUM(BlueprintType)
enum class ECommonSwitcherTransitionFallbackStrategy : uint8
{
	/** Transition fallbacks are disabled and no special handling will occur if a transitioning widget is removed. */
	None,
	/** Fall back to the nearest valid slot at a lower index than the original target, or the first slot if there are none lower */
	Previous,
	/** Fall back to the nearest valid slot at a higher index than the original target, or the last slot if there are none higher */
	Next,
	/** Fall back to the first item in the switcher */
	First,
	/** Fall back to the last item in the switcher */
	Last
};

static inline ECurveEaseFunction TransitionCurveToCurveEaseFunction(ETransitionCurve CurveType)
{
	switch (CurveType)
	{
		default:
		case ETransitionCurve::Linear: return ECurveEaseFunction::Linear;
		case ETransitionCurve::QuadIn: return ECurveEaseFunction::QuadIn;
		case ETransitionCurve::QuadOut: return ECurveEaseFunction::QuadOut;
		case ETransitionCurve::QuadInOut: return ECurveEaseFunction::QuadInOut;
		case ETransitionCurve::CubicIn: return ECurveEaseFunction::CubicIn;
		case ETransitionCurve::CubicOut: return ECurveEaseFunction::CubicOut;
		case ETransitionCurve::CubicInOut: return ECurveEaseFunction::CubicInOut;
	}
}

class SCommonAnimatedSwitcher : public SWidgetSwitcher
{
public:
	DECLARE_DELEGATE_OneParam(FOnActiveIndexChanged, int32);
	DECLARE_DELEGATE_OneParam(FOnIsTransitioningChanged, bool);

	SLATE_BEGIN_ARGS(SCommonAnimatedSwitcher)
		: _InitialIndex(0)
		, _TransitionType(ECommonSwitcherTransition::FadeOnly)
		, _TransitionCurveType(ETransitionCurve::CubicInOut)
		, _TransitionDuration(0.4f)
		, _TransitionFallbackStrategy(ECommonSwitcherTransitionFallbackStrategy::Previous)
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}
		
		SLATE_ARGUMENT(int32, InitialIndex)

		SLATE_ARGUMENT(ECommonSwitcherTransition, TransitionType)
		SLATE_ARGUMENT(ETransitionCurve, TransitionCurveType)
		SLATE_ARGUMENT(float, TransitionDuration)
		SLATE_ARGUMENT(ECommonSwitcherTransitionFallbackStrategy, TransitionFallbackStrategy)
		SLATE_EVENT(FOnActiveIndexChanged, OnActiveIndexChanged)
		SLATE_EVENT(FOnIsTransitioningChanged, OnIsTransitioningChanged)

	SLATE_END_ARGS()
	
public:
	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API void TransitionToIndex(int32 NewWidgetIndex, bool bInstantTransition = false);

	UE_API FSlot* GetChildSlot(int32 SlotIndex);

	UE_API void SetTransition(float Duration, ETransitionCurve Curve);
	UE_API void SetTransition(float Duration, ETransitionCurve Curve, ECommonSwitcherTransition NewTransitionType);
	UE_API void SetTransitionType(ECommonSwitcherTransition NewTransitionType);

	UE_API bool IsTransitionPlaying() const;
	
	TWeakPtr<SWidget> GetPendingActiveWidget() const { return PendingActiveWidget; }
	int32 GetPendingActiveWidgetIndex() const { return PendingActiveWidgetIndex; }

	void SetTransitionFallbackStrategy(const ECommonSwitcherTransitionFallbackStrategy InStrategy) { TransitionFallbackStrategy = InStrategy; }
	ECommonSwitcherTransitionFallbackStrategy GetTransitionFallbackStrategy() const { return TransitionFallbackStrategy; }
	bool IsTransitionFallbackEnabled() const { return TransitionFallbackStrategy != ECommonSwitcherTransitionFallbackStrategy::None; }

protected:
	UE_API virtual void OnSlotAdded(int32 AddedIndex) override;
	UE_API virtual void OnSlotRemoved(int32 RemovedIndex, TSharedRef<SWidget> RemovedWidget, bool bWasActiveSlot) override;
	
private:
	UE_API EActiveTimerReturnType UpdateTransition(double InCurrentTime, float InDeltaTime);
	UE_API float GetTransitionProgress() const;
	UE_API int32 GetTransitionFallbackForIndex(int32 RemovedWidgetIndex) const;
	UE_API bool TryTransitionFallbackOfPendingWidget();
	UE_API bool TryTransitionFallbackOfActiveWidget(int32 RemovedWidgetIndex);

private:
	/** Anim sequence for the transition; plays twice per transition */
	FCurveSequence TransitionSequence;

	/** The pending active widget, set when the initial transition out completes. If set to a null value then we don't have any pending widget */
	TWeakPtr<SWidget> PendingActiveWidget;

	/** Tracks the index of PendingActiveWidget, if there is one. */
	int32 PendingActiveWidgetIndex = INDEX_NONE;

	/** Controls how we will choose another widget if a transitioning widget is removed during the transition. */
	ECommonSwitcherTransitionFallbackStrategy TransitionFallbackStrategy = ECommonSwitcherTransitionFallbackStrategy::None;

	/** If true, we are transitioning content out and need to play the sequence again to transition it in */
	bool bTransitioningOut = false;

	bool bIsTransitionTimerRegistered = false;
	ECommonSwitcherTransition TransitionType;

	FOnActiveIndexChanged OnActiveIndexChanged;
	FOnIsTransitioningChanged OnIsTransitioningChanged;
};

#undef UE_API
