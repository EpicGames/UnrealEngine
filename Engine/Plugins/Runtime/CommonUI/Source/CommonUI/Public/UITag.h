// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "Trace/SlateMemoryTags.h"

#include "UITag.generated.h"

#define UE_API COMMONUI_API

/** 
 * Implementation utility for typed subclasses of FGameplayTag. 
 * Implemented here instead of directly within the macro to make debugging possible. 
 */
template <typename TagT>
class TTypedTagStaticImpl
{
	friend TagT;

	static TagT AddNativeTag(const FString& TagBody)
	{
		if (!ensure(!TagBody.IsEmpty()))
		{
			return TagT();
		}

		FString TagStr;
		FString RootTagStr = FString::Printf(TEXT("%s."), TagT::GetRootTagStr());
		if (!TagBody.StartsWith(RootTagStr))
		{
			TagStr = RootTagStr + TagBody;
		}
		else
		{
			TagStr = TagBody;
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
			ensureAlwaysMsgf(false, TEXT("Passed unnecessary prefix [%s] when creating a tag of type [%s] with the body [%s]"),
				*RootTagStr, TNameOf<TagT>::GetName(), *TagBody);
#endif
		}

		return UGameplayTagsManager::Get().AddNativeGameplayTag(FName(*TagStr));
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Intended for console commands/cheats: not for shipping code! */
	static inline TagT FindFromString_DebugOnly(const FString& PartialTagName)
	{
		return UGameplayTagsManager::Get().FindGameplayTagFromPartialString_Slow(PartialTagName);
	}
#endif

	static bool ExportTextItem(const TagT& Tag, FString& ValueStr, int32 PortFlags)
	{
		ValueStr += Tag.GetTagName().ToString();
		return true;
	}

	static TagT TryConvert(FGameplayTag VanillaTag, bool bChecked)
	{
		if (VanillaTag.MatchesTag(StaticImpl.RootTag))
		{
			return TagT(VanillaTag);
		}
		else if (VanillaTag.IsValid() && bChecked)
		{
			check(false);
		}
		return TagT();
	}

	TTypedTagStaticImpl()
	{ 
		LLM_SCOPE(ELLMTag::UI);
		OnLastChanceToAddNativeTagsHandle = UGameplayTagsManager::CallOrRegister_OnAddNativeTagsDelegate(
			FSimpleMulticastDelegate::FDelegate::CreateLambda([this]()
			{
				StaticImpl.RootTag = UGameplayTagsManager::Get().AddNativeGameplayTag(TagT::GetRootTagStr());
			}));
	}

	~TTypedTagStaticImpl()
	{ 
		UGameplayTagsManager::UnregisterNativeTagDelegate(OnLastChanceToAddNativeTagsHandle);
	}

	TagT RootTag;
	static TTypedTagStaticImpl StaticImpl;
	FDelegateHandle OnLastChanceToAddNativeTagsHandle;
};

template <typename TagT>
TTypedTagStaticImpl<TagT> TTypedTagStaticImpl<TagT>::StaticImpl;

// Intended to be the absolute last thing in the definition of a UI tag
#define END_UI_TAG_DECL(TagType, TagRoot)	\
public:	\
	TagType() {}	\
	static TagType GetRootTag() { return TTypedTagStaticImpl<TagType>::StaticImpl.RootTag; }	\
	static TagType TryConvert(FGameplayTag FromTag) { return TTypedTagStaticImpl<TagType>::TryConvert(FromTag, false); }	\
	static TagType ConvertChecked(FGameplayTag FromTag) { return TTypedTagStaticImpl<TagType>::TryConvert(FromTag, true); }	\
	static TagType AddNativeTag(const FString& TagBody) { return TTypedTagStaticImpl<TagType>::AddNativeTag(TagBody); }	\
	bool ExportTextItem(FString& ValueStr, const TagType& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const	\
	{	\
		return TTypedTagStaticImpl<TagType>::ExportTextItem(*this, ValueStr, PortFlags);	\
	}	\
protected:	\
	TagType(FGameplayTag Tag) { TagName = Tag.GetTagName(); }	\
	static const TCHAR* GetRootTagStr() { return TagRoot; }	\
	friend class TTypedTagStaticImpl<TagType>;	\
};	\
Expose_TNameOf(TagType)	\
template<>	\
struct TStructOpsTypeTraits<TagType> : public TStructOpsTypeTraitsBase2<TagType>	\
{	\
	enum	\
	{	\
		WithNetSerializer = true,	\
		WithNetSharedSerialization = true,	\
		WithPostSerialize = true,	\
		WithStructuredSerializeFromMismatchedTag = true,	\
		WithExportTextItem = true,	\
		WithImportTextItem = true	\
	};

USTRUCT(meta = (Categories = "UI"))
struct FUITag : public FGameplayTag
{
	GENERATED_BODY()
	END_UI_TAG_DECL(FUITag, TEXT("UI"))
};

USTRUCT(meta = (Categories = "UI.Action"))
struct FUIActionTag : public FUITag
{
	GENERATED_BODY()
	END_UI_TAG_DECL(FUIActionTag, TEXT("UI.Action"))
};

//
//USTRUCT(meta = (Categories = "UI.Widget"))
//struct FUIWidgetTag : public FGameplayTag
//{
//	GENERATED_BODY()
//	END_UI_TAG_DECL(FUIWidgetTag, TEXT("UI.Widget"))
//};
//
//USTRUCT(meta = (Categories = "UI.Pedestal"))
//struct FUIPedestalTag : public FGameplayTag
//{
//	GENERATED_BODY()
//	END_UI_TAG_DECL(FUIPedestalTag, TEXT("UI.Pedestal"))
//};

struct FGlobalUITags : public FGameplayTagNativeAdder
{
	FUIActionTag UIAction_Cancel;
	FUIActionTag UIAction_PreviousTab;
	FUIActionTag UIAction_NextTab;
	FUIActionTag UIAction_Confirm;

	void AddTags()
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

		UIAction_Cancel = FUIActionTag::AddNativeTag(TEXT("Cancel"));
		UIAction_Confirm = FUIActionTag::AddNativeTag(TEXT("Confirm"));
		UIAction_PreviousTab = FUIActionTag::AddNativeTag(TEXT("PreviousTab"));
		UIAction_NextTab = FUIActionTag::AddNativeTag(TEXT("NextTab"));
	}

	inline static const FGlobalUITags& Get() { return GUITags; }

private:
	static UE_API FGlobalUITags GUITags;
};

#undef UE_API
