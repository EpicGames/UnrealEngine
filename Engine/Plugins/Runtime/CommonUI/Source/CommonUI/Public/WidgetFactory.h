// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CommonPoolableWidgetInterface.h"
#include "Slate/SObjectWidget.h"

class ITableRow;
class STableViewBase;

#define ENABLE_WIDGET_FACTORY_POOLING 1

template <class WidgetType>
class TWidgetFactory : public FGCObject
{
	static_assert(TIsDerivedFrom<WidgetType, UUserWidget>::IsDerived, "Can only use TWidgetFactory with types deriving from UUserWidget");
private:
	inline UGameInstance* OwningObjectAsGameInstance() const { return Cast<UGameInstance>(OuterGetter()); }
	inline UWorld* OwningObjectAsWorld() const { return Cast<UWorld>(OuterGetter()); }
	inline APlayerController* OwningObjectAsPlayerController() const { return Cast<APlayerController>(OuterGetter()); }

public:
	typedef TFunctionRef<TSharedPtr<SObjectWidget>(UUserWidget*, TSharedRef<SWidget>)> ConstructMethodType;

	TWidgetFactory() = default;
	TWidgetFactory(UWidget& InOwningWidget, TSubclassOf<WidgetType> InWidgetClass, TFunction<UObject*(void)>&& InOuterGetter, bool InTrackActiveWidgets = false)
		: OwningWidget(&InOwningWidget)
	    , WidgetClass(InWidgetClass)
	    , OuterGetter(MoveTemp(InOuterGetter))
		, bTrackActiveWidgets(InTrackActiveWidgets)
	{
		ensure(WidgetClass);
	}

	TWidgetFactory(TWidgetFactory&& Other)
	    : ActiveWidgets(MoveTemp(Other.ActiveWidgets))
	    , InactiveWidgets(MoveTemp(Other.InactiveWidgets))
	    , CachedSlateWidgets(MoveTemp(Other.CachedSlateWidgets))
		, OwningWidget(Other.OwningWidget)
	    , WidgetClass(Other.WidgetClass)
	    , OuterGetter(MoveTemp(Other.OuterGetter))
	{
		Other.ActiveWidgets.Empty();
		Other.InactiveWidgets.Empty();
		Other.OwningWidget.Reset();
		Other.WidgetClass = nullptr;
		Other.OuterGetter = nullptr;
		Other.bTrackActiveWidgets = false;
	}

	virtual ~TWidgetFactory()
	{
		ensureMsgf(ActiveWidgets.Num() == 0, TEXT("TWidgetFactory: ActiveWidgets not empty! Please check your usage: \n\tAre you overriding ReleaseSlateResources and calling Reset in your User Widget?"));
		ensureMsgf(InactiveWidgets.Num() == 0, TEXT("TWidgetFactory: InactiveWidgets not empty! Please check your usage: \n\tAre you overriding ReleaseSlateResources and calling Reset in your User Widget?"));
		ensureMsgf(CachedSlateWidgets.Num() == 0, TEXT("TWidgetFactory: CachedSlateWidgets not empty! Please check your usage: \n\tAre you overriding ReleaseSlateResources and calling Reset in your User Widget?"));
	}

	TWidgetFactory& operator=(TWidgetFactory&& Other)
	{
		ActiveWidgets = MoveTemp(Other.ActiveWidgets);
		InactiveWidgets = MoveTemp(Other.InactiveWidgets);
		CachedSlateWidgets	= MoveTemp(Other.CachedSlateWidgets);
		OwningWidget = Other.OwningWidget;
		WidgetClass	= Other.WidgetClass;
		OuterGetter	= MoveTemp(Other.OuterGetter);
		bTrackActiveWidgets = Other.bTrackActiveWidgets;

		Other.ActiveWidgets.Empty();
		Other.InactiveWidgets.Empty();
		Other.CachedSlateWidgets.Empty();
		Other.OwningWidget.Reset();
		Other.WidgetClass = nullptr;
		Other.OuterGetter = nullptr;
		Other.bTrackActiveWidgets = false;

		return *this;
	}

	bool IsInitialized() const
	{
		return WidgetClass && OuterGetter != nullptr;
	}

	const TArray<WidgetType*>& GetActiveWidgets() const { return ActiveWidgets; }

	void PreConstruct(int32 Num)
	{
#if ENABLE_WIDGET_FACTORY_POOLING
		if (!OuterGetter)
		{
			return;
		}

		for (int32 i = 0; i < Num; ++i)
		{
			if (WidgetClass)
			{
				WidgetType* NewWidget = nullptr;
				if (UWorld* World = OwningObjectAsWorld())
				{
					NewWidget = CreateWidget<WidgetType>(World, WidgetClass);
				}
				else if (UGameInstance* GameIn = OwningObjectAsGameInstance())
				{
					NewWidget = CreateWidget<WidgetType>(GameIn, WidgetClass);
				}
				else if (APlayerController* PC = OwningObjectAsPlayerController())
				{
					NewWidget = CreateWidget<WidgetType>(PC, WidgetClass);
				}

				if (NewWidget)
				{
					InactiveWidgets.Add(NewWidget);
				}
			}
		}
#endif // ENABLE_WIDGET_FACTORY_POOLING
	}

	/** Method to get a widget from this factory, can sometimes call CreateWidget */
	WidgetType* Acquire()
	{
		WidgetType* Result = InactiveWidgets.Num() > 0 ? InactiveWidgets.Pop().Get() : nullptr;

		check(WidgetClass);
		if (!Result && OuterGetter)
		{
			if (UWorld* World = OwningObjectAsWorld())
			{
				Result = CreateWidget<WidgetType>(World, WidgetClass);
			}
			else if (UGameInstance* GameIn = OwningObjectAsGameInstance())
			{
				Result = CreateWidget<WidgetType>(GameIn, WidgetClass);
			}
			else if (APlayerController* PC = OwningObjectAsPlayerController())
			{
				Result = CreateWidget<WidgetType>(PC, WidgetClass);
			}
			else
			{
				checkNoEntry();
			}
		}

		check(Result);

#if WITH_EDITOR
		if (OwningWidget.IsValid())
		{
			Result->SetDesignerFlags(OwningWidget->GetDesignerFlags());
		}
#endif

#if ENABLE_WIDGET_FACTORY_POOLING
		if (bTrackActiveWidgets)
		{
			ActiveWidgets.Add(Result);
		}
#endif // ENABLE_WIDGET_FACTORY_POOLING
		
		if (Result->template Implements<UCommonPoolableWidgetInterface>())
		{
			ICommonPoolableWidgetInterface::Execute_OnAcquireFromPool(Result);
		}

		return Result;
	}

	/** Return a widget to the pool, allowing it to be reused in the future */
	void Release(WidgetType* Widget)
	{
		if (Widget->template Implements<UCommonPoolableWidgetInterface>())
		{
			ICommonPoolableWidgetInterface::Execute_OnReleaseToPool(Widget);
		}
#if ENABLE_WIDGET_FACTORY_POOLING
		if (bTrackActiveWidgets)
		{
			const int32 NumRemoved = ActiveWidgets.Remove(Widget);
			if (NumRemoved > 0)
			{
				InactiveWidgets.Push(Widget);
			}
		}
		else
		{
			InactiveWidgets.Push(Widget);
		}
#endif // ENABLE_WIDGET_FACTORY_POOLING
	}

	void Reset(const bool bReleaseSlate = false, const bool bMoveToInactive = false)
	{
		if (bTrackActiveWidgets)
		{
			if (!IsEngineExitRequested())
			{
				for (WidgetType* Widget : ActiveWidgets)
				{
					if (Widget && Widget->template Implements<UCommonPoolableWidgetInterface>())
					{
						ICommonPoolableWidgetInterface::Execute_OnReleaseToPool(Widget);
					}
				}
			}
		}

		if (!bMoveToInactive)
		{
			InactiveWidgets.Empty();
		}
		else
		{
			InactiveWidgets.Append(ActiveWidgets);
		}

		ActiveWidgets.Empty();

		if (bReleaseSlate)
		{
			CachedSlateWidgets.Reset();
		}
	}

	void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		if (bTrackActiveWidgets)
		{
			Collector.AddReferencedObjects<WidgetType>(ActiveWidgets, OwningWidget.Get());
		}
		Collector.AddReferencedObjects<WidgetType>(InactiveWidgets, OwningWidget.Get());
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("TWidgetFactory");
	}

	template <class DerivedWidgetType = SObjectWidget,
	          class                   = typename TEnableIf<TIsDerivedFrom<DerivedWidgetType, SObjectWidget>::IsDerived, DerivedWidgetType>::Type>
	TSharedRef<DerivedWidgetType> TakeAndCacheWidget(WidgetType* Key, ConstructMethodType ConstructMethod)
	{
		TSharedPtr<SWidget>& Cache = CachedSlateWidgets.FindOrAdd(Key);
		if (!Cache.IsValid())
		{
			Cache = Key->template TakeDerivedWidget<DerivedWidgetType>(ConstructMethod);
		}

		return StaticCastSharedRef<DerivedWidgetType>(Cache.ToSharedRef());
	}

	/** Convenience function for SObjectTableRows takes and caches the widget, then creates the object row around it */
	template <class DerivedWidgetType = SObjectWidget,
	          class                   = typename TEnableIf<TIsDerivedFrom<DerivedWidgetType, ITableRow>::IsDerived, DerivedWidgetType>::Type>
	TSharedRef<DerivedWidgetType> TakeAndCacheRow(WidgetType* Key, const TSharedRef<STableViewBase>& DestinationTable)
	{
		TSharedPtr<SWidget>& Cache = CachedSlateWidgets.FindOrAdd(Key);
		if (!Cache.IsValid())
		{
			Cache = Key->template TakeDerivedWidget<DerivedWidgetType>(
			    [DestinationTable](UUserWidget* Widget, TSharedRef<SWidget> SafeContent) 
				{
				    return SNew(DerivedWidgetType, DestinationTable, Widget)
				        [ 
							SafeContent 
						];
				});
		}

		return StaticCastSharedRef<DerivedWidgetType>(Cache.ToSharedRef());
	}

private:
	TArray<TObjectPtr<WidgetType>>	ActiveWidgets;
	TArray<TObjectPtr<WidgetType>>	InactiveWidgets;
	TMap<WidgetType*, TSharedPtr<SWidget>>	CachedSlateWidgets;
	TWeakObjectPtr<UWidget> OwningWidget;
	TSubclassOf<WidgetType>	WidgetClass;
	TFunction<UObject*(void)>	OuterGetter;
	bool	bTrackActiveWidgets;
};

using FUserWidgetFactory = TWidgetFactory<UUserWidget>;
