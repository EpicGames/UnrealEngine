// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidgetPool.h"
#include "Slate/SCommonAnimatedSwitcher.h"
#include "CommonActivatableWidgetContainer.generated.h"

#define UE_API COMMONUI_API

class SCommonAnimatedSwitcher;
enum class ECommonSwitcherTransition : uint8;
enum class ETransitionCurve : uint8;

class UCommonActivatableWidget;
class SOverlay;
class SSpacer;

/** 
 * Base of widgets built to manage N activatable widgets, displaying one at a time.
 * Intentionally meant to be black boxes that do not expose child/slot modification like a normal panel widget.
 */
UCLASS(MinimalAPI, Abstract)
class UCommonActivatableWidgetContainerBase : public UWidget
{
	GENERATED_BODY()

public:
	UE_API UCommonActivatableWidgetContainerBase(const FObjectInitializer& Initializer);

	/** Adds an activatable widget to the container. See BP_AddWidget for more info. */
	template <typename ActivatableWidgetT = UCommonActivatableWidget>
	ActivatableWidgetT* AddWidget(TSubclassOf<UCommonActivatableWidget> ActivatableWidgetClass)
	{
		// Don't actually add the widget if the cast will fail
		if (ActivatableWidgetClass && ActivatableWidgetClass->IsChildOf<ActivatableWidgetT>())
		{
			return Cast<ActivatableWidgetT>(AddWidgetInternal(ActivatableWidgetClass, [](UCommonActivatableWidget&) {}));
		}
		return nullptr;
	}

	/** 
	 * Generates (either creates or pulls from the inactive pool) instance of the given widget class and adds it to the container.
	 * The provided lambda is called after the instance has been generated and before it is actually added to the container.
	 * So if you've got setup to do on the instance before it potentially activates, the lambda is the place to do it.
	 */
	template <typename ActivatableWidgetT = UCommonActivatableWidget>
	ActivatableWidgetT* AddWidget(TSubclassOf<UCommonActivatableWidget> ActivatableWidgetClass, TFunctionRef<void(ActivatableWidgetT&)> InstanceInitFunc)
	{
		// Don't actually add the widget if the cast will fail
		if (ActivatableWidgetClass && ActivatableWidgetClass->IsChildOf<ActivatableWidgetT>())
		{
			return Cast<ActivatableWidgetT>(AddWidgetInternal(ActivatableWidgetClass, [&InstanceInitFunc] (UCommonActivatableWidget& WidgetInstance) 
				{
					InstanceInitFunc(*CastChecked<ActivatableWidgetT>(&WidgetInstance));
				}));
		}
		return nullptr;
	}

	/** 
	 * Adds an activatable widget instance to the container. 
	 * This instance is not pooled in any way by the stack and responsibility for ownership lies with the original creator of the widget.
	 * 
	 * NOTE: In general, it is *strongly* recommended that you opt for the class-based AddWidget above. This one is mostly just here for legacy support.
	 */
	UE_API void AddWidgetInstance(UCommonActivatableWidget& ActivatableWidget);

	UE_API void RemoveWidget(UCommonActivatableWidget& WidgetToRemove);

	UFUNCTION(BlueprintCallable, Category = ActivatableWidgetStack)
	UE_API UCommonActivatableWidget* GetActiveWidget() const;

	const TArray<UCommonActivatableWidget*>& GetWidgetList() const { return WidgetList; }

	UE_API int32 GetNumWidgets() const;

	UFUNCTION(BlueprintCallable, Category = ActivatableWidgetContainer)
	UE_API void ClearWidgets();

	UFUNCTION(BlueprintCallable, Category = ActivatableWidgetContainer)
	UE_API void SetTransitionDuration(float Duration);
	UFUNCTION(BlueprintCallable, Category = ActivatableWidgetContainer)
	UE_API float GetTransitionDuration() const;

	DECLARE_EVENT_OneParam(UCommonActivatableWidgetContainerBase, FOnDisplayedWidgetChanged, UCommonActivatableWidget*);
	FOnDisplayedWidgetChanged& OnDisplayedWidgetChanged() const { return OnDisplayedWidgetChangedEvent; }

	DECLARE_EVENT_TwoParams(UCommonActivatableWidgetContainerBase, FTransitioningChanged, UCommonActivatableWidgetContainerBase* /*Widget*/, bool /*bIsTransitioning*/);
	FTransitioningChanged OnTransitioningChanged;

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UE_API virtual void OnWidgetRebuilt() override;

	virtual void OnWidgetAddedToList(UCommonActivatableWidget& AddedWidget) { unimplemented(); }

	UE_API void SetSwitcherIndex(int32 TargetIndex, bool bInstantTransition = false);

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif // WITH_EDITOR

	/** The type of transition to play between widgets */
	UPROPERTY(EditAnywhere, Category = Transition)
	ECommonSwitcherTransition TransitionType;

	/** The curve function type to apply to the transition animation */
	UPROPERTY(EditAnywhere, Category = Transition)
	ETransitionCurve TransitionCurveType;

	/** The total duration of a single transition between widgets */
	UPROPERTY(EditAnywhere, Category = Transition)
	float TransitionDuration = 0.4f;

	/**
	 * Whether to completely reset the pool of widgets when slate resources are released.
	 * This usually happens when changing maps. You may not want to have all frontend screens loaded taking up memory while in game and vice versa.
	 * Enabling this means widgets will have to be loaded again when re-entering the map next time.
	 */
	UPROPERTY(EditAnywhere, Category = Performance)
	bool bResetPoolWhenReleasingSlateResources = false;

	/**
	 * Controls how we will choose another widget if a transitioning widget is removed during the transition.
	 * Note for Queues and Stacks, ECommonSwitcherTransitionFallbackStrategy::Previous is a good option.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Transition)
	ECommonSwitcherTransitionFallbackStrategy TransitionFallbackStrategy = ECommonSwitcherTransitionFallbackStrategy::None;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UCommonActivatableWidget>> WidgetList;

	UPROPERTY(Transient)
	TObjectPtr<UCommonActivatableWidget> DisplayedWidget;

	UPROPERTY(Transient)
	FUserWidgetPool GeneratedWidgetsPool;

	TSharedPtr<SOverlay> MyOverlay;
	TSharedPtr<SSpacer> MyInputGuard;
	TSharedPtr<SCommonAnimatedSwitcher> MySwitcher;

private:
	/** 
	 * Adds a widget of the given class to the container. 
	 * Note that all widgets added to the container are pooled, so the caller should not try to cache and re-use the created widget.
	 * 
	 * It is possible for multiple instances of the same class to be added to the container at once, so any instance created in the past
	 * is not guaranteed to be the one returned this time.
	 *
	 * So in practice, you should not trust that any prior state has been retained on the returned widget, and establish all appropriate properties every time.
	 */
	UFUNCTION(BlueprintCallable, Category = ActivatableWidgetStack, meta = (DeterminesOutputType = ActivatableWidgetClass, DisplayName = "Push Widget"))
	UE_API UCommonActivatableWidget* BP_AddWidget(TSubclassOf<UCommonActivatableWidget> ActivatableWidgetClass);

	UFUNCTION(BlueprintCallable, Category = ActivatableWidgetContainer)
	UE_API void RemoveWidget(UCommonActivatableWidget* WidgetToRemove);

	UE_API UCommonActivatableWidget* AddWidgetInternal(TSubclassOf<UCommonActivatableWidget> ActivatableWidgetClass, TFunctionRef<void(UCommonActivatableWidget&)> InitFunc);
	UE_API void RegisterInstanceInternal(UCommonActivatableWidget& NewWidget);

	UE_API void HandleSwitcherIsTransitioningChanged(bool bIsTransitioning);
	UE_API void HandleActiveIndexChanged(int32 ActiveWidgetIndex);
	UE_API void HandleActiveWidgetDeactivated(UCommonActivatableWidget* DeactivatedWidget);
	
	/**
	 * This is a bit hairy and very edge-casey, but a necessary measure to ensure expected Slate interaction behavior.
	 *
	 * Since we immediately remove slots from our switcher in response to changes to the active index of the switcher, we can
	 * wind up confusing the HittestGrid for that frame. The grid (correctly) thinks the widget at the previously displayed index is what it
	 * should interact with, but it skips it because we've already released all references and destroyed it. This causes pointer
	 * input (most importantly the synthetic mouse move) to fall through our container for that frame, potentially triggering interactions
	 * with elements that, as far as any user can tell, were never actually visible!
	 *
	 * So, when we remove a slot, we hold a reference to the SWidget in that slot for a single frame, to ensure hittest grid integrity.
	 * This does delay destruction of the removed SObjectWidget by one frame, but that does not present any discernable issue,
	 * as it's no different from any other inactive widget within a switcher.
	 */
	UE_API void ReleaseWidget(const TSharedRef<SWidget>& WidgetToRelease);
	TArray<TSharedPtr<SWidget>> ReleasedWidgets;

	bool bRemoveDisplayedWidgetPostTransition = false;

	mutable FOnDisplayedWidgetChanged OnDisplayedWidgetChangedEvent;
};

//////////////////////////////////////////////////////////////////////////
// UCommonActivatableWidgetStack
//////////////////////////////////////////////////////////////////////////

/** 
 * A display stack of ActivatableWidget elements. 
 * 
 * - Only the widget at the top of the stack is displayed and activated. All others are deactivated.
 * - When that top-most displayed widget deactivates, it's automatically removed and the preceding entry is displayed/activated.
 * - If RootContent is provided, it can never be removed regardless of activation state
 */
UCLASS(MinimalAPI)
class UCommonActivatableWidgetStack : public UCommonActivatableWidgetContainerBase
{
	GENERATED_BODY()

public:

	UE_API UCommonActivatableWidget* GetRootContent() const;

protected:
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void OnWidgetAddedToList(UCommonActivatableWidget& AddedWidget) override;
	
private:
	/** Optional widget to auto-generate as the permanent root element of the stack */
	UPROPERTY(EditAnywhere, Category = Content)
	TSubclassOf<UCommonActivatableWidget> RootContentWidgetClass;

	UPROPERTY(Transient)
	TObjectPtr<UCommonActivatableWidget> RootContentWidget;
};

//////////////////////////////////////////////////////////////////////////
// UCommonActivatableWidgetQueue
//////////////////////////////////////////////////////////////////////////

/** 
 * A display queue of ActivatableWidget elements. 
 *
 * - Only one widget is active/displayed at a time, all others in the queue are deactivated.
 * - When the active widget deactivates, it is automatically removed from the widget, 
 *		released back to the pool, and the next widget in the queue (if any) is displayed
 */
UCLASS(MinimalAPI)
class UCommonActivatableWidgetQueue : public UCommonActivatableWidgetContainerBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void OnWidgetAddedToList(UCommonActivatableWidget& AddedWidget) override;
};

#undef UE_API
