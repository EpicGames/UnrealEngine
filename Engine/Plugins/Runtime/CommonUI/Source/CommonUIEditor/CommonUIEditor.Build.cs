// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CommonUIEditor : ModuleRules
{
	public CommonUIEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
		new string[]
			{
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"Engine",
				"EngineAssetDefinitions",
				"PropertyEditor",
				"InputCore",
				"Slate",
				"UMG",
				"SlateCore",
				"CommonUI",
                "EditorWidgets",	
				"UnrealEd",
				"GameplayTags",
				"GameplayTagsEditor",
				"AssetRegistry",
				"ToolMenus",
			}
        );

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DataTableEditor",
			}
		);

		PublicIncludePaths.AddRange(
			new string[]
			{
			}
		);
	}
}
