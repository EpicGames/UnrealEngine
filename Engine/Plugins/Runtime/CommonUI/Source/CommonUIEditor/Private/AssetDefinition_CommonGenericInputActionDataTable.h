// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Input/CommonGenericInputActionDataTable.h"
#include "Table/AssetDefinition_DataTable.h"
#include "AssetDefinition_CommonGenericInputActionDataTable.generated.h"

UCLASS()
class UAssetDefinition_CommonGenericInputActionDataTable : public UAssetDefinition_DataTable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_CommonGenericInputActionDataTable", "Common UI InputActionDataTable"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(139.f, 69.f, 19.f); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCommonGenericInputActionDataTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "AssetDefinition_CommonGenericInputActionDataTableSubMenu", "Data Table"), ECategoryMenuType::Section)
		};
		return Categories;
	}
};
