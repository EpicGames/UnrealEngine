// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonGenericInputActionDataTableFactory.h"
#include "Input/CommonGenericInputActionDataTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CommonGenericInputActionDataTableFactory)

UCommonGenericInputActionDataTableFactory::UCommonGenericInputActionDataTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCommonGenericInputActionDataTable::StaticClass();
}

UObject* UCommonGenericInputActionDataTableFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class && Class->IsChildOf(UCommonGenericInputActionDataTable::StaticClass()));
	return NewObject<UCommonGenericInputActionDataTable>(Parent, Class, Name, Flags);
}

