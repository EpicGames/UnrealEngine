// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonUIEditorModule.h"
#include "CommonVideoPlayerCustomization.h"

#include "GameplayTagsEditorModule.h"
#include "PropertyEditorModule.h"
#include "UITag.h"

IMPLEMENT_MODULE(FCommonUIEditorModule, CommonUIEditor);

void FCommonUIEditorModule::StartupModule() 
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	PropertyModule.RegisterCustomClassLayout(
		TEXT("CommonVideoPlayer"),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCommonVideoPlayerCustomization::MakeInstance));
	
	PropertyModule.RegisterCustomPropertyTypeLayout(FUITag::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayTagCustomizationPublic::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FUIActionTag::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayTagCustomizationPublic::MakeInstance));
}

void FCommonUIEditorModule::ShutdownModule()
{
	if (!UObjectInitialized())
	{
		return;
	}

	// Unregister the details customization
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("CommonVideoPlayer"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("UITag"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("UIActionTag"));
	}
}
