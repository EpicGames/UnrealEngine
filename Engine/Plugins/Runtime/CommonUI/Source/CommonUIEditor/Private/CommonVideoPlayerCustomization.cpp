// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonVideoPlayerCustomization.h"
#include "CommonVideoPlayer.h"
#include "DetailWidgetRow.h"
#include "EditorWidgetsModule.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Modules/ModuleManager.h"

float VideoPlayerPreviewStepSize = 0.25f;
FAutoConsoleVariableRef CVarVideoPlayerPreviewStepSize(
	TEXT("CommonUI.VideoPlayer.PreviewStepSize"),
	VideoPlayerPreviewStepSize,
	TEXT(""));

TSharedRef<IDetailCustomization> FCommonVideoPlayerCustomization::MakeInstance()
{
	return MakeShareable(new FCommonVideoPlayerCustomization());
}

void FCommonVideoPlayerCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		return;
	}
	VideoPlayer = Cast<UCommonVideoPlayer>(Objects[0].Get());
	if (!VideoPlayer.IsValid())
	{
		return;
	}
	
	FTransportControlArgs TransportControlArgs;
	TransportControlArgs.OnBackwardEnd.BindSP(this, &FCommonVideoPlayerCustomization::HandleGoToStartClicked);
	TransportControlArgs.OnBackwardStep.BindSP(this, &FCommonVideoPlayerCustomization::HandleBackwardStep);
	TransportControlArgs.OnBackwardPlay.BindSP(this, &FCommonVideoPlayerCustomization::HandleReverseClicked);
	TransportControlArgs.OnForwardPlay.BindSP(this, &FCommonVideoPlayerCustomization::HandlePlayClicked);
	TransportControlArgs.OnForwardStep.BindSP(this, &FCommonVideoPlayerCustomization::HandleForwardStep);
	TransportControlArgs.OnForwardEnd.BindSP(this, &FCommonVideoPlayerCustomization::HandleGoToEndClicked);
	TransportControlArgs.OnGetPlaybackMode.BindSP(this, &FCommonVideoPlayerCustomization::GetPlaybackMode);

	TransportControlArgs.WidgetsToCreate.Emplace(ETransportControlWidgetType::BackwardEnd);
	TransportControlArgs.WidgetsToCreate.Emplace(ETransportControlWidgetType::BackwardStep);
	TransportControlArgs.WidgetsToCreate.Emplace(ETransportControlWidgetType::BackwardPlay);
	TransportControlArgs.WidgetsToCreate.Emplace(ETransportControlWidgetType::ForwardPlay);
	TransportControlArgs.WidgetsToCreate.Emplace(ETransportControlWidgetType::ForwardStep);
	TransportControlArgs.WidgetsToCreate.Emplace(ETransportControlWidgetType::ForwardEnd);
	TransportControlArgs.WidgetsToCreate.Emplace(FOnMakeTransportWidget::CreateSP(this, &FCommonVideoPlayerCustomization::HandleCreateMuteToggleWidget));

	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::Get().LoadModuleChecked<FEditorWidgetsModule>(TEXT("EditorWidgets"));

	IDetailCategoryBuilder& VideoPlayerCategory = DetailLayout.EditCategory(TEXT("VideoPlayer"));
	VideoPlayerCategory.AddCustomRow(FText::FromString(TEXT("Video Controls"))).WholeRowContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(10.f, 0.f, 0.f, 0.f)
			.AutoWidth()
			[
				EditorWidgetsModule.CreateTransportControl(TransportControlArgs)
			]
			+ SHorizontalBox::Slot()
			.Padding(10.f, 0.f, 4.f, 0.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SNumericEntryBox<float>)
					.AllowSpin(false)
					.MinValue(0.f)
					.MaxValue(this, &FCommonVideoPlayerCustomization::GetMaxPlaybackTimeValue)
					.Value(this, &FCommonVideoPlayerCustomization::GetPlaybackTimeValue)
					.OnValueCommitted(this, &FCommonVideoPlayerCustomization::HandlePlaybackTimeCommitted)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &FCommonVideoPlayerCustomization::GetMaxPlaybackTimeText)
				]
			]	
		];
}

FReply FCommonVideoPlayerCustomization::HandlePlayClicked()
{
	if (VideoPlayer->IsPlaying() && VideoPlayer->GetPlaybackRate() > 0.f)
	{
		VideoPlayer->Pause();
	}
	else
	{
		VideoPlayer->Play();
	}
	
	return FReply::Handled();
}

FReply FCommonVideoPlayerCustomization::HandleGoToStartClicked()
{
	VideoPlayer->Seek(0.f);
	return FReply::Handled();
}

FReply FCommonVideoPlayerCustomization::HandleBackwardStep()
{
	VideoPlayer->Seek(VideoPlayer->GetPlaybackTime() - VideoPlayerPreviewStepSize);
	return FReply::Handled();
}

FReply FCommonVideoPlayerCustomization::HandleForwardStep()
{
	VideoPlayer->Seek(VideoPlayer->GetPlaybackTime() + VideoPlayerPreviewStepSize);
	return FReply::Handled();
}

FReply FCommonVideoPlayerCustomization::HandleGoToEndClicked()
{
	VideoPlayer->Seek(VideoPlayer->GetVideoDuration());
	return FReply::Handled();
}

FReply FCommonVideoPlayerCustomization::HandlePauseClicked()
{
	VideoPlayer->Pause();
	return FReply::Handled();
}

FReply FCommonVideoPlayerCustomization::HandleReverseClicked()
{
	if (VideoPlayer->IsPlaying() && VideoPlayer->GetPlaybackRate() < 0.f)
	{
		VideoPlayer->Pause();
	}
	else
	{
		VideoPlayer->Reverse();
	}

	return FReply::Handled();
}

EPlaybackMode::Type FCommonVideoPlayerCustomization::GetPlaybackMode() const
{
	if (VideoPlayer->IsPlaying())
	{
		return VideoPlayer->GetPlaybackRate() > 0.f ? EPlaybackMode::PlayingForward : EPlaybackMode::PlayingReverse;
	}
	return EPlaybackMode::Stopped;
}

TOptional<float> FCommonVideoPlayerCustomization::GetMaxPlaybackTimeValue() const
{
	return VideoPlayer->GetVideoDuration();
}

TOptional<float> FCommonVideoPlayerCustomization::GetPlaybackTimeValue() const
{
	return VideoPlayer->GetPlaybackTime();
}

void FCommonVideoPlayerCustomization::HandlePlaybackTimeCommitted(float NewTime, ETextCommit::Type)
{
	VideoPlayer->Seek(NewTime);
	VideoPlayer->Pause();
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
}

FText FCommonVideoPlayerCustomization::GetMaxPlaybackTimeText() const
{
	return FText::FromString(FString::Printf(TEXT(" / %.1f"), VideoPlayer->GetVideoDuration()));
}

TSharedRef<SWidget> FCommonVideoPlayerCustomization::HandleCreateMuteToggleWidget() const
{
	return SNew(SButton)
		.OnClicked(this, &FCommonVideoPlayerCustomization::HandleToggleMuteClicked)
		.ButtonStyle(FAppStyle::Get(), TEXT("NoBorder"))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(this, &FCommonVideoPlayerCustomization::GetMuteToggleIcon)
		];
}

const FSlateBrush* FCommonVideoPlayerCustomization::GetMuteToggleIcon() const
{
	static const FVolumeControlStyle VolumeStyle = FAppStyle::GetWidgetStyle<FVolumeControlStyle>(TEXT("VolumeControl"));
	return VideoPlayer->IsMuted() ? &VolumeStyle.MutedImage : &VolumeStyle.HighVolumeImage;
}

FReply FCommonVideoPlayerCustomization::HandleToggleMuteClicked() const
{
	VideoPlayer->SetIsMuted(!VideoPlayer->IsMuted());
	return FReply::Handled();
}
