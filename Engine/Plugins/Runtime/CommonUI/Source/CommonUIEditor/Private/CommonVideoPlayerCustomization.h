// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "ITransportControl.h"

template <typename OptionalType> struct TOptional;

class IPropertyHandle;
class UCommonVideoPlayer;

class FCommonVideoPlayerCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	FReply HandlePlayClicked();
	FReply HandleGoToStartClicked();
	FReply HandleBackwardStep();
	FReply HandleForwardStep();
	FReply HandleGoToEndClicked();
	FReply HandlePauseClicked();
	FReply HandleReverseClicked();
	EPlaybackMode::Type GetPlaybackMode() const;

	TOptional<float> GetMaxPlaybackTimeValue() const;
	TOptional<float> GetPlaybackTimeValue() const;
	void HandlePlaybackTimeCommitted(float NewTime, ETextCommit::Type);
	FText GetMaxPlaybackTimeText() const;

	TSharedRef<SWidget> HandleCreateMuteToggleWidget() const;
	const FSlateBrush* GetMuteToggleIcon() const;
	FReply HandleToggleMuteClicked() const;

	TWeakObjectPtr<UCommonVideoPlayer> VideoPlayer;
};
