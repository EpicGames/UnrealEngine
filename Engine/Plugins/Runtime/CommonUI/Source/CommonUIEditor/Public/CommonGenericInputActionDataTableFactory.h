// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "CommonGenericInputActionDataTableFactory.generated.h"

UCLASS()
class UCommonGenericInputActionDataTableFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
// End of UFactory interface

};
